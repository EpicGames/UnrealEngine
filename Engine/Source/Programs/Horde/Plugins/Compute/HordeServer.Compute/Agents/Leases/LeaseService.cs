// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Telemetry;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Service that emits telemetry when leases complete.
	/// </summary>
	public class LeaseService(
		LeaseCollection leaseCollection,
		ITelemetryWriter telemetryWriter,
		IOptionsMonitor<ComputeConfig> computeConfig,
		ILogger<LeaseService> logger)
		: IHostedService
	{
		private readonly ILogger _logger = logger;

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			leaseCollection.OnLeaseComplete += OnLeaseComplete;
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken)
		{
			leaseCollection.OnLeaseComplete -= OnLeaseComplete;
			return Task.CompletedTask;
		}

		private void OnLeaseComplete(ILease lease)
		{
			if (!telemetryWriter.Enabled)
			{
				return;
			}

			EpicGames.Horde.Telemetry.TelemetryStoreId telemetryStoreId = computeConfig.CurrentValue.TelemetryStoreId;
			if (telemetryStoreId.IsEmpty)
			{
				return;
			}

			try
			{
				telemetryWriter.WriteEvent(telemetryStoreId, new LeaseCompleteTelemetry(lease));
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to write lease completion telemetry for lease {LeaseId}", lease.Id);
			}
		}
	}
}
