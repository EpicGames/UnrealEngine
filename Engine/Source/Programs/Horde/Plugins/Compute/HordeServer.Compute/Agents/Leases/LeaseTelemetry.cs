// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Telemetry record emitted when a lease completes. Models an <see cref="ILease"/>.
	/// </summary>
	[AnalyticsTableGen]
	[Table("ingest.horde.state_lease_complete")]
	public record LeaseCompleteTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The lease id.
		/// </summary>
		[Column("lease_id")]
		public LeaseId LeaseId { get; init; }

		/// <summary>
		/// The parent lease id, if any.
		/// </summary>
		[Column("parent_id")]
		public LeaseId? ParentId { get; init; }

		/// <summary>
		/// Name of the lease.
		/// </summary>
		[Column("name")]
		public string Name { get; init; }

		/// <summary>
		/// The agent that executed the lease.
		/// </summary>
		[Column("agent_id")]
		public AgentId AgentId { get; init; }

		/// <summary>
		/// The session in which the lease ran.
		/// </summary>
		[Column("session_id")]
		public SessionId SessionId { get; init; }

		/// <summary>
		/// The stream associated with the lease, if any.
		/// </summary>
		[Column("stream_id")]
		public StreamId? StreamId { get; init; }

		/// <summary>
		/// The pool associated with the lease, if any.
		/// </summary>
		[Column("pool_id")]
		public PoolId? PoolId { get; init; }

		/// <summary>
		/// The log for this lease, if any.
		/// </summary>
		[Column("log_id")]
		public LogId? LogId { get; init; }

		/// <summary>
		/// Time at which the lease started.
		/// </summary>
		[Column("start_time_utc")]
		public DateTime StartTimeUtc { get; init; }

		/// <summary>
		/// Time at which the lease finished.
		/// </summary>
		[Column("finish_time_utc")]
		public DateTime FinishTimeUtc { get; init; }

		/// <summary>
		/// Duration of the lease in seconds.
		/// </summary>
		[Column("duration_secs")]
		public double DurationSecs { get; init; }

		/// <summary>
		/// Outcome of the lease.
		/// </summary>
		[Column("outcome")]
		public LeaseOutcome Outcome { get; init; }

		/// <summary>
		/// Reason for the outcome.
		/// </summary>
		[Column("outcome_reason")]
		public LeaseOutcomeReason OutcomeReason { get; init; }

		/// <summary>
		/// Time spent in setup, in seconds.
		/// </summary>
		[Column("setup_time_secs")]
		public double? SetupTimeSecs { get; init; }

		/// <summary>
		/// Time spent in teardown, in seconds.
		/// </summary>
		[Column("teardown_time_secs")]
		public double? TeardownTimeSecs { get; init; }

		/// <summary>
		/// Number of logical CPU cores available.
		/// </summary>
		[Column("cpu_count")]
		public int? CpuCount { get; init; }

		/// <summary>
		/// Machine-wide CPU utilization during the lease.
		/// </summary>
		[Column("global_cpu_utilization")]
		public double? GlobalCpuUtilization { get; init; }

		/// <summary>
		/// Machine-wide user CPU time in seconds.
		/// </summary>
		[Column("cpu_user_time_secs")]
		public double? CpuUserTimeSecs { get; init; }

		/// <summary>
		/// Machine-wide system CPU time in seconds.
		/// </summary>
		[Column("cpu_system_time_secs")]
		public double? CpuSystemTimeSecs { get; init; }

		/// <summary>
		/// Machine-wide idle CPU time in seconds.
		/// </summary>
		[Column("cpu_idle_time_secs")]
		public double? CpuIdleTimeSecs { get; init; }

		/// <summary>
		/// Default event name for this telemetry record.
		/// </summary>
		public const string DefaultEventName = "State.LeaseComplete";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// Constructs a lease completion telemetry event from a completed lease.
		/// </summary>
		/// <param name="lease">The completed lease.</param>
		public LeaseCompleteTelemetry(ILease lease) : base(DefaultEventName, CurrentSchemaVersion)
		{
			LeaseId = lease.Id;
			ParentId = lease.ParentId;
			Name = lease.Name;
			AgentId = lease.AgentId;
			SessionId = lease.SessionId;
			StreamId = lease.StreamId;
			PoolId = lease.PoolId;
			LogId = lease.LogId;
			StartTimeUtc = lease.StartTime;
			FinishTimeUtc = lease.FinishTime ?? lease.StartTime;
			DurationSecs = (FinishTimeUtc - StartTimeUtc).TotalSeconds;
			Outcome = lease.Outcome;
			OutcomeReason = lease.OutcomeReason;
			SetupTimeSecs = lease.Metrics?.SetupTime.TotalSeconds;
			TeardownTimeSecs = lease.Metrics?.TeardownTime.TotalSeconds;
			CpuCount = lease.Metrics?.CpuCount;
			GlobalCpuUtilization = lease.Metrics?.GlobalCpuUtilization;
			CpuUserTimeSecs = lease.Metrics?.GlobalCpuTime.User.TotalSeconds;
			CpuSystemTimeSecs = lease.Metrics?.GlobalCpuTime.System.TotalSeconds;
			CpuIdleTimeSecs = lease.Metrics?.GlobalCpuTime.Idle.TotalSeconds;
		}

		/// <summary>
		/// Default constructor for ORM mapping.
		/// </summary>
		public LeaseCompleteTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			Name = String.Empty;
		}
	}
}
