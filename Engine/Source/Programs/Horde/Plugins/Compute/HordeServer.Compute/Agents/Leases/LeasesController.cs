// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using Google.Protobuf.WellKnownTypes;
using HordeServer.Server;
using HordeServer.Tasks;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Controller for the /api/v1/leases endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class LeasesController : HordeControllerBase
	{
		readonly AgentService _agentService;
		readonly IEnumerable<ITaskSource> _taskSources;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		readonly Tracer _tracer;

		/// <summary>
		/// Constructor
		/// </summary>
		public LeasesController(AgentService agentService, IEnumerable<ITaskSource> taskSources, IOptionsSnapshot<ComputeConfig> computeConfig, Tracer tracer)
		{
			_agentService = agentService;
			_taskSources = taskSources;
			_computeConfig = computeConfig;
			_tracer = tracer;
		}

		/// <summary>
		/// Find all the leases for a particular agent
		/// </summary>
		/// <param name="agentId">Unique id of the agent to find</param>
		/// <param name="sessionId">The session to query</param>
		/// <param name="startTime">Start of the time window to consider</param>
		/// <param name="finishTime">End of the time window to consider</param>
		/// <param name="minFinishTime">Start of the time window to consider when querying by finish time (if set, other criteria are ignored)</param>
		/// <param name="maxFinishTime">End of the time window to consider when querying by finish time (if set, other criteria are ignored)</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">Filter to apply to the results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sessions </returns>
		[HttpGet]
		[Route("/api/v1/leases")]
		[ProducesResponseType(200, Type = typeof(List<GetAgentLeaseResponse>))]
		public async Task<ActionResult<List<object>>> FindLeasesAsync(
			[FromQuery] AgentId? agentId,
			[FromQuery] SessionId? sessionId,
			[FromQuery] DateTimeOffset? startTime,
			[FromQuery] DateTimeOffset? finishTime,
			[FromQuery] DateTimeOffset? minFinishTime,
			[FromQuery] DateTimeOffset? maxFinishTime,
			[FromQuery] int index = 0,
			[FromQuery] int count = 1000,
			[FromQuery] PropertyFilter? filter = null,
			CancellationToken cancellationToken = default)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeases, User))
			{
				return Forbid(LeaseAclAction.ViewLeases);
			}

			bool includeCosts = _computeConfig.Value.Authorize(ServerAclAction.ViewCosts, User);

			IReadOnlyList<ILease> leases;
			if (minFinishTime == null && maxFinishTime == null)
			{
				leases = await _agentService.FindLeasesAsync(agentId, sessionId, startTime?.UtcDateTime, finishTime?.UtcDateTime, index, count, cancellationToken);
			}
			else
			{
				// Optimized path for queries made by finish time
				leases = await _agentService.FindLeasesByFinishTimeAsync(minFinishTime?.UtcDateTime, maxFinishTime?.UtcDateTime, index, count, cancellationToken);
			}

			List<object> responses = new List<object>();

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(LeasesController)}.{nameof(FindLeasesAsync)}");
			{
				Dictionary<AgentId, double?> cachedAgentRates = new Dictionary<AgentId, double?>();
				foreach (ILease lease in leases)
				{
					double? agentRate = null;
					if (includeCosts && !cachedAgentRates.TryGetValue(lease.AgentId, out agentRate))
					{
						agentRate = await _agentService.GetRateAsync(lease.AgentId, cancellationToken);
						cachedAgentRates.Add(lease.AgentId, agentRate);
					}

					Dictionary<string, string>? details = await _agentService.GetPayloadDetailsAsync(lease.Payload, cancellationToken);
					responses.Add(PropertyFilter.Apply(AgentsController.CreateGetAgentLeaseResponse(lease, details, agentRate), filter));
				}
			}

			return responses;
		}

		/// <summary>
		/// Get info about a particular lease
		/// </summary>
		/// <param name="leaseId">Unique id of the particular lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Lease matching the given id</returns>
		[HttpGet]
		[Route("/api/v1/leases/{leaseId}")]
		public async Task<ActionResult<GetAgentLeaseResponse>> GetLeaseAsync(LeaseId leaseId, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeases, User))
			{
				return Forbid(LeaseAclAction.ViewLeases);
			}

			ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
			if (lease == null)
			{
				return NotFound(leaseId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(lease.AgentId, cancellationToken);
			double? agentRate = null;
			if (_computeConfig.Value.Authorize(ServerAclAction.ViewCosts, User) && agent != null)
			{
				agentRate = await _agentService.GetRateAsync(agent.Id, cancellationToken);
			}

			Dictionary<string, string>? details = await _agentService.GetPayloadDetailsAsync(lease.Payload, cancellationToken);
			return AgentsController.CreateGetAgentLeaseResponse(lease, details, agentRate);
		}

		/// <summary>
		/// Gets the protobuf task descriptor  for a lease
		/// </summary>
		/// <param name="leaseId">Unique id of the particular lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Lease matching the given id</returns>
		[HttpGet]
		[Route("/api/v1/leases/{leaseId}/task")]
		public async Task<ActionResult<object>> GetLeaseTaskAsync(LeaseId leaseId, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeaseTasks, User))
			{
				return Forbid(LeaseAclAction.ViewLeaseTasks);
			}

			ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
			if (lease == null)
			{
				return NotFound(leaseId);
			}

			Any any = lease.GetTask();

			object? decoded = null;
			foreach (ITaskSource taskSource in _taskSources)
			{
				if (any.Is(taskSource.Descriptor))
				{
					decoded = taskSource.Descriptor.Parser.ParseFrom(any.Value);
				}
			}

			return new { type = any.TypeUrl, content = any.Value.ToBase64(), decoded };
		}

		/// <summary>
		/// Get lease log, redirecting from lease id to log id
		/// </summary>
		/// <param name="leaseId">Unique id of the particular lease</param>
		/// <param name="path">Subresource for the lease's log</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Redirect to log endpoint</returns>
		[HttpGet]
		[Route("/api/v1/leases/{leaseId}/log/{*path}")]
		public async Task<ActionResult> GetLeaseLogAsync(LeaseId leaseId, string? path = null, CancellationToken cancellationToken = default)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeases, User))
			{
				return Forbid(LeaseAclAction.ViewLeases);
			}

			ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
			if (lease == null)
			{
				return NotFound(leaseId);
			}

			if (lease.LogId == null)
			{
				return NotFound("null lease.LogId");
			}

			return new RedirectResult($"/api/v1/logs/{lease.LogId}/{path}");
		}

		/// <summary>
		/// Update a particular lease
		/// </summary>
		/// <param name="leaseId">Unique id of the particular lease</param>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Lease matching the given id</returns>
		[HttpPut]
		[Route("/api/v1/leases/{leaseId}")]
		public async Task<ActionResult> UpdateLeaseAsync(LeaseId leaseId, [FromBody] UpdateLeaseRequest request, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			// only update supported right now is abort
			if (!request.Aborted.HasValue || !request.Aborted.Value)
			{
				return Ok();
			}

			ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
			if (lease == null)
			{
				return NotFound(leaseId);
			}

			await _agentService.CancelLeaseAsync(lease, LeaseOutcomeReason.UserInitiated, cancellationToken);
			return Ok();
		}
	}
}
