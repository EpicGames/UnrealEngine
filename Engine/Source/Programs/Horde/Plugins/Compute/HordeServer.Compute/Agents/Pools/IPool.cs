// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Nodes;
using System.Text.Json.Serialization;
using EpicGames.Horde.Common;
using HordeServer.Agents.Fleet;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Metadata for configuring and picking a pool sizing strategy
	/// </summary>
	public class PoolSizeStrategyInfo
	{
		/// <summary>
		/// Strategy implementation to use
		/// </summary>
		public PoolSizeStrategy Type { get; set; }

		/// <summary>
		/// Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Configuration for the strategy, serialized as JSON
		/// </summary>
		[JsonConverter(typeof(JsonObjectOrStringConverter))]
		public JsonObject Config { get; set; } = new JsonObject();

		/// <summary>
		/// Integer to add after pool size has been calculated. Can also be negative.
		/// </summary>
		public int ExtraAgentCount { get; set; } = 0;

		/// <summary>
		/// Empty constructor for JSON serialization
		/// </summary>
		[JsonConstructor]
		public PoolSizeStrategyInfo()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolSizeStrategyInfo(PoolSizeStrategy type, Condition? condition, int extraAgentCount = 0)
		{
			Type = type;
			Condition = condition;
			ExtraAgentCount = extraAgentCount;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolSizeStrategyInfo(PoolSizeStrategy type, Condition? condition, string config, int extraAgentCount = 0)
		{
			Type = type;
			Condition = condition;
			if (String.IsNullOrEmpty(config))
			{
				Config = new JsonObject();
			}
			else
			{
				Config = (JsonObject)JsonNode.Parse(config, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip })!;
			}
			ExtraAgentCount = extraAgentCount;
		}
	}

	/// <summary>
	/// Metadata for configuring and picking a fleet manager
	/// </summary>
	public class FleetManagerInfo
	{
		/// <summary>
		/// Fleet manager type implementation to use
		/// </summary>
		public FleetManagerType Type { get; set; }

		/// <summary>
		/// Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Configuration for the strategy, serialized as JSON
		/// </summary>
		[JsonConverter(typeof(JsonObjectOrStringConverter))]
		public JsonObject? Config { get; set; }

		/// <summary>
		/// Empty constructor for BSON/JSON serialization
		/// </summary>
		public FleetManagerInfo()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		[JsonConstructor]
		public FleetManagerInfo(FleetManagerType type, Condition? condition)
		{
			Type = type;
			Condition = condition;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FleetManagerInfo(FleetManagerType type, Condition? condition, string config)
		{
			Type = type;
			Condition = condition;
			if (String.IsNullOrEmpty(config))
			{
				Config = new JsonObject();
			}
			else
			{
				Config = (JsonObject?)JsonNode.Parse(config, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true });
			}
		}
	}

	/// <summary>
	/// Allows parsing a string or json object into a native JsonObject instance
	/// </summary>
	class JsonObjectOrStringConverter : JsonConverter<JsonObject>
	{
		static readonly JsonNodeOptions s_nodeOptions = new JsonNodeOptions { PropertyNameCaseInsensitive = true };
		static readonly JsonDocumentOptions s_documentOptions = new JsonDocumentOptions { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip };

		/// <inheritdoc/>
		public override JsonObject? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			JsonObject? result = null;
			if (reader.TokenType == JsonTokenType.String)
			{
				result = JsonNode.Parse(reader.GetString() ?? "{}", s_nodeOptions, s_documentOptions) as JsonObject;
			}
			else if (reader.TokenType == JsonTokenType.StartObject)
			{
				result = JsonNode.Parse(ref reader, s_nodeOptions) as JsonObject;
			}
			return result;
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, JsonObject value, JsonSerializerOptions options)
			=> JsonSerializer.Serialize(writer, value, options);
	}

	/// <summary>
	/// Pool of machines
	/// </summary>
	public interface IPool : IPoolConfig
	{
		/// <summary>
		/// Last time the pool was (auto) scaled up
		/// </summary>
		public DateTime? LastScaleUpTime { get; }

		/// <summary>
		/// Last time the pool was (auto) scaled down
		/// </summary>
		public DateTime? LastScaleDownTime { get; }

		/// <summary>
		/// Last known agent count
		/// </summary>
		public int? LastAgentCount { get; }

		/// <summary>
		/// Last known desired agent count
		/// </summary>
		public int? LastDesiredAgentCount { get; }

		/// <summary>
		/// Last result from scaling the pool
		/// </summary>
		public ScaleResult? LastScaleResult { get; }

		/// <summary>
		/// Updates a pool
		/// </summary>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IPool?> TryUpdateAsync(UpdatePoolOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Settings for updating a pool state
	/// </summary>
	/// <param name="LastScaleUpTime"> New time for last (auto) scale up</param>
	/// <param name="LastScaleDownTime"> New time for last (auto) scale down</param>
	/// <param name="LastScaleResult"> Result from last scaling in/out attempt</param>
	/// <param name="LastAgentCount"> Last calculated agent count</param>
	/// <param name="LastDesiredAgentCount"> Last calculated desired agent count</param>
	public record UpdatePoolOptions(DateTime? LastScaleUpTime = null, DateTime? LastScaleDownTime = null, ScaleResult? LastScaleResult = null, int? LastAgentCount = null, int? LastDesiredAgentCount = null);
}
