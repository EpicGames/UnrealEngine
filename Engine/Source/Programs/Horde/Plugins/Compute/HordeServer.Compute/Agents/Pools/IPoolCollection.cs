// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using HordeServer.Agents.Fleet;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Collection of pool documents
	/// </summary>
	public interface IPoolCollection
	{
		/// <summary>
		/// Creates a new pool
		/// </summary>
		/// <param name="id">Unique id for the new pool</param>
		/// <param name="name">Name of the new pool</param>
		/// <param name="options">Options for the new pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new pool document</returns>
		[Obsolete("Pools should be configured through globals.json")]
		Task CreateConfigAsync(PoolId id, string name, CreatePoolConfigOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Enumerates all the pools
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The pool documents</returns>
		Task<IReadOnlyList<IPoolConfig>> GetConfigsAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a pool by ID
		/// </summary>
		/// <param name="id">Unique id of the pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The pool document</returns>
		Task<IPool?> GetAsync(PoolId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a list of all valid pool ids
		/// </summary>
		/// <returns>List of pool ids</returns>
		Task<List<PoolId>> GetPoolIdsAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a pool by ID
		/// </summary>
		/// <param name="id">Unique id of the pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The pool document</returns>
		[Obsolete("Pools should be configured through globals.json")]
		Task<bool> DeleteConfigAsync(PoolId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates a pool
		/// </summary>
		/// <param name="poolId">The pool to update</param>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[Obsolete("Pools should be configured through globals.json")]
		Task<bool> UpdateConfigAsync(PoolId poolId, UpdatePoolConfigOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Settings for creating a new pool
	/// </summary>
	public class CreatePoolConfigOptions
	{
		/// <summary>
		/// Condition for agents to be included in this pool
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Whether to enable autoscaling for this pool
		/// </summary>
		public bool? EnableAutoscaling { get; set; }

		/// <summary>
		/// Minimum number of agents in the pool
		/// </summary>
		public int? MinAgents { get; set; }

		/// <summary>
		/// Minimum number of idle agents to maintain
		/// </summary>
		public int? NumReserveAgents { get; set; }

		/// <summary>
		/// Interval between conforms. Set to zero to disable.
		/// </summary>
		public TimeSpan? ConformInterval { get; set; }

		/// <summary>
		/// Cooldown time between scale-out events
		/// </summary>
		public TimeSpan? ScaleOutCooldown { get; set; }

		/// <summary>
		/// Cooldown time between scale-in events
		/// </summary>
		public TimeSpan? ScaleInCooldown { get; set; }

		/// <summary>
		/// Pool sizing strategies
		/// </summary>
		public List<PoolSizeStrategyInfo>? SizeStrategies { get; set; }

		/// <summary>
		/// Fleet managers
		/// </summary>
		public List<FleetManagerInfo>? FleetManagers { get; set; }

		/// <summary>
		/// Pool sizing strategy
		/// </summary>
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <summary>
		/// Settings for lease utilization strategy
		/// </summary>
		public LeaseUtilizationSettings? LeaseUtilizationSettings { get; set; }

		/// <summary>
		/// Settings for job queue strategy
		/// </summary>
		public JobQueueSettings? JobQueueSettings { get; set; }

		/// <summary>
		/// Settings for compute queue AWS metric strategy
		/// </summary>
		public ComputeQueueAwsMetricSettings? ComputeQueueAwsMetricSettings { get; set; }

		/// <summary>
		/// Properties for the pool
		/// </summary>
		public IEnumerable<KeyValuePair<string, string>>? Properties { get; set; }
	}

	/// <summary>
	/// Settings for updating a pool state
	/// </summary>
	public class UpdatePoolConfigOptions
	{
		/// <summary>
		/// New name for the pool
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// New condition for the pool
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// New setting for whether to enable autoscaling
		/// </summary>
		public bool? EnableAutoscaling { get; set; }

		/// <summary>
		/// Minimum number of agents in the pool
		/// </summary>
		public int? MinAgents { get; set; }

		/// <summary>
		/// Minimum number of idle agents to maintain
		/// </summary>
		public int? NumReserveAgents { get; set; }

		/// <summary>
		/// New workspaces for the pool
		/// </summary>
		public List<AgentWorkspaceInfo>? Workspaces { get; set; }

		/// <summary>
		/// Settings for the autosdk workspace
		/// </summary>
		public AutoSdkConfig? AutoSdkConfig { get; set; }

		/// <summary>
		/// New properties for the pool
		/// </summary>
		public Dictionary<string, string?>? Properties { get; set; }

		/// <summary>
		/// Interval between conforms. Set to zero to disable.
		/// </summary>
		public TimeSpan? ConformInterval { get; set; }

		/// <summary>
		/// Cooldown time between scale-out events
		/// </summary>
		public TimeSpan? ScaleOutCooldown { get; set; }

		/// <summary>
		/// Cooldown time between scale-in events
		/// </summary>
		public TimeSpan? ScaleInCooldown { get; set; }

		/// <summary>
		/// Time to wait before shutting down a disabled agent
		/// </summary>
		public TimeSpan? ShutdownIfDisabledGracePeriod { get; set; }

		/// <summary>
		/// Pool sizing strategy
		/// </summary>
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <summary>
		/// List of pool sizing strategies
		/// </summary>
		public List<PoolSizeStrategyInfo>? SizeStrategies { get; set; }

		/// <summary>
		/// List of fleet managers
		/// </summary>
		public List<FleetManagerInfo>? FleetManagers { get; set; }

		/// <summary>
		/// Settings for lease utilization strategy
		/// </summary>
		public LeaseUtilizationSettings? LeaseUtilizationSettings { get; set; }

		/// <summary>
		/// Settings for job queue strategy
		/// </summary>
		public JobQueueSettings? JobQueueSettings { get; set; }

		/// <summary>
		/// Settings for compute queue AWS metric strategy
		/// </summary>
		public ComputeQueueAwsMetricSettings? ComputeQueueAwsMetricSettings { get; set; }

		/// <summary>
		/// Whether to use the default strategy
		/// </summary>
		public bool? UseDefaultStrategy { get; set; }
	}

	/// <summary>
	/// Exception when conflicting definitions for a pool are encountered
	/// </summary>
	public sealed class PoolConflictException : Exception
	{
		/// <summary>
		/// Pool with conflicting definitions
		/// </summary>
		public PoolId PoolId { get; }

		/// <summary>
		/// The first revision string
		/// </summary>
		public string PrevRevision { get; }

		/// <summary>
		/// The second revision string
		/// </summary>
		public string NextRevision { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolConflictException(PoolId poolId, string prevRevision, string nextRevision)
			: base($"Duplicate definitions for {poolId} - first in {prevRevision}, now {nextRevision}")
		{
			PoolId = poolId;
			PrevRevision = prevRevision;
			NextRevision = nextRevision;
		}
	}
}
