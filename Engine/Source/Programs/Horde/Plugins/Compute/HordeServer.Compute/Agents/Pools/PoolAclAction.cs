// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// ACL actions for pools
	/// </summary>
	public static class PoolAclAction
	{
		/// <summary>
		/// Create a global pool of agents
		/// </summary>
		public static AclAction CreatePool { get; } = new AclAction("CreatePool");

		/// <summary>
		/// Modify an agent pool
		/// </summary>
		public static AclAction UpdatePool { get; } = new AclAction("UpdatePool");

		/// <summary>
		/// Delete an agent pool
		/// </summary>
		public static AclAction DeletePool { get; } = new AclAction("DeletePool");

		/// <summary>
		/// Ability to view a pool
		/// </summary>
		public static AclAction ViewPool { get; } = new AclAction("ViewPool");

		/// <summary>
		/// View all the available agent pools
		/// </summary>
		public static AclAction ListPools { get; } = new AclAction("ListPools");
	}
}
