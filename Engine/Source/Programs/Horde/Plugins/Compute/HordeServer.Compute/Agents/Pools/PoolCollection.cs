// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using HordeServer.Agents.Fleet;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Collection of pool documents
	/// </summary>
	public class PoolCollection : IPoolCollection
	{
		// Legacy document stores config info in the database
		class PoolDocumentV1 : IPoolConfig
		{
			[BsonId]
			public PoolId Id { get; set; }

			[BsonRequired]
			public string Name { get; set; } = null!;

			[BsonIgnoreIfNull]
			public Condition? Condition { get; set; }

			public List<AgentWorkspaceInfo> Workspaces { get; set; } = new List<AgentWorkspaceInfo>();

			PoolColor IPoolConfig.Color => PoolColor.Default;
			IReadOnlyList<AgentWorkspaceInfo> IPoolConfig.Workspaces => Workspaces;

			[BsonIgnoreIfNull]
			public AutoSdkConfig? AutoSdkConfig { get; set; }

			public Dictionary<string, string> Properties { get; set; } = new Dictionary<string, string>();

			[BsonIgnoreIfDefault, BsonDefaultValue(true)]
			public bool EnableAutoscaling { get; set; } = true;

			[BsonIgnoreIfNull]
			public int? MinAgents { get; set; }

			[BsonIgnoreIfNull]
			public int? NumReserveAgents { get; set; }

			[BsonIgnoreIfNull]
			public TimeSpan? ConformInterval { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? LastScaleUpTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? LastScaleDownTime { get; set; }

			[BsonIgnoreIfNull]
			public TimeSpan? ScaleOutCooldown { get; set; }

			[BsonIgnoreIfNull]
			public TimeSpan? ScaleInCooldown { get; set; }

			[BsonIgnoreIfNull]
			public TimeSpan? ShutdownIfDisabledGracePeriod { get; set; }

			[BsonIgnoreIfNull]
			public ScaleResult? LastScaleResult { get; set; }

			[BsonIgnoreIfNull]
			public int? LastAgentCount { get; set; }

			[BsonIgnoreIfNull]
			public int? LastDesiredAgentCount { get; set; }

			[BsonIgnoreIfNull]
			public PoolSizeStrategy? SizeStrategy { get; set; }

			[BsonIgnoreIfNull]
			public List<PoolSizeStrategyInfo>? SizeStrategies { get; set; }

			[BsonIgnoreIfNull]
			public List<FleetManagerInfo>? FleetManagers { get; set; }

			[BsonIgnoreIfNull]
			public LeaseUtilizationSettings? LeaseUtilizationSettings { get; set; }

			[BsonIgnoreIfNull]
			public JobQueueSettings? JobQueueSettings { get; set; }

			[BsonIgnoreIfNull]
			public ComputeQueueAwsMetricSettings? ComputeQueueAwsMetricSettings { get; set; }

			[BsonIgnoreIfNull]
			public string? Revision { get; set; }

			public int UpdateIndex { get; set; }

			IReadOnlyDictionary<string, string> IPoolConfig.Properties => Properties;
			IReadOnlyList<PoolSizeStrategyInfo> IPoolConfig.SizeStrategies => (IReadOnlyList<PoolSizeStrategyInfo>?)SizeStrategies ?? Array.Empty<PoolSizeStrategyInfo>();
			IReadOnlyList<FleetManagerInfo> IPoolConfig.FleetManagers => (IReadOnlyList<FleetManagerInfo>?)FleetManagers ?? Array.Empty<FleetManagerInfo>();

			public PoolDocumentV1()
			{
			}

			public PoolDocumentV1(IPoolConfig other, string? otherRevision)
			{
				Id = other.Id;
				Name = String.IsNullOrEmpty(other.Name) ? other.Id.ToString() : other.Name;
				Condition = other.Condition;
				if (other.Properties != null && other.Properties.Count > 0)
				{
					Properties = new Dictionary<string, string>(other.Properties);
				}
				EnableAutoscaling = other.EnableAutoscaling;
				MinAgents = other.MinAgents;
				NumReserveAgents = other.NumReserveAgents;
				ConformInterval = other.ConformInterval;
#pragma warning disable CS0618 // Type or member is obsolete
				SizeStrategy = other.SizeStrategy;
#pragma warning restore CS0618 // Type or member is obsolete
				if (other.SizeStrategies != null && other.SizeStrategies.Count > 0)
				{
					SizeStrategies = new List<PoolSizeStrategyInfo>(other.SizeStrategies);
				}
				if (other.FleetManagers != null && other.FleetManagers.Count > 0)
				{
					FleetManagers = new List<FleetManagerInfo>(other.FleetManagers);
				}
				LeaseUtilizationSettings = other.LeaseUtilizationSettings;
				JobQueueSettings = other.JobQueueSettings;
				Revision = otherRevision;
			}
		}

		class PoolDocumentV2
		{
			[BsonId]
			public PoolId Id { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? LastScaleUpTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? LastScaleDownTime { get; set; }

			[BsonIgnoreIfNull]
			public ScaleResult? LastScaleResult { get; set; }

			[BsonIgnoreIfNull]
			public int? LastAgentCount { get; set; }

			[BsonIgnoreIfNull]
			public int? LastDesiredAgentCount { get; set; }

			public int UpdateIndex { get; set; }
		}

		class Pool : IPool
		{
			readonly PoolCollection _collection;
			readonly IPoolConfig _config;
			readonly PoolDocumentV2 _document;

			public Pool(PoolCollection collection, IPoolConfig config, PoolDocumentV2 document)
			{
				_collection = collection;
				_config = config;
				_document = document;
			}

			public PoolId Id => _config.Id;
			public DateTime? LastScaleUpTime => _document.LastScaleUpTime;
			public DateTime? LastScaleDownTime => _document.LastScaleDownTime;
			public int? LastAgentCount => _document.LastAgentCount;
			public int? LastDesiredAgentCount => _document.LastDesiredAgentCount;
			public ScaleResult? LastScaleResult => _document.LastScaleResult;
			public int UpdateIndex => _document.UpdateIndex;

			#region Config accessors

			public string Name => _config.Name;
			public Condition? Condition => _config.Condition;
			public IReadOnlyDictionary<string, string>? Properties => _config.Properties;
			public PoolColor Color => _config.Color;
			public bool EnableAutoscaling => _config.EnableAutoscaling;
			public IReadOnlyList<AgentWorkspaceInfo> Workspaces => _config.Workspaces;
			public AutoSdkConfig? AutoSdkConfig => _config.AutoSdkConfig;
			public TimeSpan? ScaleOutCooldown => _config.ScaleOutCooldown;
			public TimeSpan? ScaleInCooldown => _config.ScaleInCooldown;
			public int? MinAgents => _config.MinAgents;
			public int? NumReserveAgents => _config.NumReserveAgents;
			public TimeSpan? ConformInterval => _config.ConformInterval;
			public TimeSpan? ShutdownIfDisabledGracePeriod => _config.ShutdownIfDisabledGracePeriod;

			[Obsolete]
			public PoolSizeStrategy? SizeStrategy => _config.SizeStrategy;

			public IReadOnlyList<PoolSizeStrategyInfo>? SizeStrategies => _config.SizeStrategies;
			public IReadOnlyList<FleetManagerInfo>? FleetManagers => _config.FleetManagers;
			public LeaseUtilizationSettings? LeaseUtilizationSettings => _config.LeaseUtilizationSettings;
			public JobQueueSettings? JobQueueSettings => _config.JobQueueSettings;

			#endregion

			public async Task<IPool?> TryUpdateAsync(UpdatePoolOptions options, CancellationToken cancellationToken)
			{
				PoolDocumentV2? newDocument = await _collection.UpdateStateAsync(_document, options, cancellationToken);
				if (newDocument == null)
				{
					return null;
				}
				return new Pool(_collection, _config, newDocument);
			}
		}

		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly IMongoCollection<PoolDocumentV1> _poolsV1;
		readonly IMongoCollection<PoolDocumentV2> _poolsV2;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service instance</param>
		/// <param name="computeConfig"></param>
		public PoolCollection(IMongoService mongoService, IOptionsMonitor<ComputeConfig> computeConfig)
		{
			_poolsV1 = mongoService.GetCollection<PoolDocumentV1>("Pools");
			_poolsV2 = mongoService.GetCollection<PoolDocumentV2>("PoolsV2");
			_computeConfig = computeConfig;
		}

		/// <inheritdoc/>
		public async Task CreateConfigAsync(PoolId id, string name, CreatePoolConfigOptions options, CancellationToken cancellationToken = default)
		{
			PoolDocumentV1 pool = new PoolDocumentV1();
			pool.Id = id;
			pool.Name = name;
			pool.Condition = options.Condition;
			if (options.EnableAutoscaling != null)
			{
				pool.EnableAutoscaling = options.EnableAutoscaling.Value;
			}
			pool.MinAgents = options.MinAgents;
			pool.NumReserveAgents = options.NumReserveAgents;
			if (options.Properties != null)
			{
				pool.Properties = new Dictionary<string, string>(options.Properties);
			}

			pool.ConformInterval = options.ConformInterval;
			pool.ScaleOutCooldown = options.ScaleOutCooldown;
			pool.ScaleInCooldown = options.ScaleInCooldown;
			pool.SizeStrategy = options.SizeStrategy;
			pool.LeaseUtilizationSettings = options.LeaseUtilizationSettings;
			pool.JobQueueSettings = options.JobQueueSettings;
			pool.ComputeQueueAwsMetricSettings = options.ComputeQueueAwsMetricSettings;
			if (options.SizeStrategies != null)
			{
				pool.SizeStrategies = options.SizeStrategies;
			}
			if (options.FleetManagers != null)
			{
				pool.FleetManagers = options.FleetManagers;
			}

			await _poolsV1.InsertOneAsync(pool, null, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IPoolConfig>> GetConfigsAsync(CancellationToken cancellationToken)
		{
			ComputeConfig computeConfig = _computeConfig.CurrentValue;

			List<IPoolConfig> pools = new List<IPoolConfig>();
			pools.AddRange(computeConfig.Pools);
			if (computeConfig.VersionEnum < ConfigVersion.PoolsInConfigFiles)
			{
				// V0/V1: include all DB pools (no dedup — legacy behavior preserved)
				pools.AddRange(await _poolsV1.Find(FilterDefinition<PoolDocumentV1>.Empty).ToListAsync(cancellationToken));
			}

			return pools;
		}

		/// <inheritdoc/>
		public async Task<IPool?> GetAsync(PoolId id, CancellationToken cancellationToken)
		{
			ComputeConfig computeConfig = _computeConfig.CurrentValue;

			IPoolConfig? poolConfig = null;
			if (computeConfig.TryGetPool(id, out PoolConfig? globalPoolConfig))
			{
				poolConfig = globalPoolConfig;
			}
			else if (computeConfig.VersionEnum < ConfigVersion.PoolsInConfigFiles)
			{
				// V0/V1: fall back to database if not found in config
				poolConfig = await _poolsV1.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			}

			if (poolConfig == null)
			{
				return null;
			}

			PoolDocumentV2? state = await _poolsV2.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			if (state == null)
			{
				PoolDocumentV2 newState = new PoolDocumentV2 { Id = id };

				// Try to copy scaling state from V1 document.
				// Use `as` cast so poolV1 is declared in outer scope for both paths.
				PoolDocumentV1? poolV1 = poolConfig as PoolDocumentV1;
				if (poolV1 != null)
				{
					newState.LastScaleUpTime = poolV1.LastScaleUpTime;
					newState.LastScaleDownTime = poolV1.LastScaleDownTime;
					newState.LastScaleResult = poolV1.LastScaleResult;
					newState.LastAgentCount = poolV1.LastAgentCount;
					newState.LastDesiredAgentCount = poolV1.LastDesiredAgentCount;
				}

				if (!await _poolsV2.InsertOneIgnoreDuplicatesAsync(newState, cancellationToken))
				{
					// Another instance created the V2 doc first; read the actual persisted state
					// to ensure UpdateIndex matches for optimistic concurrency in TryUpdateAsync.
					state = await _poolsV2.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken) ?? newState;
				}
				else
				{
					state = newState;
				}
			}

			return new Pool(this, poolConfig, state);
		}

		/// <inheritdoc/>
		public async Task<List<PoolId>> GetPoolIdsAsync(CancellationToken cancellationToken)
		{
			ComputeConfig computeConfig = _computeConfig.CurrentValue;

			List<PoolId> pools = new List<PoolId>();
			pools.AddRange(computeConfig.Pools.Select(x => x.Id));

			if (computeConfig.VersionEnum < ConfigVersion.PoolsInConfigFiles)
			{
				// V0/V1: include all DB pool IDs (no dedup — legacy behavior preserved)
				ProjectionDefinition<PoolDocumentV1, BsonDocument> projection = Builders<PoolDocumentV1>.Projection.Include(x => x.Id);
				List<BsonDocument> results = await _poolsV1.Find(FilterDefinition<PoolDocumentV1>.Empty).Project(projection).ToListAsync(cancellationToken);
				pools.AddRange(results.Select(x => new PoolId(x.GetElement("_id").Value.AsString)));
			}

			return pools;
		}

		/// <inheritdoc/>
		public async Task<bool> DeleteConfigAsync(PoolId id, CancellationToken cancellationToken)
		{
			FilterDefinition<PoolDocumentV1> filter = Builders<PoolDocumentV1>.Filter.Eq(x => x.Id, id);
			DeleteResult result = await _poolsV1.DeleteOneAsync(filter, null, cancellationToken);
			return result.DeletedCount > 0;
		}

		/// <inheritdoc/>
		public async Task<bool> UpdateConfigAsync(PoolId poolId, UpdatePoolConfigOptions options, CancellationToken cancellationToken)
		{
			List<UpdateDefinition<PoolDocumentV1>> updates = new List<UpdateDefinition<PoolDocumentV1>>();
			if (options.Name != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.Name, options.Name));
			}
			if (options.Condition != null)
			{
				if (options.Condition.IsEmpty())
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.Condition!));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.Condition, options.Condition));
				}
			}
			if (options.EnableAutoscaling != null)
			{
				if (options.EnableAutoscaling.Value)
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.EnableAutoscaling));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.EnableAutoscaling, options.EnableAutoscaling.Value));
				}
			}
			if (options.MinAgents != null)
			{
				if (options.MinAgents.Value < 0)
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.MinAgents!));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.MinAgents, options.MinAgents.Value));
				}
			}
			if (options.NumReserveAgents != null)
			{
				if (options.NumReserveAgents.Value < 0)
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.NumReserveAgents!));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.NumReserveAgents, options.NumReserveAgents.Value));
				}
			}
			if (options.Workspaces != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.Workspaces, options.Workspaces));
			}
			if (options.AutoSdkConfig != null)
			{
				if (options.AutoSdkConfig == AutoSdkConfig.None)
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.AutoSdkConfig!));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.AutoSdkConfig, options.AutoSdkConfig));
				}
			}
			if (options.Properties != null)
			{
				foreach ((string key, string? value) in options.Properties)
				{
					if (value == null)
					{
						updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.Properties[key]));
					}
					else
					{
						updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.Properties[key], value));
					}
				}
			}
			if (options.ConformInterval != null)
			{
				if (options.ConformInterval.Value < TimeSpan.Zero)
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Unset(x => x.ConformInterval!));
				}
				else
				{
					updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.ConformInterval, options.ConformInterval));
				}
			}
			if (options.ScaleOutCooldown != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.ScaleOutCooldown, options.ScaleOutCooldown));
			}
			if (options.ScaleInCooldown != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.ScaleInCooldown, options.ScaleInCooldown));
			}
			if (options.ShutdownIfDisabledGracePeriod != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.ShutdownIfDisabledGracePeriod, options.ShutdownIfDisabledGracePeriod));
			}
			if (options.SizeStrategy != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.SizeStrategy, options.SizeStrategy));
			}
			else if (options.UseDefaultStrategy != null && options.UseDefaultStrategy.Value)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.SizeStrategy, null));
			}
			if (options.SizeStrategies != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.SizeStrategies, options.SizeStrategies));
			}
			if (options.FleetManagers != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.FleetManagers, options.FleetManagers));
			}
			if (options.LeaseUtilizationSettings != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.LeaseUtilizationSettings, options.LeaseUtilizationSettings));
			}
			if (options.JobQueueSettings != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.JobQueueSettings, options.JobQueueSettings));
			}
			if (options.ComputeQueueAwsMetricSettings != null)
			{
				updates.Add(Builders<PoolDocumentV1>.Update.Set(x => x.ComputeQueueAwsMetricSettings, options.ComputeQueueAwsMetricSettings));
			}

			if (updates.Count > 0)
			{
				FilterDefinition<PoolDocumentV1> filter = Builders<PoolDocumentV1>.Filter.Expr(x => x.Id == poolId);
				UpdateDefinition<PoolDocumentV1> update = Builders<PoolDocumentV1>.Update.Combine(updates);

				UpdateResult result = await _poolsV1.UpdateOneAsync(filter, update, null, cancellationToken);
				if (result.ModifiedCount == 0)
				{
					return false;
				}
			}

			return true;
		}

		async Task<PoolDocumentV2> UpdateStateAsync(PoolDocumentV2 document, UpdatePoolOptions options, CancellationToken cancellationToken)
		{
			List<UpdateDefinition<PoolDocumentV2>> updates = new List<UpdateDefinition<PoolDocumentV2>>();
			if (options.LastScaleResult != null)
			{
				updates.Add(Builders<PoolDocumentV2>.Update.Set(x => x.LastScaleResult, options.LastScaleResult));
			}
			if (options.LastAgentCount != null)
			{
				updates.Add(Builders<PoolDocumentV2>.Update.Set(x => x.LastAgentCount, options.LastAgentCount));
			}
			if (options.LastDesiredAgentCount != null)
			{
				updates.Add(Builders<PoolDocumentV2>.Update.Set(x => x.LastDesiredAgentCount, options.LastDesiredAgentCount));
			}
			if (options.LastScaleUpTime != null)
			{
				updates.Add(Builders<PoolDocumentV2>.Update.Set(x => x.LastScaleUpTime, options.LastScaleUpTime));
			}
			if (options.LastScaleDownTime != null)
			{
				updates.Add(Builders<PoolDocumentV2>.Update.Set(x => x.LastScaleDownTime, options.LastScaleDownTime));
			}

			FilterDefinition<PoolDocumentV2> filter = Builders<PoolDocumentV2>.Filter.Expr(x => x.Id == document.Id && x.UpdateIndex == document.UpdateIndex);
			UpdateDefinition<PoolDocumentV2> update = Builders<PoolDocumentV2>.Update.Combine(updates).Inc(x => x.UpdateIndex, 1);
			return await _poolsV2.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<PoolDocumentV2, PoolDocumentV2> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}
	}
}
