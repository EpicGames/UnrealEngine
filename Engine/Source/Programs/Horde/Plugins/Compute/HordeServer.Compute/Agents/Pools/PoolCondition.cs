// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using EpicGames.Horde.Common;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Defined pool property names, for use in dashboard filter expressions
	/// </summary>
	static class PoolPropertyNames
	{
		/// <summary>
		/// Identifier for the pool
		/// </summary>
		public const string Id = "Id";

		/// <summary>
		/// Number of agents which are in the pool
		/// </summary>
		public const string NumAgents = "NumAgents";

		/// <summary>
		/// Number of agents in the pool which are online
		/// </summary>
		public const string NumOnline = "NumOnline";

		/// <summary>
		/// Number of agents in the pool which are offline
		/// </summary>
		public const string NumOffline = "NumOffline";

		/// <summary>
		/// Number of agents in the pool which are enabled
		/// </summary>
		public const string NumEnabled = "NumEnabled";

		/// <summary>
		/// Number of agents in the pool which are disabled
		/// </summary>
		public const string NumDisabled = "NumDisabled";

		/// <summary>
		/// Number of agents in the pool which are currently executing work
		/// </summary>
		public const string NumBusy = "NumBusy";

		/// <summary>
		/// Number of agents in the pool which are idle
		/// </summary>
		public const string NumIdle = "NumIdle";

		/// <summary>
		/// Prefix for properties within the pool configuration
		/// </summary>
		public const string PropertiesPrefix = "Properties.";
	}

	/// <summary>
	/// Information about a pool
	/// </summary>
	class PoolInfo
	{
		public int NumAgents { get; set; }
		public int NumIdle { get; set; }
		public int NumBusy { get; set; }
		public int NumOffline { get; set; }
		public int NumDisabled { get; set; }
		public List<IAgent>? Agents { get; set; }
		public List<double>? Utilization { get; set; }

		public void AddAgent(IAgent agent, DateTime utcNow)
		{
			NumAgents++;

			if (!agent.IsSessionValid(utcNow))
			{
				NumOffline++;
			}
			else if (!agent.Enabled)
			{
				NumDisabled++;
			}
			else if (agent.Leases.Count > 0)
			{
				NumBusy++;
			}
			else
			{
				NumIdle++;
			}

			Agents ??= new List<IAgent>();
			Agents.Add(agent);
		}
	}

	/// <summary>
	/// Static methods for querying pools
	/// </summary>
	static class PoolCondition
	{
		/// <summary>
		/// Check whether a pool satisfies a condition
		/// </summary>
		public static bool Evaluate(Condition condition, IPoolConfig poolConfig, PoolInfo? poolInfo)
		{
			return condition.Evaluate(propertyName => GetPoolProperties(propertyName, poolConfig, poolInfo));
		}

		static IEnumerable<string> GetPoolProperties(string name, IPoolConfig poolConfig, PoolInfo? poolInfo)
		{
			Func<IPoolConfig, PoolInfo?, string>? handler;
			if (s_poolPropertyHandlers.TryGetValue(name, out handler))
			{
				yield return handler(poolConfig, poolInfo);
			}
			else if (name.StartsWith(Pools.PoolPropertyNames.PropertiesPrefix, StringComparison.OrdinalIgnoreCase))
			{
				string propertyName = name.Substring(Pools.PoolPropertyNames.PropertiesPrefix.Length);
				if (poolConfig.Properties != null && poolConfig.Properties.TryGetValue(propertyName, out string? propertyValue))
				{
					yield return propertyValue;
				}
			}
			else
			{
				throw new ConditionException($"Unknown property '{name}' in pool condition");
			}
		}

		static readonly Dictionary<string, Func<IPoolConfig, PoolInfo?, string>> s_poolPropertyHandlers = CreatePoolPropertyHandlers();

		static Dictionary<string, Func<IPoolConfig, PoolInfo?, string>> CreatePoolPropertyHandlers()
		{
			Dictionary<string, Func<IPoolConfig, PoolInfo?, string>> handlers = new(StringComparer.OrdinalIgnoreCase);
			handlers[Pools.PoolPropertyNames.Id] = (config, info) => config.Id.ToString();
			handlers[Pools.PoolPropertyNames.NumAgents] = (config, info) => MustDeref(info).NumAgents.ToString(CultureInfo.InvariantCulture);
			handlers[Pools.PoolPropertyNames.NumOffline] = (config, info) => MustDeref(info).NumOffline.ToString(CultureInfo.InvariantCulture);
			handlers[Pools.PoolPropertyNames.NumDisabled] = (config, info) => MustDeref(info).NumDisabled.ToString(CultureInfo.InvariantCulture);
			handlers[Pools.PoolPropertyNames.NumBusy] = (config, info) => MustDeref(info).NumBusy.ToString(CultureInfo.InvariantCulture);
			handlers[Pools.PoolPropertyNames.NumIdle] = (config, info) => MustDeref(info).NumIdle.ToString(CultureInfo.InvariantCulture);
			return handlers;
		}

		static PoolInfo MustDeref(PoolInfo? poolInfo)
			=> poolInfo ?? throw new ConditionException("Pass the stats=true query parameter to evaluate queries against pool stats");
	}
}
