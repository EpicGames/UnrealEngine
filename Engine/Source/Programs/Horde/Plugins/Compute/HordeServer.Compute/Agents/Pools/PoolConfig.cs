// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using HordeServer.Agents.Fleet;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Color to use for labels of this pool
	/// </summary>
	public enum PoolColor
	{
#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member
		Default = 0,
		Blue,
		Orange,
		Green,
		Gray,
#pragma warning restore CS1591 // Missing XML comment for publicly visible type or member
	}

	/// <summary>
	/// Configuration for a pool of machines
	/// </summary>
	public interface IPoolConfig
	{
		/// <summary>
		/// Unique id for this pool
		/// </summary>
		PoolId Id { get; }

		/// <summary>
		/// Name of the pool
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Condition for agents to automatically be included in this pool
		/// </summary>
		Condition? Condition { get; }

		/// <summary>
		/// Color to use for this pool on the dashboard
		/// </summary>
		PoolColor Color { get; }

		/// <summary>
		/// List of workspaces currently assigned to this pool
		/// </summary>
		IReadOnlyList<AgentWorkspaceInfo> Workspaces { get; }

		/// <summary>
		/// Arbitrary properties related to this pool
		/// </summary>
		IReadOnlyDictionary<string, string>? Properties { get; }

		/// <summary>
		/// Whether to enable autoscaling for this pool
		/// </summary>
		bool EnableAutoscaling { get; }

		/// <summary>
		/// AutoSDK view for this pool
		/// </summary>
		AutoSdkConfig? AutoSdkConfig { get; }

		/// <summary>
		/// Cooldown time between scale-out events
		/// </summary>
		TimeSpan? ScaleOutCooldown { get; }

		/// <summary>
		/// Cooldown time between scale-in events
		/// </summary>
		TimeSpan? ScaleInCooldown { get; }

		/// <summary>
		/// The minimum number of agents to keep in the pool
		/// </summary>
		int? MinAgents { get; }

		/// <summary>
		/// The minimum number of idle agents to hold in reserve
		/// </summary>
		int? NumReserveAgents { get; }

		/// <summary>
		/// Interval between conforms. If zero, the pool will not conform on a schedule.
		/// </summary>
		TimeSpan? ConformInterval { get; }

		/// <summary>
		/// Time to wait before shutting down an agent that has been disabled
		/// </summary>
		TimeSpan? ShutdownIfDisabledGracePeriod { get; }

		/// <inheritdoc/>
		[Obsolete("Use SizeStrategies instead")]
		PoolSizeStrategy? SizeStrategy { get; }

		/// <summary>
		/// List of pool sizing strategies for this pool. The first strategy with a matching condition will be picked.
		/// </summary>
		IReadOnlyList<PoolSizeStrategyInfo>? SizeStrategies { get; }

		/// <summary>
		/// List of fleet managers for this pool. The first strategy with a matching condition will be picked.
		/// If empty or no conditions match, a default fleet manager will be used.
		/// </summary>
		IReadOnlyList<FleetManagerInfo>? FleetManagers { get; }

		/// <summary>
		/// Settings for lease utilization pool sizing strategy (if used)
		/// </summary>
		LeaseUtilizationSettings? LeaseUtilizationSettings { get; }

		/// <summary>
		/// Settings for job queue pool sizing strategy (if used)
		/// </summary>
		JobQueueSettings? JobQueueSettings { get; }
	}

	/// <summary>
	/// Mutable configuration for a pool
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class PoolConfig : IPoolConfig
	{
		PoolId _id;

		/// <inheritdoc/>
		public PoolId Id
		{
			get => _id.IsEmpty ? new PoolId(StringId.Sanitize(Name)) : _id;
			set => _id = value;
		}

		/// <summary>
		/// Base pool config to copy settings from
		/// </summary>
		public PoolId? Base { get; set; }

		/// <inheritdoc/>
		public string Name { get; set; } = String.Empty;

		/// <inheritdoc/>
		public Condition? Condition { get; set; }

		/// <inheritdoc cref="IPoolConfig.Properties"/>
		public Dictionary<string, string>? Properties { get; set; }

		/// <inheritdoc/>
		IReadOnlyDictionary<string, string>? IPoolConfig.Properties => Properties;

		/// <inheritdoc/>
		public PoolColor? Color { get; set; }

		PoolColor IPoolConfig.Color => Color ?? PoolColor.Default;

		/// <inheritdoc/>
		[JsonIgnore]
		public List<AgentWorkspaceInfo> Workspaces { get; set; } = new List<AgentWorkspaceInfo>();

		IReadOnlyList<AgentWorkspaceInfo> IPoolConfig.Workspaces => Workspaces;

		/// <inheritdoc cref="IPoolConfig.EnableAutoscaling"/>
		public bool? EnableAutoscaling { get; set; }

		/// <inheritdoc/>
		bool IPoolConfig.EnableAutoscaling => EnableAutoscaling ?? true;

		/// <inheritdoc/>
		[JsonIgnore]
		public AutoSdkConfig? AutoSdkConfig { get; set; }

		/// <inheritdoc/>
		public int? MinAgents { get; set; }

		/// <inheritdoc/>
		public int? NumReserveAgents { get; set; }

		/// <inheritdoc/>
		public TimeSpan? ConformInterval { get; set; }

		/// <inheritdoc/>
		public TimeSpan? ScaleOutCooldown { get; set; }

		/// <inheritdoc/>
		public TimeSpan? ScaleInCooldown { get; set; }

		/// <inheritdoc/>
		public TimeSpan? ShutdownIfDisabledGracePeriod { get; set; }

		/// <inheritdoc/>
		[Obsolete("Use SizeStrategies instead")]
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <inheritdoc/>
		public List<PoolSizeStrategyInfo>? SizeStrategies { get; set; }

		/// <inheritdoc cref="IPoolConfig.SizeStrategies"/>
		IReadOnlyList<PoolSizeStrategyInfo>? IPoolConfig.SizeStrategies => SizeStrategies;

		/// <inheritdoc/>
		public List<FleetManagerInfo>? FleetManagers { get; set; }

		/// <inheritdoc cref="IPoolConfig.FleetManagers"/>
		IReadOnlyList<FleetManagerInfo>? IPoolConfig.FleetManagers => FleetManagers;

		/// <inheritdoc/>
		public LeaseUtilizationSettings? LeaseUtilizationSettings { get; set; }

		/// <inheritdoc/>
		public JobQueueSettings? JobQueueSettings { get; set; }

		/// <inheritdoc/>
		public ComputeQueueAwsMetricSettings? ComputeQueueAwsMetricSettings { get; set; }
	}

	/// <summary>
	/// Extension methods for IPool
	/// </summary>
	public static class PoolConfigExtensions
	{
		record struct RgbColor(byte R, byte G, byte B)
		{
			public static RgbColor Lerp(RgbColor lhs, RgbColor rhs, float t)
				=> new RgbColor((byte)(lhs.R + (rhs.R - lhs.R) * t), (byte)(lhs.G + (rhs.G - lhs.G) * t), (byte)(lhs.B + (rhs.B - lhs.B) * t));

			public string ToHexString() => $"#{R:x2}{G:x2}{B:x2}";
		}

		static readonly RgbColor[] s_colorTable =
		{
			new RgbColor(0x00, 0xbc, 0xf2),
			new RgbColor(0x5a, 0xc9, 0x5a),
			new RgbColor(0xff, 0x66, 0x00),
			new RgbColor(0xdf, 0x8b, 0xe5),
			new RgbColor(0x00, 0xbc, 0xf2),
		};

		static readonly Dictionary<PoolColor, RgbColor[]> s_colorNameToTable = new Dictionary<PoolColor, RgbColor[]>()
		{
			[PoolColor.Blue] = new[]
			{
				new RgbColor(0x00, 0x80, 0xf2),
				new RgbColor(0x00, 0xbc, 0xf2),
			},
			[PoolColor.Orange] = new[]
			{
				new RgbColor(0xff, 0x70, 0x00),
				new RgbColor(0xff, 0x40, 0x00),
			},
			[PoolColor.Green] = new[]
			{
				new RgbColor(0x3a, 0xc9, 0x3a),
				new RgbColor(0x7a, 0xc9, 0x7a),
			},
			[PoolColor.Gray] = new[]
			{
				new RgbColor(0x4a, 0x4a, 0x4a),
				new RgbColor(0x6a, 0x6a, 0x6a),
			}
		};

		/// <summary>
		/// Gets the color for a pool
		/// </summary>
		public static string GetColorValue(this IPoolConfig poolConfig)
		{
			if (poolConfig.Color == PoolColor.Default)
			{
				// Get the desired color from the properties object on the pool config
				if (poolConfig.Properties != null && poolConfig.Properties.TryGetValue("Color", out string? colorText) && UInt32.TryParse(colorText, out uint colorInt))
				{
					return GetColorValue(colorInt / 600.0f, s_colorTable);
				}
			}

			RgbColor[]? colorTable;
			if (!s_colorNameToTable.TryGetValue(poolConfig.Color, out colorTable))
			{
				colorTable = s_colorNameToTable.First().Value;
			}

			float slider = IoHash.Compute(Encoding.UTF8.GetBytes(poolConfig.Id.ToString())).ToByteArray()[0] / 255.0f;
			return GetColorValue(slider, colorTable);
		}

		static string GetColorValue(float slider, RgbColor[] colorTable)
		{
			// Convert to an index/lerp value
			float value = slider * (colorTable.GetLength(0) - 1);
			int idx = Math.Clamp((int)value, 0, colorTable.Length - 2);
			float t = Math.Clamp(value - idx, 0.0f, 1.0f);

			// Create the final rgb value
			return RgbColor.Lerp(colorTable[idx], colorTable[idx + 1], t).ToHexString();
		}

		/// <summary>
		/// Evaluates a condition against a pool
		/// </summary>
		/// <param name="poolConfig">The pool to evaluate</param>
		/// <param name="condition">The condition to evaluate</param>
		/// <returns>True if the pool satisfies the condition</returns>
		public static bool SatisfiesCondition(this IPoolConfig poolConfig, Condition condition)
		{
			return condition.Evaluate(propKey =>
			{
				if (poolConfig.Properties != null && poolConfig.Properties.TryGetValue(propKey, out string? propValue))
				{
					return new[] { propValue };
				}

				return Array.Empty<string>();
			});
		}

		/// <summary>
		/// Determine whether a pool offers agents that are compatible with the given requirements.
		/// Since a pool does not have beforehand knowledge of what type of agents will be created, their expected
		/// properties must be specified on the IPool. 
		/// </summary>
		/// <param name="poolConfig">The pool to verify</param>
		/// <param name="requirements">Requirements</param>
		/// <returns>True if the pool satisfies the requirements</returns>
		public static bool MeetsRequirements(this IPoolConfig poolConfig, Requirements requirements)
		{
			if (requirements.Condition != null && !poolConfig.SatisfiesCondition(requirements.Condition))
			{
				return false;
			}

			return true;
		}
	}
}
