// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Text.Json.Nodes;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using HordeServer.Agents.Fleet;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents.Pools
{
	/// <see cref="PoolSizeStrategyInfo" />
	public class PoolSizeStrategyMessage
	{
		/// <summary>
		/// Strategy implementation to use
		/// </summary>
		public PoolSizeStrategy Type { get; set; }

		/// <summary>
		/// Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Configuration for the strategy, serialized as JSON
		/// </summary>
		public JsonObject Config { get; set; } = new JsonObject();

		/// <summary>
		/// Integer to add after pool size has been calculated. Can also be negative.
		/// </summary>
		public int ExtraAgentCount { get; set; } = 0;

		/// <summary>
		/// Empty constructor for JSON serialization
		/// </summary>
		public PoolSizeStrategyMessage()
		{
		}

		/// <summary>
		/// Construct a public REST API representation from the internal one
		/// </summary>
		/// <param name="strategy"></param>
		public PoolSizeStrategyMessage(PoolSizeStrategyInfo strategy)
		{
			Type = strategy.Type;
			Condition = strategy.Condition;
			Config = (JsonObject)strategy.Config.DeepClone();
			ExtraAgentCount = strategy.ExtraAgentCount;
		}

		/// <summary>
		/// Convert this public REST API object to an internal representation
		/// </summary>
		/// <returns></returns>
		public PoolSizeStrategyInfo Convert()
		{
			return new PoolSizeStrategyInfo(Type, Condition, ExtraAgentCount) { Config = (JsonObject)Config.DeepClone() };
		}
	}

	/// <see cref="FleetManagerInfo" />
	public class FleetManagerMessage
	{
		/// <summary>
		/// Strategy implementation to use
		/// </summary>
		public FleetManagerType Type { get; set; }

		/// <summary>
		/// Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Configuration for the strategy, serialized as JSON
		/// </summary>
		public JsonObject? Config { get; set; }

		/// <summary>
		/// Empty constructor for JSON serialization
		/// </summary>
		public FleetManagerMessage()
		{
		}

		/// <summary>
		/// Construct a public REST API representation from the internal one
		/// </summary>
		/// <param name="info"></param>
		public FleetManagerMessage(FleetManagerInfo info)
		{
			Type = info.Type;
			Condition = info.Condition;
			Config = info.Config;
		}

		/// <summary>
		/// Convert this public REST API object to an internal representation
		/// </summary>
		/// <returns></returns>
		public FleetManagerInfo Convert()
		{
			return new FleetManagerInfo(Type, Condition) { Config = (JsonObject?)Config?.DeepClone() };
		}
	}

	/// <see cref="LeaseUtilizationSettings" />
	public class LeaseUtilizationSettingsMessage
	{
		/// <summary>
		/// Construct a public REST API representation from the internal one
		/// </summary>
		/// <param name="settings"></param>
#pragma warning disable IDE0060 // Remove unused parameter
		public LeaseUtilizationSettingsMessage(LeaseUtilizationSettings settings)
#pragma warning restore IDE0060 // Remove unused parameter
		{
		}

		/// <summary>
		/// Convert this public REST API object to an internal representation
		/// </summary>
		/// <returns></returns>
		public LeaseUtilizationSettings Convert()
		{
			_ = this;
			return new LeaseUtilizationSettings();
		}
	}

	/// <see cref="JobQueueSettings" />
	public class JobQueueSettingsMessage
	{
		/// <summary>
		/// Factor translating queue size to additional agents to grow the pool with
		/// The result is always rounded up to nearest integer. 
		/// Example: if there are 20 jobs in queue, a factor 0.25 will result in 5 new agents being added (20 * 0.25)
		/// </summary>
		public double ScaleOutFactor { get; set; }

		/// <summary>
		/// Factor by which to shrink the pool size with when queue is empty
		/// The result is always rounded up to nearest integer.
		/// Example: when the queue size is zero, a default value of 0.9 will shrink the pool by 10% (current agent count * 0.9)
		/// </summary>
		public double ScaleInFactor { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JobQueueSettingsMessage()
		{
		}

		/// <summary>
		/// Construct a public REST API representation from the internal one
		/// </summary>
		/// <param name="settings"></param>
		public JobQueueSettingsMessage(JobQueueSettings settings)
		{
			ScaleInFactor = settings.ScaleInFactor;
			ScaleOutFactor = settings.ScaleOutFactor;
		}

		/// <summary>
		/// Convert this public REST API object to an internal representation
		/// </summary>
		/// <returns></returns>
		public JobQueueSettings Convert()
		{
			return new JobQueueSettings(ScaleOutFactor, ScaleInFactor);
		}
	}

	/// <see cref="ComputeQueueAwsMetricSettings" />
	public class ComputeQueueAwsMetricSettingsMessage
	{
		/// <summary>
		/// Compute cluster ID to observe
		/// </summary>
		public string ComputeClusterId { get; set; } = null!;

		/// <summary>
		/// AWS CloudWatch namespace to write metrics in
		/// </summary>
		public string Namespace { get; set; } = null!;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeQueueAwsMetricSettingsMessage()
		{
		}

		/// <summary>
		/// Construct a public REST API representation from the internal one
		/// </summary>
		/// <param name="settings"></param>
		public ComputeQueueAwsMetricSettingsMessage(ComputeQueueAwsMetricSettings settings)
		{
			ComputeClusterId = settings.ComputeClusterId;
			Namespace = settings.Namespace;
		}

		/// <summary>
		/// Convert this public REST API object to an internal representation
		/// </summary>
		/// <returns></returns>
		public ComputeQueueAwsMetricSettings Convert()
		{
			return new ComputeQueueAwsMetricSettings(ComputeClusterId, Namespace);
		}
	}

	/// <summary>
	/// Parameters to create a new pool
	/// </summary>
	public class CreatePoolRequest
	{
		/// <summary>
		/// Name for the new pool
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// Condition to satisfy for agents to be included in this pool
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Whether to enable autoscaling for this pool
		/// </summary>
		public bool? EnableAutoscaling { get; set; }

		/// <summary>
		/// Interval between conforms in hours. Set to zero to disable.
		/// </summary>
		public int? ConformInterval { get; set; }

		/// <summary>
		/// Cooldown time between scale-out events in seconds
		/// </summary>
		public int? ScaleOutCooldown { get; set; }

		/// <summary>
		/// Cooldown time between scale-in events in seconds
		/// </summary>
		public int? ScaleInCooldown { get; set; }

		/// <summary>
		/// Pool sizing strategies
		/// </summary>
		public List<PoolSizeStrategyMessage>? SizeStrategies { get; set; }

		/// <summary>
		/// Fleet managers
		/// </summary>
		public List<FleetManagerMessage>? FleetManagers { get; set; }

		/// <summary>
		/// Pool sizing strategy
		/// </summary>
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <summary>
		/// Settings for lease utilization pool sizing strategy (if used)
		/// </summary>
		public LeaseUtilizationSettingsMessage? LeaseUtilizationSettings { get; set; }

		/// <summary>
		/// Settings for job queue pool sizing strategy (if used) 
		/// </summary>
		public JobQueueSettingsMessage? JobQueueSettings { get; set; }

		/// <summary>
		/// Settings for compute queue pool sizing strategy for AWS metrics (if used) 
		/// </summary>
		public ComputeQueueAwsMetricSettingsMessage? ComputeQueueAwsMetricSettings { get; set; }

		/// <summary>
		/// The minimum number of agents to retain in this pool
		/// </summary>
		public int? MinAgents { get; set; }

		/// <summary>
		/// The minimum number of idle agents in this pool, if autoscaling is enabled
		/// </summary>
		public int? NumReserveAgents { get; set; }

		/// <summary>
		/// Properties for the new pool
		/// </summary>
		public Dictionary<string, string>? Properties { get; set; }
	}

	/// <summary>
	/// Response from creating a new pool
	/// </summary>
	public class CreatePoolResponse
	{
		/// <summary>
		/// Unique id for the new pool
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id for the new pool</param>
		public CreatePoolResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Parameters to update a pool
	/// </summary>
	public class UpdatePoolRequest
	{
		/// <summary>
		/// Optional new name for the pool
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Requirements for this pool
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Whether to enable autoscaling for this pool
		/// </summary>
		public bool? EnableAutoscaling { get; set; }

		/// <summary>
		/// Interval between conforms in hours. Set to -1 to reset to the default, or 0 to disable.
		/// </summary>
		public int? ConformInterval { get; set; }

		/// <summary>
		/// Cooldown time between scale-out events in seconds
		/// </summary>
		public int? ScaleOutCooldown { get; set; }

		/// <summary>
		/// Cooldown time between scale-in events in seconds
		/// </summary>
		public int? ScaleInCooldown { get; set; }

		/// <summary>
		/// Pool sizing strategy
		/// </summary>
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <summary>
		/// Pool sizing strategies
		/// </summary>
		public List<PoolSizeStrategyMessage>? SizeStrategies { get; set; }

		/// <summary>
		/// Fleet managers
		/// </summary>
		public List<FleetManagerMessage>? FleetManagers { get; set; }

		/// <summary>
		/// Set pool to use default strategy
		/// </summary>
		public bool? UseDefaultStrategy { get; set; }

		/// <summary>
		/// Settings for lease utilization pool sizing strategy (if used)
		/// </summary>
		public LeaseUtilizationSettingsMessage? LeaseUtilizationSettings { get; set; }

		/// <summary>
		/// Settings for job queue pool sizing strategy (if used) 
		/// </summary>
		public JobQueueSettingsMessage? JobQueueSettings { get; set; }

		/// <summary>
		/// Settings for compute queue AWS metric pool sizing strategy (if used) 
		/// </summary>
		public ComputeQueueAwsMetricSettingsMessage? ComputeQueueAwsMetricSettings { get; set; }

		/// <summary>
		/// The minimum number of agents to retain in this pool
		/// </summary>
		public int? MinAgents { get; set; }

		/// <summary>
		/// The minimum number of idle agents in this pool, if autoscaling is enabled
		/// </summary>
		public int? NumReserveAgents { get; set; }

		/// <summary>
		/// Properties to update for the pool. Properties set to null will be removed.
		/// </summary>
		public Dictionary<string, string?>? Properties { get; set; }
	}

	/// <summary>
	/// Parameters to update a pool
	/// </summary>
	public class BatchUpdatePoolRequest
	{
		/// <summary>
		///  ID of the pool to update
		/// </summary>
		[Required]
		public string Id { get; set; } = null!;

		/// <summary>
		/// Optional new name for the pool
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Properties to update for the pool. Properties set to null will be removed.
		/// </summary>
		public Dictionary<string, string?>? Properties { get; set; }
	}

	/// <summary>
	/// Response describing a pool
	/// </summary>
	public class GetPoolResponse
	{
		/// <summary>
		/// Unique id of the pool
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Name of the pool
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Condition for agents to be auto-added to the pool
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Color to display for this pool
		/// </summary>
		public string ColorValue { get; set; }

		/// <summary>
		/// Whether to enable autoscaling for this pool
		/// </summary>
		public bool EnableAutoscaling { get; set; }

		/// <summary>
		/// Frequency to run conforms, in hours.
		/// </summary>
		public int? ConformInterval { get; set; }

		/// <summary>
		/// Cooldown time between scale-out events in seconds
		/// </summary>
		public int? ScaleOutCooldown { get; set; }

		/// <summary>
		/// Cooldown time between scale-in events in seconds
		/// </summary>
		public int? ScaleInCooldown { get; set; }

		/// <summary>
		/// Pool sizing strategies to be used for this pool
		/// </summary>
		public List<PoolSizeStrategyMessage> SizeStrategies { get; set; }

		/// <summary>
		/// Fleet managers to be used for this pool
		/// </summary>
		public List<FleetManagerMessage> FleetManagers { get; set; }

		/// <summary>
		/// Pool sizing strategy to be used for this pool (deprecated, see SizeStrategies field)
		/// </summary>
		public PoolSizeStrategy? SizeStrategy { get; set; }

		/// <summary>
		/// Settings for lease utilization pool sizing strategy (deprecated, see SizeStrategies field)
		/// </summary>
		public LeaseUtilizationSettingsMessage? LeaseUtilizationSettings { get; set; }

		/// <summary>
		/// Settings for job queue pool sizing strategy (deprecated, see SizeStrategies field)
		/// </summary>
		public JobQueueSettingsMessage? JobQueueSettings { get; set; }

		/// <summary>
		/// Settings for compute queue pool sizing strategy for AWS metrics (deprecated, see SizeStrategies field)
		/// </summary>
		public ComputeQueueAwsMetricSettingsMessage? ComputeQueueAwsMetricSettings { get; set; }

		/// <summary>
		/// The minimum nunmber of agents to retain in this pool
		/// </summary>
		public int? MinAgents { get; set; }

		/// <summary>
		/// The minimum number of idle agents in this pool, if autoscaling is enabled
		/// </summary>
		public int? NumReserveAgents { get; set; }

		/// <summary>
		/// Time to wait before shutting down an agent that has been disabled
		/// </summary>
		public TimeSpan? ShutdownIfDisabledGracePeriod { get; set; }

		/// <summary>
		/// List of workspaces that this agent contains
		/// </summary>
		public List<GetAgentWorkspaceResponse> Workspaces { get; set; }

		/// <summary>
		/// AutoSDK configuration for the pool 
		/// </summary>
		public AutoSdkConfig? AutoSdkConfig { get; set; }

		/// <summary>
		/// Arbitrary properties for this pool.
		/// </summary>
		public IReadOnlyDictionary<string, string> Properties { get; set; }

		/// <summary>
		/// Whether this pool is managed via config files (true) or the database (false).
		/// Null when the distinction is not applicable (V0/V1/V2).
		/// Config-managed pools cannot be edited through the REST API or dashboard.
		/// </summary>
		public bool? ConfigManaged { get; set; }

		/// <summary>
		/// Whether this config-managed pool also has a database entry (shadow pool).
		/// When true, the database entry can be deleted to complete migration cleanup.
		/// Only set when <see cref="ConfigManaged"/> is true.
		/// </summary>
		public bool? HasDatabaseEntry { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="pool">The pool to construct from</param>
		public GetPoolResponse(IPoolConfig pool)
		{
			Id = pool.Id.ToString();
			Name = pool.Name;
			Condition = pool.Condition;
			ColorValue = pool.GetColorValue();
			EnableAutoscaling = pool.EnableAutoscaling;
			ConformInterval = pool.ConformInterval == null ? null : (int)pool.ConformInterval.Value.TotalHours;
			ScaleOutCooldown = pool.ScaleOutCooldown == null ? null : (int)pool.ScaleOutCooldown.Value.TotalSeconds;
			ScaleInCooldown = pool.ScaleInCooldown == null ? null : (int)pool.ScaleInCooldown.Value.TotalSeconds;
			SizeStrategies = pool.SizeStrategies?.Select(x => new PoolSizeStrategyMessage(x))?.ToList() ?? new List<PoolSizeStrategyMessage>();
			FleetManagers = pool.FleetManagers?.Select(x => new FleetManagerMessage(x))?.ToList() ?? new List<FleetManagerMessage>();
#pragma warning disable CS0618 // Type or member is obsolete
			SizeStrategy = pool.SizeStrategy ?? default;
#pragma warning restore CS0618 // Type or member is obsolete
			LeaseUtilizationSettings = pool.LeaseUtilizationSettings == null ? null : new LeaseUtilizationSettingsMessage(pool.LeaseUtilizationSettings);
			JobQueueSettings = pool.JobQueueSettings == null ? null : new JobQueueSettingsMessage(pool.JobQueueSettings);
			MinAgents = pool.MinAgents;
			NumReserveAgents = pool.NumReserveAgents;
			ShutdownIfDisabledGracePeriod = pool.ShutdownIfDisabledGracePeriod;
			Workspaces = pool.Workspaces.Select(x => AgentsController.CreateGetAgentWorkspaceResponse(x)).ToList();
			AutoSdkConfig = pool.AutoSdkConfig;
			Properties = pool.Properties ?? new Dictionary<string, string>();
		}
	}

	/// <summary>
	/// Response describing a pool
	/// </summary>
	/// <param name="Id">Identifier for the pool</param>
	/// <param name="Name">Name of the pool</param>
	/// <param name="ColorValue">Color to render the pool label</param>
	/// <param name="Autoscaled">Whether autoscaling is enabled for this pool</param>
	/// <param name="Stats">Counts for agents in differrent states</param>
	/// <param name="Agents">Truncated list of agents</param>
	/// <param name="Utilization">Utilization samples for the pool, from zero to one, with one sample for each hour</param>
	/// <param name="ConfigManaged">Whether this pool is managed by configuration</param>
	/// <param name="HasDatabaseEntry">Whether this config-managed pool also has a database entry (shadow pool)</param>
	public record class GetPoolSummaryResponse(PoolId Id, string Name, string ColorValue, bool Autoscaled, GetPoolStatsResponse? Stats, List<GetPoolAgentSummaryResponse>? Agents, List<double>? Utilization, bool? ConfigManaged = null, bool? HasDatabaseEntry = null);

	/// <summary>
	/// Numbers of agents matching various criteria
	/// </summary>
	/// <param name="NumAgents">Number of agents in the pool</param>
	/// <param name="NumIdle">Number of agents that are ready</param>
	/// <param name="NumOffline">Number of agents offline</param>
	/// <param name="NumDisabled">Number of agents that are disabled</param>
	public record class GetPoolStatsResponse(int NumAgents, int NumIdle, int NumOffline, int NumDisabled);

	/// <summary>
	/// Response describing an agent in a pool
	/// </summary>
	/// <param name="AgentId">Identifier for the pool</param>
	/// <param name="Idle">Whether the agent is idle</param>
	/// <param name="Offline">Whether the agent is online</param>
	/// <param name="Disabled">Whether the agent is disabled</param>
	public record class GetPoolAgentSummaryResponse(AgentId AgentId, bool? Idle = null, bool? Offline = null, bool? Disabled = null);
}
