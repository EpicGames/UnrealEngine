// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Pools;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Wraps functionality for manipulating pools
	/// </summary>
	public class PoolService
	{
		/// <summary>
		/// Collection of pool documents
		/// </summary>
		readonly IPoolCollection _pools;

		/// <summary>
		/// Returns the current time
		/// </summary>
		readonly IClock _clock;

		/// <summary>
		/// Cached set of pools, along with the timestamp that it was obtained
		/// </summary>
		Tuple<DateTime, Dictionary<PoolId, IPoolConfig>>? _cachedPoolLookup;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="pools">Collection of pool documents</param>
		/// <param name="clock"></param>
		public PoolService(IPoolCollection pools, IClock clock)
		{
			_pools = pools;
			_clock = clock;
		}

		/// <summary>
		/// Creates a new pool
		/// </summary>
		/// <param name="name">Name of the new pool</param>
		/// <param name="options">Options for the new pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new pool document</returns>
		[Obsolete("Pools should be configured through globals.json")]
		public Task CreatePoolAsync(string name, CreatePoolConfigOptions options, CancellationToken cancellationToken = default)
		{
			return _pools.CreateConfigAsync(new PoolId(StringId.Sanitize(name)), name, options, cancellationToken);
		}

		/// <summary>
		/// Deletes a pool
		/// </summary>
		/// <param name="poolId">Unique id of the pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task object</returns>
		[Obsolete("Pools should be configured through globals.json")]
		public Task<bool> DeletePoolAsync(PoolId poolId, CancellationToken cancellationToken = default)
		{
			return _pools.DeleteConfigAsync(poolId, cancellationToken);
		}

		/// <summary>
		/// Updates an existing pool
		/// </summary>
		/// <param name="poolId">The pool to update</param>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task object</returns>
		[Obsolete("Pools should be configured through globals.json")]
		public Task UpdateConfigAsync(PoolId poolId, UpdatePoolConfigOptions options, CancellationToken cancellationToken)
		{
			return _pools.UpdateConfigAsync(poolId, options, cancellationToken);
		}

		/// <summary>
		/// Gets all the available pools
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of pool documents</returns>
		public Task<IReadOnlyList<IPoolConfig>> GetPoolsAsync(CancellationToken cancellationToken)
		{
			return _pools.GetConfigsAsync(cancellationToken);
		}

		/// <summary>
		/// Gets a pool by ID
		/// </summary>
		/// <param name="poolId">Unique id of the pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The pool document</returns>
		public Task<IPool?> GetPoolAsync(PoolId poolId, CancellationToken cancellationToken = default)
		{
			return _pools.GetAsync(poolId, cancellationToken);
		}

		/// <summary>
		/// Gets a cached pool definition
		/// </summary>
		/// <param name="poolId"></param>
		/// <param name="validAtTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IPoolConfig?> GetPoolAsync(PoolId poolId, DateTime validAtTime, CancellationToken cancellationToken)
		{
			IReadOnlyDictionary<PoolId, IPoolConfig> poolMapping = await GetPoolLookupAsync(validAtTime, cancellationToken);
			poolMapping.TryGetValue(poolId, out IPoolConfig? pool);
			return pool;
		}

		/// <summary>
		/// Gets a cached pool definition
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="validAtTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<List<IPoolConfig>> GetPoolsAsync(IAgent agent, DateTime validAtTime, CancellationToken cancellationToken)
		{
			IReadOnlyDictionary<PoolId, IPoolConfig> poolMapping = await GetPoolLookupAsync(validAtTime, cancellationToken);

			List<IPoolConfig> pools = new List<IPoolConfig>();
			foreach (PoolId poolId in agent.Pools)
			{
				if (poolMapping.TryGetValue(poolId, out IPoolConfig? pool))
				{
					pools.Add(pool);
				}
			}

			return pools;
		}

		/// <summary>
		/// Gets a mapping from pool identifiers to definitions
		/// </summary>
		/// <param name="validAtTime">Absolute time at which we expect the results to be valid. Values may be cached as long as they are after this time.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Map of pool ids to pool documents</returns>
		public async Task<IReadOnlyDictionary<PoolId, IPoolConfig>> GetPoolLookupAsync(DateTime validAtTime, CancellationToken cancellationToken)
		{
			Tuple<DateTime, Dictionary<PoolId, IPoolConfig>>? cachedPoolLookupCopy = _cachedPoolLookup;
			if (cachedPoolLookupCopy == null || cachedPoolLookupCopy.Item1 < validAtTime)
			{
				// Get a new list of cached pools
				DateTime newCacheTime = _clock.UtcNow;
				IReadOnlyList<IPoolConfig> newPools = await _pools.GetConfigsAsync(cancellationToken);
				Tuple<DateTime, Dictionary<PoolId, IPoolConfig>> newCachedPoolLookup = Tuple.Create(newCacheTime, newPools.ToDictionary(x => x.Id, x => x));

				// Try to swap it with the current version
				while (cachedPoolLookupCopy == null || cachedPoolLookupCopy.Item1 < newCacheTime)
				{
					Tuple<DateTime, Dictionary<PoolId, IPoolConfig>>? originalValue = Interlocked.CompareExchange(ref _cachedPoolLookup, newCachedPoolLookup, cachedPoolLookupCopy);
					if (originalValue == cachedPoolLookupCopy)
					{
						cachedPoolLookupCopy = newCachedPoolLookup;
						break;
					}
					cachedPoolLookupCopy = originalValue;
				}
			}
			return cachedPoolLookupCopy.Item2;
		}
	}
}
