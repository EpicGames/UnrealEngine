// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Utilization;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Controller for the /api/v1/pools endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class PoolsController : HordeControllerBase
	{
		readonly AgentService _agentService;
		readonly IPoolCollection _poolCollection;
		readonly IUtilizationDataCollection _utilizationDataCollection;
		readonly IClock _clock;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolsController(AgentService agentService, IPoolCollection poolCollection, IUtilizationDataCollection utilizationDataCollection, IClock clock, IOptionsSnapshot<ComputeConfig> computeConfig)
		{
			_agentService = agentService;
			_poolCollection = poolCollection;
			_utilizationDataCollection = utilizationDataCollection;
			_clock = clock;
			_computeConfig = computeConfig;
		}

		/// <summary>
		/// Creates a new pool
		/// </summary>
		/// <param name="create">Parameters for the new pool.</param>
		/// <returns>Http result code</returns>
		[HttpPost]
		[Obsolete("Pools should be configured through globals.json")]
		[Route("/api/v1/pools")]
		public async Task<ActionResult<CreatePoolResponse>> CreatePoolAsync([FromBody] CreatePoolRequest create)
		{
			if (!_computeConfig.Value.Authorize(PoolAclAction.CreatePool, User))
			{
				return Forbid(PoolAclAction.CreatePool);
			}

			if (_computeConfig.Value.VersionEnum >= ConfigVersion.PoolsInConfigFiles)
			{
				return BadRequest("All pools are managed via config files. Pool creation through the API is disabled.");
			}

			List<PoolSizeStrategyInfo>? sizeStrategies = create.SizeStrategies?.Select(x => x.Convert()).ToList();
			List<FleetManagerInfo>? fleetManagers = create.FleetManagers?.Select(x => x.Convert()).ToList();
			LeaseUtilizationSettings? luSettings = create.LeaseUtilizationSettings?.Convert();
			JobQueueSettings? jqSettings = create.JobQueueSettings?.Convert();
			ComputeQueueAwsMetricSettings? cqamSettings = create.ComputeQueueAwsMetricSettings?.Convert();

			TimeSpan? conformInterval = create.ConformInterval == null ? null : TimeSpan.FromHours(create.ConformInterval.Value);
			TimeSpan? scaleOutCooldown = create.ScaleOutCooldown == null ? null : TimeSpan.FromSeconds(create.ScaleOutCooldown.Value);
			TimeSpan? scaleInCooldown = create.ScaleInCooldown == null ? null : TimeSpan.FromSeconds(create.ScaleInCooldown.Value);

			CreatePoolConfigOptions options = new CreatePoolConfigOptions();
			options.Condition = create.Condition;
			options.EnableAutoscaling = create.EnableAutoscaling;
			options.MinAgents = create.MinAgents;
			options.NumReserveAgents = create.NumReserveAgents;
			options.ConformInterval = conformInterval;
			options.ScaleOutCooldown = scaleOutCooldown;
			options.ScaleInCooldown = scaleInCooldown;
			options.SizeStrategies = sizeStrategies;
			options.FleetManagers = fleetManagers;
			options.SizeStrategy = create.SizeStrategy;
			options.LeaseUtilizationSettings = luSettings;
			options.JobQueueSettings = jqSettings;
			options.ComputeQueueAwsMetricSettings = cqamSettings;
			options.Properties = create.Properties;

			PoolId poolId = new PoolId(StringId.Sanitize(create.Name));
			await _poolCollection.CreateConfigAsync(poolId, create.Name, options);

			return new CreatePoolResponse(poolId.ToString());
		}

		/// <summary>
		/// Query all the pools
		/// </summary>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the pools</returns>
		[HttpGet]
		[Route("/api/v1/pools")]
		[ProducesResponseType(typeof(List<GetPoolResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetPoolsAsync([FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ComputeConfig computeConfig = _computeConfig.Value;
			if (!computeConfig.Authorize(PoolAclAction.ListPools, User))
			{
				return Forbid(PoolAclAction.ListPools);
			}

			IReadOnlyList<IPoolConfig> poolConfigs = await _poolCollection.GetConfigsAsync(cancellationToken);

			List<object> responses = new List<object>();
			foreach (IPoolConfig poolConfig in poolConfigs)
			{
				responses.Add(new GetPoolResponse(poolConfig).ApplyFilter(filter));
			}
			return responses;
		}

		/// <summary>
		/// Query all the pools
		/// </summary>
		/// <param name="condition">Condition to select which pools to include</param>
		/// <param name="stats">Whether to include stats in the response</param>
		/// <param name="numAgents">Number of agents to include for each pool in the response</param>
		/// <param name="numUtilizationSamples">Number of utilization samples to include for each pool</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the pools</returns>
		[HttpGet]
		[Route("/api/v2/pools")]
		[ProducesResponseType(typeof(List<GetPoolResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetPoolSummariesAsync([FromQuery] Condition? condition = null, [FromQuery] bool stats = false, [FromQuery] int numAgents = 0, [FromQuery] int numUtilizationSamples = 0, CancellationToken cancellationToken = default)
		{
			ComputeConfig computeConfig = _computeConfig.Value;
			if (!computeConfig.Authorize(PoolAclAction.ListPools, User))
			{
				return Forbid(PoolAclAction.ListPools);
			}

			DateTime utcNow = _clock.UtcNow;
			IReadOnlyList<IPoolConfig> poolConfigs = await _poolCollection.GetConfigsAsync(cancellationToken);

			// Compute all the expensive response stuff
			Dictionary<PoolId, PoolInfo>? poolIdToInfo = null;
			if (stats || numAgents > 0 || numUtilizationSamples > 0)
			{
				poolIdToInfo = poolConfigs.ToDictionary(x => x.Id, x => new PoolInfo());

				if (stats || numAgents > 0)
				{
					IReadOnlyList<IAgent> agents = await _agentService.GetCachedAgentsAsync(cancellationToken);
					foreach (IAgent agent in agents)
					{
						foreach (PoolId poolId in agent.Pools)
						{
							PoolInfo? poolInfo;
							if (poolIdToInfo.TryGetValue(poolId, out poolInfo))
							{
								poolInfo.AddAgent(agent, utcNow);
							}
						}
					}
				}

				if (numUtilizationSamples > 0)
				{
					IReadOnlyList<IUtilizationData> utilizationDataList = await _utilizationDataCollection.GetUtilizationDataAsync(count: numUtilizationSamples, cancellationToken: cancellationToken);
					for (int sampleIdx = 0; sampleIdx < utilizationDataList.Count; sampleIdx++)
					{
						IUtilizationData utilizationData = utilizationDataList[sampleIdx];
						foreach (IPoolUtilizationData poolUtilizationData in utilizationData.Pools)
						{
							if (poolUtilizationData.NumAgents > 0)
							{
								PoolInfo? poolInfo;
								if (poolIdToInfo.TryGetValue(poolUtilizationData.PoolId, out poolInfo))
								{
									poolInfo.Utilization ??= new List<double>();
									while (poolInfo.Utilization!.Count < sampleIdx)
									{
										poolInfo.Utilization.Add(0.0);
									}

									double activeTime = poolUtilizationData.AdminTime + poolUtilizationData.HibernatingTime + poolUtilizationData.OtherTime + poolUtilizationData.Streams.Sum(x => x.Time);
									poolInfo.Utilization.Add(activeTime / poolUtilizationData.NumAgents);
								}
							}
						}
					}
				}
			}

			List<object> responses = new List<object>();
			foreach (IPoolConfig poolConfig in poolConfigs)
			{
				PoolInfo? poolInfo = null;
				if (poolIdToInfo != null && !poolIdToInfo.TryGetValue(poolConfig.Id, out poolInfo))
				{
					poolInfo = null;
				}

				if (condition == null || PoolCondition.Evaluate(condition, poolConfig, poolInfo))
				{
					GetPoolStatsResponse? statsResponse = null;
					List<GetPoolAgentSummaryResponse>? agentResponses = null;
					List<double>? utilizationResponses = null;

					if (poolInfo != null)
					{
						if (stats)
						{
							statsResponse = new GetPoolStatsResponse(poolInfo.NumAgents, poolInfo.NumIdle, poolInfo.NumOffline, poolInfo.NumDisabled);
						}
						if (numAgents > 0)
						{
							agentResponses = new List<GetPoolAgentSummaryResponse>();
							foreach (IAgent agent in poolInfo.Agents?.OrderBy(x => x.Id).Take(numAgents) ?? Enumerable.Empty<IAgent>())
							{
								bool? idle = (agent.Leases.Count == 0) ? (bool?)true : null;
								bool? offline = agent.IsSessionValid(utcNow) ? (bool?)null : true;
								bool? disabled = agent.Enabled ? (bool?)null : true;
								agentResponses.Add(new GetPoolAgentSummaryResponse(agent.Id, idle, offline, disabled));
							}
						}
						if (numUtilizationSamples > 0)
						{
							utilizationResponses = poolInfo.Utilization ?? new List<double>();
						}
					}

					GetPoolSummaryResponse response = new GetPoolSummaryResponse(poolConfig.Id, poolConfig.Name, poolConfig.GetColorValue(), poolConfig.EnableAutoscaling, statsResponse, agentResponses, utilizationResponses, null, null);
					responses.Add(response);
				}
			}
			return responses;
		}

		/// <summary>
		/// Retrieve information about a specific pool
		/// </summary>
		/// <param name="poolId">Id of the pool to get information about</param>
		/// <param name="filter">Filter to apply to the returned properties</param>
		/// <returns>Information about the requested pool</returns>
		[HttpGet]
		[Route("/api/v1/pools/{poolId}")]
		[ProducesResponseType(typeof(GetPoolResponse), 200)]
		public async Task<ActionResult<object>> GetPoolAsync(string poolId, [FromQuery] PropertyFilter? filter = null)
		{
			if (!_computeConfig.Value.Authorize(PoolAclAction.ViewPool, User))
			{
				return Forbid(PoolAclAction.ViewPool);
			}

			PoolId poolIdValue = new PoolId(poolId);

			IPool? pool = await _poolCollection.GetAsync(poolIdValue);
			if (pool == null)
			{
				return NotFound(poolIdValue);
			}

			return new GetPoolResponse(pool).ApplyFilter(filter);
		}

		/// <summary>
		/// Update a pool's properties.
		/// </summary>
		/// <param name="poolId">Id of the pool to update</param>
		/// <param name="update">Items on the pool to update</param>
		/// <returns>Http result code</returns>
		[HttpPut]
		[Obsolete("Pools should be configured through globals.json")]
		[Route("/api/v1/pools/{poolId}")]
		public async Task<ActionResult> UpdatePoolAsync(string poolId, [FromBody] UpdatePoolRequest update)
		{
			if (!_computeConfig.Value.Authorize(PoolAclAction.UpdatePool, User))
			{
				return Forbid(PoolAclAction.UpdatePool);
			}

			PoolId poolIdValue = new PoolId(poolId);

			if (_computeConfig.Value.VersionEnum >= ConfigVersion.PoolsInConfigFiles)
			{
				return BadRequest($"All pools are managed via config files. Pool '{poolId}' cannot be modified through the API.");
			}

			List<PoolSizeStrategyInfo>? newSizeStrategies = update.SizeStrategies?.Select(x => x.Convert()).ToList();
			List<FleetManagerInfo>? newFleetManagers = update.FleetManagers?.Select(x => x.Convert()).ToList();
			TimeSpan? conformInterval = update.ConformInterval == null ? null : TimeSpan.FromHours(update.ConformInterval.Value);
			TimeSpan? scaleOutCooldown = update.ScaleOutCooldown == null ? null : TimeSpan.FromSeconds(update.ScaleOutCooldown.Value);
			TimeSpan? scaleInCooldown = update.ScaleInCooldown == null ? null : TimeSpan.FromSeconds(update.ScaleInCooldown.Value);

			UpdatePoolConfigOptions options = new UpdatePoolConfigOptions();
			options.Name = update.Name;
			options.Condition = update.Condition;
			options.EnableAutoscaling = update.EnableAutoscaling;
			options.MinAgents = update.MinAgents;
			options.NumReserveAgents = update.NumReserveAgents;
			options.Properties = update.Properties;
			options.ConformInterval = conformInterval;
			options.ScaleOutCooldown = scaleOutCooldown;
			options.ScaleInCooldown = scaleInCooldown;
			options.SizeStrategy = update.SizeStrategy;
			options.SizeStrategies = newSizeStrategies;
			options.FleetManagers = newFleetManagers;
			options.LeaseUtilizationSettings = update.LeaseUtilizationSettings?.Convert();
			options.JobQueueSettings = update.JobQueueSettings?.Convert();
			options.ComputeQueueAwsMetricSettings = update.ComputeQueueAwsMetricSettings?.Convert();
			options.UseDefaultStrategy = update.UseDefaultStrategy;

			await _poolCollection.UpdateConfigAsync(poolIdValue, options);
			return new OkResult();
		}

		/// <summary>
		/// Delete a pool
		/// </summary>
		/// <param name="poolId">Id of the pool to delete</param>
		/// <returns>Http result code</returns>
		[HttpDelete]
		[Obsolete("Pools should be configured through globals.json")]
		[Route("/api/v1/pools/{poolId}")]
		public async Task<ActionResult> DeletePoolAsync(string poolId)
		{
			if (!_computeConfig.Value.Authorize(PoolAclAction.DeletePool, User))
			{
				return Forbid(PoolAclAction.DeletePool);
			}

			PoolId poolIdValue = new PoolId(poolId);

			if (_computeConfig.Value.VersionEnum >= ConfigVersion.PoolsInConfigFiles)
			{
				return BadRequest("All pools are managed via config files. Pool deletion through the API is disabled.");
			}

			if (!await _poolCollection.DeleteConfigAsync(poolIdValue))
			{
				return NotFound(poolIdValue);
			}
			return Ok();
		}

		/// <summary>
		/// Batch update pool properties
		/// </summary>
		/// <param name="batchUpdates">List of pools to update</param>
		/// <returns>Http result code</returns>
		[HttpPut]
		[Obsolete("Pools should be configured through globals.json")]
		[Route("/api/v1/pools")]
		public async Task<ActionResult> UpdatePoolAsync([FromBody] List<BatchUpdatePoolRequest> batchUpdates)
		{
			if (!_computeConfig.Value.Authorize(PoolAclAction.UpdatePool, User))
			{
				return Forbid(PoolAclAction.UpdatePool);
			}

			if (_computeConfig.Value.VersionEnum >= ConfigVersion.PoolsInConfigFiles)
			{
				return BadRequest("All pools are managed via config files. Batch updates through the API are disabled.");
			}

			foreach (BatchUpdatePoolRequest update in batchUpdates)
			{
				PoolId poolIdValue = new PoolId(update.Id);
				if (!await _poolCollection.UpdateConfigAsync(poolIdValue, new UpdatePoolConfigOptions { Name = update.Name, Properties = update.Properties }))
				{
					return NotFound(poolIdValue);
				}
			}
			return Ok();
		}
	}
}
