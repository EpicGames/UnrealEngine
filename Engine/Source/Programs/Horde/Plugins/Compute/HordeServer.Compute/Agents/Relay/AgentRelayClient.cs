// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using Grpc.Core;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;

namespace HordeServer.Agents.Relay;

/// <summary>
/// Relay service mirroring port mappings sent from server
/// These mappings will port forward specific ports to agents sitting behind a firewall.
/// An agent using this relay mode usually has multiple IPs assigned to allow bridging.
/// </summary>
public class AgentRelayClient
{
	/// <summary>
	/// Cooldown after an exception occurs. Primarily set to speed up tests.
	/// </summary>
	public TimeSpan CooldownOnException { get; set; } = TimeSpan.FromSeconds(5);

	/// <summary>
	/// Last received revision number after a long-polling responses has returned
	/// </summary>
	public int RevisionNumber { get; private set; } = -2;

	private readonly string _clusterId;
	private readonly string _agentId;
	private readonly List<string> _ipAddresses;
	private readonly Nftables _nftables;
	private readonly RelayRpc.RelayRpcClient _relayRpcClient;
	private readonly IClock _clock;
	private readonly ILogger _logger;
	private List<PortMapping>? _lastPortMappings; 

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="clusterId">Cluster ID this relay agent belongs to</param>
	/// <param name="agentId">Unique ID of this relay agent</param>
	/// <param name="ipAddresses">IP addresses this relay agent is listening on</param>
	/// <param name="nftables"></param>
	/// <param name="clock"></param>
	/// <param name="relayRpcClient"></param>
	/// <param name="logger"></param>
	public AgentRelayClient(string clusterId, string agentId, List<string> ipAddresses, Nftables nftables, RelayRpc.RelayRpcClient relayRpcClient, IClock clock, ILogger logger)
	{
		_clusterId = clusterId;
		_agentId = agentId;
		_ipAddresses = ipAddresses;
		_nftables = nftables;
		_relayRpcClient = relayRpcClient;
		_clock = clock;
		_logger = logger;
	}

	/// <summary>
	/// Get a list of port mappings from the server using a streaming response.
	/// Using streaming, the relay service can act immediately once new mappings are sent.
	/// This avoids excessive and repetitive polling, and in turn load on the server.
	/// </summary>
	/// <param name="cancellationToken"></param>
	/// <returns>List of new port mappings</returns>
	public async Task<List<PortMapping>?> GetPortMappingsLongPollAsync(CancellationToken cancellationToken)
	{
		_logger.LogDebug("Long polling for port mappings... (revision={Revision})", RevisionNumber);

		GetPortMappingsRequest request = new() { AgentId = _agentId, ClusterId = _clusterId, RevisionCount = RevisionNumber };
		request.IpAddresses.AddRange(_ipAddresses);
		using AsyncServerStreamingCall<GetPortMappingsResponse> cursor = _relayRpcClient.GetPortMappings(request, null, null, cancellationToken);
		await foreach (GetPortMappingsResponse response in cursor.ResponseStream.ReadAllAsync(cancellationToken))
		{
			if (RevisionNumber == response.RevisionCount)
			{
				_logger.LogDebug("Revision did not change");
				return null;
			}
			RevisionNumber = response.RevisionCount;
			return response.PortMappings.ToList();
		}

		return null;
	}

	/// <summary>
	/// Continuously listen for port mappings from server and apply them
	/// </summary>
	/// <param name="cancellationToken"></param>
	public async Task ListenForPortMappingsAsync(CancellationToken cancellationToken)
	{
		// Server cancels the long poll at its discretion
		while (!cancellationToken.IsCancellationRequested)
		{
			try
			{
				List<PortMapping>? portMappings = await GetPortMappingsLongPollAsync(cancellationToken);
				if (portMappings != null)
				{
					_logger.LogDebug("Received {NumMappings} port mappings...", portMappings.Count);
					LogPortMappingChanges(portMappings);
					await _nftables.ApplyPortForwardingAsync(portMappings);
					_lastPortMappings = portMappings;
				}
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				break;
			}
			catch (RpcException re) when (re.StatusCode == StatusCode.Cancelled)
			{
				// Re-connect as normal
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Failed getting and applying port mappings");
				await Task.Delay(CooldownOnException, CancellationToken.None);
			}
		}
	}
	
	private void LogPortMappingChanges(List<PortMapping> portMappings)
	{
		(IReadOnlySet<PortMapping> added, IReadOnlySet<PortMapping> removed) = ComparePortMappings(_lastPortMappings, portMappings);
		
		[SuppressMessage("Usage", "CA2254:Template should be a static expression")]
		void Log(string prefix, PortMapping pm)
		{
			string ports = String.Join(", ", pm.Ports.Select(p => $"{p.Protocol} {p.RelayPort} -> {p.AgentPort}"));
			string sourceIps = pm.AllowedSourceIps.Count switch
			{
				0 => "*",
				1 => pm.AllowedSourceIps.First(),
				_ => $"[{String.Join(", ", pm.AllowedSourceIps)}]"
			};
			string? createdAt = pm.CreatedAt?.ToDateTime().ToString("o");
			long? ageMs = null;
			if (pm.CreatedAt != null)
			{
				ageMs = (long)(_clock.UtcNow - pm.CreatedAt.ToDateTime()).TotalMilliseconds;
			}
			_logger.LogInformation(prefix + " {LeaseId}: {SourceIps} to {AgentIp} [{Ports}] ({CreatedAt} - {AgeMs} ms old)",
				pm.LeaseId, sourceIps, pm.AgentIp, ports, createdAt, ageMs);
		}
		
		added.ToList().ForEach(pm => Log("Added", pm));
		removed.ToList().ForEach(pm => Log("Removed", pm));
	}
	
	private class PortMappingComparer : IEqualityComparer<PortMapping>
	{
		public bool Equals(PortMapping? x, PortMapping? y)
		{
			return x?.LeaseId == y?.LeaseId;
		}
		
		public int GetHashCode(PortMapping obj)
		{
			return obj.LeaseId.GetHashCode(StringComparison.Ordinal);
		}
	}

	internal static (IReadOnlySet<PortMapping> added, IReadOnlySet<PortMapping> removed) ComparePortMappings(List<PortMapping>? oldMappings, List<PortMapping> newMappings)
	{
		PortMappingComparer comparer = new ();
		HashSet<PortMapping> oldMappingSet = new(oldMappings ?? [], comparer);
		HashSet<PortMapping> newMappingSet = new(newMappings, comparer);
		
		return (
			added: new HashSet<PortMapping>(newMappingSet.Except(oldMappingSet, comparer), comparer),
			removed: new HashSet<PortMapping>(oldMappingSet.Except(newMappingSet, comparer), comparer)
		);
	}
}