// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Immutable;
using System.Net;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Compute;
using EpicGames.Redis;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;
using StackExchange.Redis;
using Status = Grpc.Core.Status;
using StatusCode = Grpc.Core.StatusCode;

namespace HordeServer.Agents.Relay;

/// <summary>
/// Exception thrown by agent relay related code
/// </summary>
public class AgentRelayException : Exception
{
	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="message"></param>
	public AgentRelayException(string? message) : base(message)
	{
	}
}

/// <summary>
/// Result of a requested port forward for a specific agent
/// </summary>
public class PortMappingResult
{
	/// <summary>
	/// IP addresses to relays that are listening and forwarding traffic for the specified ports
	/// </summary>
	public IReadOnlyList<string> IpAddresses { get; }

	/// <summary>
	/// Mapping between relay and local agent ports
	/// </summary>
	public IReadOnlyList<Port> Ports { get; }

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="ipAddresses"></param>
	/// <param name="ports"></param>
	public PortMappingResult(IReadOnlyList<string> ipAddresses, IReadOnlyList<Port> ports)
	{
		IpAddresses = ipAddresses;
		Ports = ports;
	}
}

/// <summary>
/// Service handling relaying of agent traffic
/// The Horde agent can be started in a special mode where it will relay traffic to and from other agents.
/// It connects back to this service over gRPC and mirror the current port mappings set by the server.
/// </summary>
public sealed class AgentRelayService : RelayRpc.RelayRpcBase, IHostedService, IAsyncDisposable
{
	private record PortMappingsInfo(int Revision, List<PortMapping> PortMappings);

	private string _redisChannelUpdate = "relay/update";
	private static string KeyClusters() => "relay/clusters";
	private static string KeyPortMappings(ClusterId clusterId) => $"relay/port-mappings/{clusterId}";
	private static string KeyPortMappingRevision(ClusterId clusterId) => $"relay/port-mappings-revision/{clusterId}";
	private static string KeyUsedPorts(ClusterId clusterId) => $"relay/used-ports/{clusterId}";
	private static string KeyAgents(ClusterId clusterId) => $"relay/agents/{clusterId}";

	private readonly IRedisService _redis;
	private readonly IClock _clock;
	private readonly Tracer _tracer;
	private readonly ILogger<AgentRelayService> _logger;
	private readonly object _lock = new();
	private TaskCompletionSource _onPortMappingUpdated = new();
	private readonly ConcurrentDictionary<ClusterId, PortMappingsInfo> _clusterPortMappings = new();
	private TimeSpan _longPollTimeout = TimeSpan.FromSeconds(20);
	private TimeSpan _agentExpirationTimeout;
	private readonly AsyncTaskQueue _updateTaskQueue;
	private IAsyncDisposable? _redisSubscription;
	private int _minPort = 10000;
	private int _maxPort = 30000; // Linux ephemeral port range is usually 32768-60999 so staying below 30000 seems like a sane default
	private Random _random = new ();
	private Func<Task>? _testDelayBeforeTaskCapture;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="redis"></param>
	/// <param name="clock"></param>
	/// <param name="tracer"></param>
	/// <param name="logger"></param>
	public AgentRelayService(IRedisService redis, IClock clock, Tracer tracer, ILogger<AgentRelayService> logger)
	{
		_redis = redis;
		_clock = clock;
		_tracer = tracer;
		_logger = logger;
		_updateTaskQueue = new AsyncTaskQueue(logger);
		_agentExpirationTimeout = _longPollTimeout + TimeSpan.FromSeconds(5);
	}

	/// <inheritdoc/>
	public async ValueTask DisposeAsync()
	{
		await _updateTaskQueue.DisposeAsync();
	}

	/// <inheritdoc/>
	public async Task StartAsync(CancellationToken cancellationToken)
	{
		_redisSubscription = await SubscribeToUpdateEventAsync(OnPortMappingUpdate);
	}

	/// <inheritdoc/>
	public async Task StopAsync(CancellationToken cancellationToken)
	{
		if (_redisSubscription != null)
		{
			await _redisSubscription.DisposeAsync();
			_redisSubscription = null;
		}
		await _updateTaskQueue.FlushAsync(cancellationToken);
	}

	/// <summary>
	/// Get a list of all cluster IDs having registered a port mapping
	/// </summary>
	/// <returns>List of cluster IDs</returns>
	public async Task<HashSet<ClusterId>> GetClustersAsync()
	{
		IDatabase redis = _redis.GetDatabase();
		HashEntry[] entries = await redis.HashGetAllAsync(KeyClusters());
		return new HashSet<ClusterId>(entries.Select(e => new ClusterId(e.Name.ToString())));
	}

	/// <summary>
	/// Retrieve a specific port mapping
	/// </summary>
	/// <param name="clusterId">Cluster ID</param>
	/// <param name="leaseId">Lease ID</param>
	/// <returns>The port mapping, or null if not found</returns>
	public async Task<PortMapping?> GetPortMappingAsync(ClusterId clusterId, LeaseId leaseId)
	{
		IDatabase redis = _redis.GetDatabase();
		RedisValue value = await redis.HashGetAsync(KeyPortMappings(clusterId), leaseId.ToString());
		return value.IsNullOrEmpty ? null : PortMapping.Parser.ParseFrom(value);
	}

	/// <summary>
	/// Retrieve all current port mappings
	/// </summary>
	/// <param name="clusterId">Cluster ID</param>
	/// <returns>List of all port mappings</returns>
	public async Task<(int revision, List<PortMapping> portMappings)> GetPortMappingsAsync(ClusterId clusterId)
	{
		IDatabase redis = _redis.GetDatabase();
		ITransaction transaction = redis.CreateTransaction();
		Task<HashEntry[]> entriesTask = transaction.HashGetAllAsync(KeyPortMappings(clusterId));
		Task<RedisValue> revisionTask = transaction.StringGetAsync(KeyPortMappingRevision(clusterId));
		if (await transaction.ExecuteAsync())
		{
			HashEntry[] entries = await entriesTask;
			RedisValue revisionStr = await revisionTask;
			int revision = String.IsNullOrEmpty(revisionStr) ? 0 : Convert.ToInt32(revisionStr);
			return (revision, entries.Select(x => PortMapping.Parser.ParseFrom(x.Value)).ToList());
		}

		throw new RedisException("Transaction failed for GetPortMappingsAsync");
	}

	/// <summary>
	/// Add a new port mapping
	/// </summary>
	/// <param name="clusterId">Cluster ID this mapping is for</param>
	/// <param name="leaseId">Lease ID this mapping is for</param>
	/// <param name="clientIp">Source IP for requester. Used for access filtering. Null allow any IP to access forwarded ports</param>
	/// <param name="agentIp">What agent IP to forward all traffic to</param>
	/// <param name="ports">Ports agent is listening</param>
	/// <param name="numRetries">Number of retries before giving up</param>
	/// <returns>A port mapping with listen ports assigned</returns>
	/// <exception cref="Exception"></exception>
	public async Task<PortMapping> AddPortMappingAsync(ClusterId clusterId, LeaseId leaseId, IPAddress? clientIp, IPAddress agentIp, IList<Port> ports, int numRetries = 10)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentRelayService)}.{nameof(RemovePortMappingAsync)}");
		span.SetAttribute("clusterId", clusterId.ToString());
		span.SetAttribute("leaseId", leaseId.ToString());
		IDatabase redis = _redis.GetDatabase();

		// TODO: Add pinging to and do not consider stale agent relays
		// TODO: Check failed deserialization from Redis

		for (int retryAttempt = 0; retryAttempt < numRetries; retryAttempt++)
		{
			HashSet<int> usedPorts = (await redis.SetMembersAsync(KeyUsedPorts(clusterId))).Select(x => (int)x).ToHashSet();
			int numPorts = ports.Count;
			IReadOnlySet<int> portRange = FindAvailablePorts(_random, usedPorts, numPorts, _minPort, _maxPort);
			if (portRange.Count == 0)
			{
				throw new Exception("No ports are available");
			}

			// Create a list of ports with the newly acquired ports
			IEnumerable<Port> newPorts = portRange
				.ToList()
				.OrderBy(x => x)
				.Zip(ports, (newRelayPort, port) => new Port { RelayPort = newRelayPort, AgentPort = port.AgentPort, Protocol = port.Protocol });

			PortMapping newPortMapping = new() { LeaseId = leaseId.ToString(), AgentIp = agentIp.ToString(), CreatedAt = Timestamp.FromDateTime(_clock.UtcNow) };
			newPortMapping.Ports.AddRange(newPorts);

			if (clientIp != null)
			{
				newPortMapping.AllowedSourceIps.Add(clientIp.ToString());
			}

			ITransaction transaction = redis.CreateTransaction();
			RedisValue[] portRangeRedis = portRange.Select(x => Convert.ToString(x)).Select(x => new RedisValue(x)).ToArray();
			foreach (int port in portRange)
			{
				transaction.AddCondition(Condition.SetNotContains(KeyUsedPorts(clusterId), port));
			}

			_ = transaction.SetAddAsync(KeyUsedPorts(clusterId), portRangeRedis);
			_ = transaction.HashSetAsync(KeyPortMappings(clusterId), leaseId, newPortMapping.ToByteArray());
			_ = transaction.StringIncrementAsync(KeyPortMappingRevision(clusterId));
			_ = transaction.HashSetAsync(KeyClusters(), clusterId, _clock.UtcNow.ToFileTimeUtc());
			bool isSuccessful = await transaction.ExecuteAsync();
			
			AddPortMappingMetadataToSpan(span, newPortMapping);
			if (isSuccessful)
			{
				await PublishUpdateEventAsync(clusterId.ToString());
				span.SetAttribute("added", true);
				return newPortMapping;
			}
		}
		
		span.SetAttribute("added", false);
		throw new Exception("Unable to find an available port range");
	}

	/// <summary>
	/// Remove a port mapping for a lease
	/// </summary>
	/// <param name="clusterId">Cluster ID</param>
	/// <param name="leaseId">Lease ID to remove</param>
	/// <returns>True if successful</returns>
	public async Task<bool> RemovePortMappingAsync(ClusterId clusterId, LeaseId leaseId)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentRelayService)}.{nameof(RemovePortMappingAsync)}");
		span.SetAttribute("clusterId", clusterId.ToString());
		span.SetAttribute("leaseId", leaseId.ToString());
		
		PortMapping? portMapping = await GetPortMappingAsync(clusterId, leaseId);
		if (portMapping == null)
		{
			span.SetAttribute("removed", false);
			return false;
		}
		
		AddPortMappingMetadataToSpan(span, portMapping);

		RedisValue[] ports = portMapping.Ports.Select(x => new RedisValue(Convert.ToString(x.RelayPort))).ToArray();
		ITransaction transaction = _redis.GetDatabase().CreateTransaction();
		_ = transaction.SetRemoveAsync(KeyUsedPorts(clusterId), ports);
		_ = transaction.HashDeleteAsync(KeyPortMappings(clusterId), leaseId.ToString());
		_ = transaction.StringIncrementAsync(KeyPortMappingRevision(clusterId));
		
		span.SetAttribute("removed", true);
		return await transaction.ExecuteAsync();
	}

	/// <summary>
	/// Remove a port mapping for a lease from all clusters
	/// </summary>
	/// <param name="leaseId">Lease ID to remove</param>
	/// <returns>True if successful</returns>
	public async Task<bool> RemovePortMappingAsync(LeaseId leaseId)
	{
		ISet<ClusterId> clusterIds = await GetClustersAsync();
		bool success = false;
		foreach (ClusterId clusterId in clusterIds)
		{
			success = success || await RemovePortMappingAsync(clusterId, leaseId);
		}

		return success;
	}

	/// <summary>
	/// Request a port mapping for a given lease and agent
	/// </summary>
	/// <param name="clusterId">Cluster ID this mapping is for</param>
	/// <param name="leaseId">Lease ID this mapping is for</param>
	/// <param name="clientIp">Source IP for requester. Used for access filtering. Null allow any IP to access forwarded ports</param>
	/// <param name="agentIp">Destination IP for agent to forward traffic to</param>
	/// <param name="ports">List of ports to map</param>
	/// <returns>A result describing the port mapping</returns>
	/// <exception cref="AgentRelayException"></exception>
	public async Task<PortMappingResult> RequestPortMappingAsync(ClusterId clusterId, LeaseId leaseId, IPAddress? clientIp, IPAddress agentIp, IList<Port> ports)
	{
		List<RelayAgentInfo> agents = await GetAvailableRelayAgentsAsync(clusterId);
		if (agents.Count == 0)
		{
			throw new AgentRelayException($"No relay agents available for cluster {clusterId}");
		}

		PortMapping portMapping = await AddPortMappingAsync(clusterId, leaseId, clientIp, agentIp, ports);

		List<string> ipAddresses = agents.SelectMany(x => x.IpAddresses.ToList()).ToList();
		return new PortMappingResult(ipAddresses, portMapping.Ports);
	}

	internal async Task<List<RelayAgentInfo>> GetAvailableRelayAgentsAsync(ClusterId clusterId)
	{
		List<RelayAgentInfo> agents = await GetRelayAgentsAsync(clusterId);
		List<RelayAgentInfo> validAgents = new();
		foreach (RelayAgentInfo agent in agents)
		{
			bool hasExpired = agent.LastUpdate.ToDateTime() + _agentExpirationTimeout < _clock.UtcNow;
			if (hasExpired)
			{
				await _redis.GetDatabase().HashDeleteAsync(KeyAgents(clusterId), agent.AgentId);
			}
			else
			{
				validAgents.Add(agent);
			}
		}

		return validAgents;
	}

	private async Task<List<RelayAgentInfo>> GetRelayAgentsAsync(ClusterId clusterId)
	{
		HashEntry[] entries = await _redis.GetDatabase().HashGetAllAsync(KeyAgents(clusterId));
		return entries.Select(x => RelayAgentInfo.Parser.ParseFrom(x.Value)).ToList();
	}

	internal async Task UpdateAgentHeartbeatAsync(ClusterId clusterId, string agentRelayId, IEnumerable<IPAddress> ipAddresses)
	{
		RelayAgentInfo info = new() { AgentId = agentRelayId, LastUpdate = Timestamp.FromDateTime(_clock.UtcNow) };
		info.IpAddresses.AddRange(ipAddresses.Select(x => x.ToString()));
		await _redis.GetDatabase().HashSetAsync(KeyAgents(clusterId), agentRelayId, info.ToByteArray());
	}

	private Task PublishUpdateEventAsync(string clusterId)
	{
		return _redis.GetDatabase().PublishAsync(RedisChannel.Literal(_redisChannelUpdate), clusterId);
	}

	private async Task<IAsyncDisposable> SubscribeToUpdateEventAsync(Action<string> onUpdate)
	{
		return await _redis.GetDatabase().Multiplexer.SubscribeAsync(_redisChannelUpdate, onUpdate);
	}

	private void OnPortMappingUpdate(string clusterIdStr)
	{
		_updateTaskQueue.Enqueue(_ => OnPortMappingUpdateAsync(clusterIdStr));
	}

	private async Task OnPortMappingUpdateAsync(string clusterIdStr)
	{
		try
		{
			ClusterId clusterId = new(clusterIdStr);
			(int revision, List<PortMapping> portMappings) = await GetPortMappingsAsync(clusterId);
			lock (_lock)
			{
				_clusterPortMappings[clusterId] = new PortMappingsInfo(revision, portMappings);
				_onPortMappingUpdated.SetResult();
				_onPortMappingUpdated = new TaskCompletionSource();
			}
		}
		catch (Exception e)
		{
			_logger.LogError(e, "Failed updating port mappings");
		}
	}

	internal static IReadOnlySet<int> FindAvailablePortRange(IReadOnlySet<int> usedPorts, int numPorts, int minPort, int maxPort)
	{
		for (int start = minPort; start <= maxPort - numPorts + 1; start++)
		{
			HashSet<int> potentialPorts = Enumerable.Range(start, numPorts).ToHashSet();
			if (!potentialPorts.Overlaps(usedPorts))
			{
				return potentialPorts;
			}
		}

		return ImmutableSortedSet<int>.Empty;
	}
	
	/// <summary>
	/// Find random ports available
	/// </summary>
	/// <param name="random">Random generator</param>
	/// <param name="usedPorts">Ports already in use</param>
	/// <param name="numPorts">Number of ports to find</param>
	/// <param name="minPort">Min port number</param>
	/// <param name="maxPort">Max port number</param>
	/// <returns>A list of random but available ports</returns>
	/// <exception cref="Exception">If port assignment does not complete within max attempts</exception>
	internal static IReadOnlySet<int> FindAvailablePorts(Random random, IReadOnlySet<int> usedPorts, int numPorts, int minPort, int maxPort)
	{
		const int MaxAttempts = 10000;
		
		if (minPort > maxPort)
		{
			throw new ArgumentException("minPort must be less than or equal to maxPort");
		}
		
		int availablePortRange = maxPort - minPort + 1;
		if (numPorts > availablePortRange - usedPorts.Count)
		{
			throw new ArgumentException("The number of requested ports exceeds the available port range");
		}
		
		HashSet<int> acquiredPorts = new(numPorts);
		int numAttempts = 0;
		while (acquiredPorts.Count < numPorts)
		{
			numAttempts++;
			if (numAttempts >= MaxAttempts)
			{
				throw new Exception($"Failed to acquire {numPorts} random ports after {numAttempts} attempts. Acquired ports: {acquiredPorts.Count}");
			}
			
			int port = random.Next(minPort, maxPort + 1);
			if (usedPorts.Contains(port) || acquiredPorts.Contains(port))
			{
				continue; // Try assigning a new random port
			}
			
			acquiredPorts.Add(port);
		}
		
		return acquiredPorts;
	}

	internal void SetMinMaxPorts(int minPort, int maxPort)
	{
		_minPort = minPort;
		_maxPort = maxPort;
	}

	internal void SetTimeouts(int longPollMs, int expirationMs)
	{
		_longPollTimeout = TimeSpan.FromMilliseconds(longPollMs);
		_agentExpirationTimeout = TimeSpan.FromMilliseconds(expirationMs);
	}
	
	internal void SetRandomSeed(int seed)
	{
		_random = new Random(seed);
	}

	internal void SetRedisChannel(string channel)
	{
		_redisChannelUpdate = channel;
	}

	internal void SetTestDelayBeforeTaskCapture(Func<Task>? delay)
	{
		_testDelayBeforeTaskCapture = delay;
	}

	private static void AddPortMappingMetadataToSpan(TelemetrySpan span, PortMapping portMapping)
	{
		span.SetAttribute("leaseId", portMapping.LeaseId);
		span.SetAttribute("agentIp", portMapping.AgentIp);
		span.SetAttribute("createdAt", portMapping.CreatedAt.ToDateTimeOffset().ToUnixTimeMilliseconds());
		span.SetAttribute("relayPorts", portMapping.Ports.Select(x => x.RelayPort).ToArray());
		span.SetAttribute("agentPorts", portMapping.Ports.Select(x => x.AgentPort).ToArray());
		span.SetAttribute("protocols", portMapping.Ports.Select(x => x.Protocol.ToString()).ToArray());
		span.SetAttribute("allowedSourceIps", portMapping.AllowedSourceIps.ToArray());
	}

	/// <inheritdoc/>
	public override async Task GetPortMappings(GetPortMappingsRequest request, IServerStreamWriter<GetPortMappingsResponse> responseStream, ServerCallContext context)
	{
		if (String.IsNullOrWhiteSpace(request.ClusterId))
		{
			throw new RpcException(new Status(StatusCode.InvalidArgument, "Invalid cluster ID"));
		}

		if (String.IsNullOrWhiteSpace(request.AgentId))
		{
			throw new RpcException(new Status(StatusCode.InvalidArgument, "Invalid agent ID"));
		}

		if (request.IpAddresses.Count == 0)
		{
			throw new RpcException(new Status(StatusCode.InvalidArgument, "No IP addresses specified"));
		}

		try
		{
			ClusterId clusterId = new(request.ClusterId);
			await UpdateAgentHeartbeatAsync(clusterId, request.AgentId, request.IpAddresses.Select(IPAddress.Parse));

			// Capture notification task BEFORE reading state to avoid race condition
			// By capturing first, we always get a task that will be signaled if an update occurs.
			Task mappingUpdatedTask = _onPortMappingUpdated.Task;
			
			(int revisionCount, List<PortMapping> portMappings) = await GetPortMappingsAsync(clusterId);
			if (_testDelayBeforeTaskCapture != null)
			{
				await _testDelayBeforeTaskCapture();
			}
			if (request.RevisionCount == revisionCount || request.RevisionCount == -1)
			{
				// Client is in sync with latest revision server has or explicitly requested long-polling by setting revision to -1
				// Long-poll for any new updates...

				Task result = await Task.WhenAny(mappingUpdatedTask, Task.Delay(_longPollTimeout, context.CancellationToken));
				if (result == mappingUpdatedTask)
				{
					// Port mappings are updated. Fetch from in-memory cache to avoid every client re-reading from Redis again.
					if (!_clusterPortMappings.TryGetValue(clusterId, out PortMappingsInfo? pmi))
					{
						(revisionCount, portMappings) = await GetPortMappingsAsync(clusterId);
						pmi = new PortMappingsInfo(revisionCount, portMappings);
					}

					GetPortMappingsResponse response = new() { RevisionCount = pmi.Revision };
					response.PortMappings.AddRange(pmi.PortMappings);
					await responseStream.WriteAsync(response);
				}
			}
			else
			{
				// Client is out of sync with server
				// Immediately return an update and let the client retry the request with an updated revision count
				GetPortMappingsResponse response = new() { RevisionCount = revisionCount };
				response.PortMappings.AddRange(portMappings);
				await responseStream.WriteAsync(response);
			}
		}
		catch (OperationCanceledException) when (context.CancellationToken.IsCancellationRequested)
		{
			// Ignore cancellations
		}
		catch (InvalidOperationException ioe) when (ioe.Message.Contains("request is complete", StringComparison.Ordinal))
		{
			// Ignore write error due to request already being completed
		}
	}
}

