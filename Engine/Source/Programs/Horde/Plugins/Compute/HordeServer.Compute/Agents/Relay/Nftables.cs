// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text;
using EpicGames.Core;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Agents.Relay;

/// <summary>
/// nftables specific exception
/// </summary>
public class NftablesException : Exception
{
	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="message"></param>
	public NftablesException(string? message) : base(message)
	{
	}
}

/// <summary>
/// Class wrapping invocation of 'nft' CLI tool which controls nftables
/// nftables are used to do port forwarding for relaying traffic to agents behind a firewall
/// Requires Linux and appropriate packages installed for nft/nftables
/// </summary>
public class Nftables
{
	private const string NftablesExecutable = "nft";
	private static readonly SemaphoreSlim s_nftSemaphore = new (1, 1);

	/// <summary>
	/// Prefix the invocation of 'nft' executable with 'sudo'
	/// </summary>
	public bool RunWithSudo { get; set; } = true;

	private readonly ILogger<Nftables> _logger;
	private int? _configurableExitCode;
	private string? _configurableOutputText;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="logger"></param>
	public Nftables(ILogger<Nftables> logger)
	{
		_logger = logger;
	}

	/// <summary>
	/// Create a nullable version of this class, used for testing
	/// </summary>
	/// <param name="exitCode"></param>
	/// <param name="outputText"></param>
	/// <param name="logger"></param>
	/// <returns>Nullable Nftables</returns>
	internal static Nftables CreateNull(int exitCode = 0, string outputText = "", ILogger<Nftables>? logger = null)
	{
		return new Nftables(logger ?? NullLogger<Nftables>.Instance)
		{
			_configurableExitCode = exitCode,
			_configurableOutputText = outputText
		};
	}

	/// <summary>
	/// Initialize nftables
	/// </summary>
	public async Task InitializeAsync(CancellationToken cancellationToken)
	{
		string version = await GetNftablesVersionAsync(cancellationToken);
		_logger.LogInformation("nftables version: {Version}", version);
	}

	/// <summary>
	/// Configure nftables to relay traffic through port forwarding
	/// </summary>
	public async Task ApplyPortForwardingAsync(List<PortMapping> portMappings)
	{
		LogPortMappings(portMappings);

		using CancellationTokenSource cts = new(TimeSpan.FromSeconds(3));
		(int exitCode, string output) = await ExecuteNftAsync(new List<string>() { "--file", "-" }, GenerateNftFile(portMappings), cts.Token);
		if (exitCode != 0)
		{
			throw new NftablesException($"Unable to apply port forwarding. Exit code {exitCode}. Output: {output}");
		}
	}

	internal async Task<(int exitCode, string output)> ExecuteNftAsync(List<string> arguments, string? stdin = null, CancellationToken cancellationToken = default)
	{
		string executable = NftablesExecutable;
		List<string> argumentsCopy = new(arguments);
		if (RunWithSudo)
		{
			executable = "sudo";
			argumentsCopy.Insert(0, NftablesExecutable);
		}

		if (_configurableExitCode != null || _configurableOutputText != null)
		{
			return (_configurableExitCode ?? 0, _configurableOutputText ?? "");
		}
		
		await s_nftSemaphore.WaitAsync(cancellationToken);
		try
		{
			byte[]? stdinData = stdin != null ? Encoding.UTF8.GetBytes(stdin) : null;
			using ManagedProcessGroup processGroup = new();
			using ManagedProcess process = new(processGroup, executable, CommandLineArguments.Join(argumentsCopy), null, null, stdinData, ProcessPriorityClass.Normal);
			await process.WaitForExitAsync(cancellationToken);
			string outputText = await process.StdOutText.ReadToEndAsync(cancellationToken);
			return (process.ExitCode, outputText);
		}
		finally
		{
			s_nftSemaphore.Release();
		}
	}

	private void LogPortMappings(List<PortMapping> leaseMappings)
	{
		if (_logger.IsEnabled(LogLevel.Debug))
		{
			foreach (PortMapping rlm in leaseMappings)
			{
				foreach (Port rlmPort in rlm.Ports)
				{
					_logger.LogDebug("Applying {LeaseId} {Protocol} {RelayPort} -> {AgentIp}:{AgentPort}",
						rlm.LeaseId, rlmPort.Protocol, rlmPort.RelayPort, rlm.AgentIp, rlmPort.AgentPort);
				}
			}
		}
	}

	private async Task<string> GetNftablesVersionAsync(CancellationToken cancellationToken)
	{
		if (!OperatingSystem.IsLinux())
		{
			throw new NftablesException("nftables can only run under Linux");
		}

		(int exitCode, string output) = await ExecuteNftAsync(new List<string> { "--version" }, cancellationToken: cancellationToken);
		if (exitCode != 0)
		{
			throw new NftablesException("Unable to get nftables version. Is 'nft' tool installed?");
		}

		return output;
	}

	internal static List<string> GenerateNftRules(List<PortMapping> leaseMappings)
	{
		return leaseMappings.SelectMany(x => x.Ports, (mapping, port) =>
		{
			string protocol = port.Protocol switch
			{
				PortProtocol.Udp => "udp",
				_ => "tcp"
			};

			string sourceIps = "";
			if (mapping.AllowedSourceIps.Count > 0)
			{
				sourceIps = "ip saddr { " + String.Join(", ", mapping.AllowedSourceIps) + " } ";
			}

			return $"{sourceIps}{protocol} dport {port.RelayPort} dnat to {mapping.AgentIp.ToString()}:{port.AgentPort} comment \"leaseId={mapping.LeaseId}\"";
		}).ToList();
	}

	internal static string GenerateNftFile(List<PortMapping> leaseMappings)
	{
		StringBuilder sb = new();
		sb.Append("table ip horde\n");
		sb.Append("delete table ip horde\n");
		sb.Append("table ip horde {\n");
		sb.Append("  chain prerouting {\n");
		sb.Append("    type nat hook prerouting priority -100; policy accept;\n");
		sb.AppendJoin('\n', GenerateNftRules(leaseMappings).Select(x => $"    {x}"));
		sb.Append('\n');
		sb.Append("  }\n");
		sb.Append("  chain postrouting {\n");
		sb.Append("    type nat hook postrouting priority 100; policy accept;\n");
		sb.Append("    masquerade\n");
		sb.Append("  }\n");
		sb.Append("}\n");
		return sb.ToString();
	}
}