// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using HordeServer.Server;
using StackExchange.Redis;

namespace HordeServer.Agents
{
	/// <summary>
	/// Tracks session conflict counts per agent in a Redis sorted set.
	/// Consumed by NotificationService to produce a 12-hour Slack report.
	/// </summary>
	public class SessionConflictService
	{
		static readonly RedisKey s_redisKey = new("session-conflicts");
		static readonly TimeSpan s_keyTtl = TimeSpan.FromHours(48);

		readonly IRedisService _redisService;

		/// <summary>
		/// Constructor
		/// </summary>
		public SessionConflictService(IRedisService redisService)
			=> _redisService = redisService;

		/// <summary>
		/// Record a session conflict for the given agent.
		/// </summary>
		/// <returns>The total conflict count for this agent since the last drain.</returns>
		public async Task<long> RecordConflictAsync(AgentId agentId, CancellationToken cancellationToken = default)
		{
			_ = cancellationToken; // Redis client doesn't accept CT directly; kept for API consistency
			IDatabase db = _redisService.GetDatabase();
			long count = (long)await db.SortedSetIncrementAsync(s_redisKey, agentId.ToString(), 1.0);
			await db.KeyExpireAsync(s_redisKey, s_keyTtl, CommandFlags.FireAndForget);
			return count;
		}

		/// <summary>
		/// Drains all recorded conflicts and clears the set.
		/// Returns each agent ID paired with its total mismatch count since the last drain.
		/// </summary>
		public async Task<List<(AgentId Id, int Count)>> DrainConflictsAsync(CancellationToken cancellationToken = default)
		{
			_ = cancellationToken;
			IDatabase db = _redisService.GetDatabase();

			ITransaction transaction = db.CreateTransaction();
			Task<SortedSetEntry[]> entriesTask = transaction.SortedSetRangeByScoreWithScoresAsync(s_redisKey);
			_ = transaction.KeyDeleteAsync(s_redisKey, CommandFlags.FireAndForget);

			if (!await transaction.ExecuteAsync())
			{
				return [];
			}

			SortedSetEntry[] entries = await entriesTask;
			return entries.Select(e => (new AgentId(e.Element.ToString()), (int)e.Score)).ToList();
		}
	}
}
