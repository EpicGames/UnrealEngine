// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Sessions;

namespace HordeServer.Agents.Sessions
{
	/// <summary>
	/// Information about an agent session.
	/// </summary>
	public interface ISession
	{
		/// <summary>
		/// Unique id for this session
		/// </summary>
		public SessionId Id { get; }

		/// <summary>
		/// The agent id
		/// </summary>
		public AgentId AgentId { get; }

		/// <summary>
		/// Start time for this session
		/// </summary>
		public DateTime StartTime { get; }

		/// <summary>
		/// Finishing time for this session
		/// </summary>
		public DateTime? FinishTime { get; }

		/// <summary>
		/// Version of the agent software
		/// </summary>
		public string Version { get; }
	}
}
