// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Agents.Sessions
{
	/// <summary>
	/// Actions valid for agent sessions
	/// </summary>
	public static class SessionAclAction
	{
		/// <summary>
		/// Granted to agents to call CreateSession, which returns a bearer token identifying themselves valid to call UpdateSesssion via gRPC.
		/// </summary>
		public static AclAction CreateSession { get; } = new AclAction("CreateSession");

		/// <summary>
		/// Allows viewing information about an agent session
		/// </summary>
		public static AclAction ViewSession { get; } = new AclAction("ViewSession");
	}
}
