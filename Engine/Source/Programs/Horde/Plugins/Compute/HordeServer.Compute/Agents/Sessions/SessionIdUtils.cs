// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Sessions;
using HordeServer.Utilities;

namespace HordeServer.Agents.Sessions
{
	/// <summary>
	/// Server helper methods for <see cref="SessionId"/>
	/// </summary>
	static class SessionIdUtils
	{
		/// <summary>
		/// Creates a new <see cref="SessionId"/>
		/// </summary>
		public static SessionId GenerateNewId() => new SessionId(BinaryIdUtils.CreateNew());
	}
}
