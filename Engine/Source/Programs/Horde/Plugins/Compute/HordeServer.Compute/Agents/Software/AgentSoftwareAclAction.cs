// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Agents.Software
{
	/// <summary>
	/// ACL actions for agent software
	/// </summary>
	public static class AgentSoftwareAclAction
	{
		/// <summary>
		/// Ability to upload new versions of the agent software
		/// </summary>
		public static AclAction UploadSoftware { get; } = new AclAction("UploadSoftware");

		/// <summary>
		/// Ability to download the agent software
		/// </summary>
		public static AclAction DownloadSoftware { get; } = new AclAction("DownloadSoftware");

		/// <summary>
		/// Ability to delete agent software
		/// </summary>
		public static AclAction DeleteSoftware { get; } = new AclAction("DeleteSoftware");
	}
}
