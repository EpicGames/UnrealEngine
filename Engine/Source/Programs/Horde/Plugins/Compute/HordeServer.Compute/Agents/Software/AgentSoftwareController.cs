// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Tools;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using Microsoft.Net.Http.Headers;

namespace HordeServer.Agents.Software
{
	/// <summary>
	/// Information about an agent software channel
	/// </summary>
	public class GetAgentSoftwareChannelResponse
	{
		/// <summary>
		/// Version number of this software
		/// </summary>
		public string? Version { get; set; }
	}

	/// <summary>
	/// Controller for the /api/v1/agentsoftware endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class AgentSoftwareController : ControllerBase
	{
		private readonly IToolCollection _toolCollection;
		private readonly IClock _clock;
		private readonly IOptionsSnapshot<ComputeConfig> _computeConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentSoftwareController(IToolCollection toolCollection, IClock clock, IOptionsSnapshot<ComputeConfig> computeConfig)
		{
			_toolCollection = toolCollection;
			_clock = clock;
			_computeConfig = computeConfig;
		}

		/// <summary>
		/// Finds all uploaded software matching the given criteria
		/// </summary>
		/// <returns>Http response</returns>
		[HttpGet]
		[Obsolete("Agent software is now stored as a tool. This endpoint exists for backwards compatibility, but will be removed in the future.")]
		[Route("/api/v1/agentsoftware/default")]
		[ProducesResponseType(typeof(GetAgentSoftwareChannelResponse), 200)]
		public async Task<ActionResult<object>> FindSoftwareAsync()
		{
			if (!_computeConfig.Value.Authorize(AgentSoftwareAclAction.DownloadSoftware, User))
			{
				return Forbid();
			}

			ITool? tool = await _toolCollection.GetAsync(AgentExtensions.AgentToolId);
			if (tool == null)
			{
				return NotFound("No agent software tool is currently registered");
			}

			IToolDeployment? deployment = tool.GetCurrentDeployment(1.0, _clock.UtcNow);
			if (deployment == null)
			{
				return NotFound("No deployment currently set for agent software");
			}

			return new GetAgentSoftwareChannelResponse { Version = deployment.Version };
		}

		/// <summary>
		/// Gets the zip file for a specific channel
		/// </summary>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Http response</returns>
		[HttpGet]
		[Obsolete("Agent software is now stored as a tool. This endpoint exists for backwards compatibility, but will be removed in the future.")]
		[Route("/api/v1/agentsoftware/default/zip")]
		public async Task<ActionResult> GetArchiveAsync(CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.Authorize(AgentSoftwareAclAction.DownloadSoftware, User))
			{
				return Forbid();
			}

			ITool? tool = await _toolCollection.GetAsync(AgentExtensions.AgentToolId, cancellationToken);
			if (tool == null)
			{
				return NotFound("No agent software tool is currently registered");
			}

			IToolDeployment? deployment = tool.GetCurrentDeployment(1.0, _clock.UtcNow);
			if (deployment == null)
			{
				return NotFound("No deployment currently set for agent software");
			}

			Stream stream = await deployment.OpenZipStreamAsync(cancellationToken);
			return new FileStreamResult(stream, new MediaTypeHeaderValue("application/octet-stream")) { FileDownloadName = $"HordeAgent.zip" };
		}
	}
}
