// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Agents;
using HordeServer.Agents.Telemetry;

namespace HordeServer.Agents
{
	/// <summary>
	/// Record used for Agent telemetry.
	/// </summary>
	[AnalyticsTableGen]
	[TelemetryEvent(DefaultEventName)]
	[Table("horde.state_agent_telemetry", Schema = "ingest")]
	public record AgentTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The ID of the agent we're reporting telemetry from.
		/// </summary>
		[Column("agent_id")]
		public AgentId AgentId { get; init; }

		/// <summary>
		/// The user cpu usage.
		/// </summary>
		[Column("user_cpu")]
		public float UserCpu { get; init; }

		/// <summary>
		/// The idle cpu usage.
		/// </summary>
		[Column("idle_cpu")]
		public float IdleCpu { get; init; }

		/// <summary>
		/// The system cpu usage.
		/// </summary>
		[Column("system_cpu")]
		public float SystemCpu { get; init; }

		/// <summary>
		/// Free memory in MB.
		/// </summary>
		[Column("free_ram_mb")]
		public int FreeRamMb { get; init; }

		/// <summary>
		/// Used memory in MB.
		/// </summary>
		[Column("used_ram_mb")]
		public int UsedRamMb { get; init; }

		/// <summary>
		/// Total memory in MB.
		/// </summary>
		[Column("total_ram_mb")]
		public int TotalRamMb { get; init; }

		/// <summary>
		/// The agent's working disk free space in MB.
		/// </summary>
		[Column("working_free_disk_mb")]
		public long WorkingFreeDiskMb { get; init; }

		/// <summary>
		/// The agent's working disk total space in MB.
		/// </summary>
		[Column("working_total_disk_mb")]
		public long WorkingTotalDiskMb { get; init; }

		/// <summary>
		/// The system's free disk in MB.
		/// </summary>
		[Column("system_free_disk_mb")]
		public long SystemFreeDiskMb { get; init; }

		/// <summary>
		/// The system's total disk in MB.
		/// </summary>
		[Column("system_total_disk_mb")]
		public long SystemTotalDiskMb { get; init; }

		/// <summary>
		/// Default event name for the Agent Telemetry event.
		/// </summary>
		public const string DefaultEventName = "State.Agent.Telemetry";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// Constructor from pre-existing NewAgentTelemetry object
		/// </summary>
		/// <param name="telemetry">The telemetry object used as a basis for this telemetry record</param>
		/// <param name="agentId">The agent's ID</param>
		public AgentTelemetry(NewAgentTelemetry telemetry, AgentId agentId)
			: base(DefaultEventName, CurrentSchemaVersion)
		{
			UserCpu = telemetry.UserCpuPct;
			IdleCpu = telemetry.IdleCpuPct;
			SystemCpu = telemetry.SystemCpuPct;
			FreeRamMb = telemetry.FreeRamMb;
			UsedRamMb = telemetry.UsedRamMb;
			TotalRamMb = telemetry.TotalRamMb;
			WorkingFreeDiskMb = telemetry.WorkingFreeDiskMb;
			WorkingTotalDiskMb = telemetry.WorkingTotalDiskMb;
			SystemFreeDiskMb = telemetry.SystemFreeDiskMb;
			SystemTotalDiskMb = telemetry.SystemTotalDiskMb;

			AgentId = agentId;
		}

		private const string ORMPlaceholder = "_";

		/// <summary>
		/// Default constructor for ORM mapping.
		/// </summary>
		public AgentTelemetry()
			: base(DefaultEventName, CurrentSchemaVersion)
		{
			UserCpu = 0;
			IdleCpu = 0;
			SystemCpu = 0;
			FreeRamMb = 0;
			UsedRamMb = 0;
			TotalRamMb = 0;
			WorkingFreeDiskMb = 0;
			WorkingTotalDiskMb = 0;
			SystemFreeDiskMb = 0;
			SystemTotalDiskMb = 0;

			AgentId = new AgentId(ORMPlaceholder);
		}
	}
}