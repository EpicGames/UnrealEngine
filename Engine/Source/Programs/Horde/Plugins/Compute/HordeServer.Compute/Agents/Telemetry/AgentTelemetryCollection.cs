// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Agents.Telemetry
{
	/// <summary>
	/// Collection which writes agent data to a MongoDB collection
	/// </summary>
	public sealed class AgentTelemetryCollection : IAgentTelemetryCollection, IHostedService, IAsyncDisposable
	{
		class AgentTelemetryDocument : IAgentTelemetry
		{
			public ObjectId Id { get; set; }

			[BsonElement("tm")]
			public DateTime TimeUtc { get; set; }

			[BsonElement("agt")]
			public AgentId AgentId { get; set; }

			[BsonElement("cpu_usr")]
			public float UserCpu { get; set; }

			[BsonElement("cpu_sys")]
			public float SystemCpu { get; set; }

			[BsonElement("cpu_idle")]
			public float IdleCpu { get; set; }

			[BsonElement("ram_tot")]
			public int TotalRam { get; set; }

			[BsonElement("ram_free")]
			public int FreeRam { get; set; }

			[BsonElement("ram_used")]
			public int UsedRam { get; set; }

			[BsonElement("hdd_tot")]
			public long WorkingTotalDisk { get; set; }

			[BsonElement("hdd_free")]
			public long WorkingFreeDisk { get; set; }

			[BsonElement("sys_hdd_tot")]
			public long SystemTotalDisk { get; set; }

			[BsonElement("sys_hdd_free")]
			public long SystemFreeDisk { get; set; }
		}

		static TimeSpan RetainDays { get; } = TimeSpan.FromDays(30.0);

		readonly IClock _clock;
		readonly IMongoCollection<AgentTelemetryDocument> _collection;
#pragma warning disable CA2213 // False positive? _writer is disposed in DisposeAsync.
		readonly MongoBufferedWriter<AgentTelemetryDocument> _writer;
#pragma warning restore CA2213
		readonly ITicker _cleanupTicker;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentTelemetryCollection(IMongoService mongoService, IClock clock, ILogger<AgentTelemetryCollection> logger)
		{
			_clock = clock;

			List<MongoIndex<AgentTelemetryDocument>> agentIndexes = new List<MongoIndex<AgentTelemetryDocument>>();
			agentIndexes.Add(MongoIndex.Create<AgentTelemetryDocument>(x => x.Descending(x => x.TimeUtc)));
			agentIndexes.Add(MongoIndex.Create<AgentTelemetryDocument>(x => x.Descending(x => x.AgentId).Descending(x => x.TimeUtc)));
			_collection = mongoService.GetCollection<AgentTelemetryDocument>("Telemetry.Agents", agentIndexes);

			_writer = new MongoBufferedWriter<AgentTelemetryDocument>(_collection, logger);
			_cleanupTicker = clock.AddSharedTicker<AgentTelemetryCollection>(TimeSpan.FromHours(4.0), CleanupAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _writer.StartAsync();
			await _cleanupTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cleanupTicker.StopAsync();
			await _writer.StopAsync(cancellationToken);
		}

		async ValueTask CleanupAsync(CancellationToken cancellationToken)
		{
			DateTime baseTime = _clock.UtcNow - RetainDays;
			DeleteResult result = await _collection.DeleteManyAsync(x => x.TimeUtc < baseTime, cancellationToken);
			_logger.LogInformation("Deleted {NumItems} agent telemetry records", result.DeletedCount);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await FlushAsync(default);
			await _cleanupTicker.DisposeAsync();
			await _writer.DisposeAsync();
		}

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken)
			=> _writer.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public void Add(AgentId agentId, NewAgentTelemetry telemetry)
		{
			AgentTelemetryDocument document = new AgentTelemetryDocument();
			document.AgentId = agentId;
			document.TimeUtc = _clock.UtcNow;
			document.UserCpu = telemetry.UserCpuPct;
			document.IdleCpu = telemetry.IdleCpuPct;
			document.SystemCpu = telemetry.SystemCpuPct;
			document.FreeRam = telemetry.FreeRamMb;
			document.UsedRam = telemetry.UsedRamMb;
			document.TotalRam = telemetry.TotalRamMb;
			document.WorkingFreeDisk = telemetry.WorkingFreeDiskMb;
			document.WorkingTotalDisk = telemetry.WorkingTotalDiskMb;
			document.SystemTotalDisk = telemetry.SystemTotalDiskMb;
			document.SystemFreeDisk = telemetry.SystemFreeDiskMb;
			_writer.Write(document);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IAgentTelemetry>> FindAsync(AgentId agentId, DateTime minTimeUtc, DateTime maxTimeUtc, CancellationToken cancellationToken = default)
		{
			return await _collection.Find(x => x.AgentId == agentId && (x.TimeUtc >= minTimeUtc && x.TimeUtc <= maxTimeUtc)).ToListAsync(cancellationToken);
		}
	}
}
