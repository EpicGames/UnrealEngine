// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Agents.Telemetry
{
	/// <summary>
	/// Telemetry for an agent
	/// </summary>
	public interface IAgentTelemetry
	{
		/// <summary>
		/// Time that the data was captured
		/// </summary>
		DateTime TimeUtc { get; set; }

		/// <summary>
		/// Percentage of time the CPU was busy executing code in user space
		/// </summary>
		float UserCpu { get; set; }

		/// <summary>
		/// Percentage of time the CPU was busy executing code in kernel space
		/// </summary>
		float SystemCpu { get; set; }

		/// <summary>
		/// Percentage of time the CPU was idling
		/// </summary>
		float IdleCpu { get; set; }

		/// <summary>
		/// Total memory installed (megabytes)
		/// </summary>
		int TotalRam { get; set; }

		/// <summary>
		/// Available memory (megabytes)
		/// </summary>
		int FreeRam { get; set; }

		/// <summary>
		/// Used memory (megabytes)
		/// </summary>
		int UsedRam { get; set; }

		/// <summary>
		/// Total disk space (megabytes)
		/// </summary>
		long WorkingTotalDisk { get; set; }

		/// <summary>
		/// Free disk space (megabytes)
		/// </summary>
		long WorkingFreeDisk { get; set; }

		/// <summary>
		/// Total disk space (megabytes)
		/// </summary>
		long SystemTotalDisk { get; set; }

		/// <summary>
		/// Free disk space (megabytes)
		/// </summary>
		long SystemFreeDisk { get; set; }
	}
}
