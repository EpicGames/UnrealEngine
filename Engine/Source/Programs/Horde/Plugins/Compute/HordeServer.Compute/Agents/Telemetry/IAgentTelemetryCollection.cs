// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;

namespace HordeServer.Agents.Telemetry
{
	/// <summary>
	/// Interface for a collection of agent telemetry documents
	/// </summary>
	public interface IAgentTelemetryCollection
	{
		/// <summary>
		/// Adds a new telemetry event
		/// </summary>
		void Add(AgentId agentId, NewAgentTelemetry telemetry);

		/// <summary>
		/// Finds telemetry samples for an agent in a particular time range
		/// </summary>
		/// <param name="agentId">Agent identifier</param>
		/// <param name="minTimeUtc">Minimum time to return data for</param>
		/// <param name="maxTimeUtc">Maximum time to return data for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IReadOnlyList<IAgentTelemetry>> FindAsync(AgentId agentId, DateTime minTimeUtc, DateTime maxTimeUtc, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// New telemetry data
	/// </summary>
	public record class NewAgentTelemetry(float UserCpuPct, float IdleCpuPct, float SystemCpuPct, int FreeRamMb, int UsedRamMb, int TotalRamMb, long WorkingFreeDiskMb, long WorkingTotalDiskMb, long SystemFreeDiskMb, long SystemTotalDiskMb);
}
