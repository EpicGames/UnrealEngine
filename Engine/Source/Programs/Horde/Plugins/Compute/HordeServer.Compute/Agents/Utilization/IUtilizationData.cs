// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Streams;

namespace HordeServer.Agents.Utilization
{
	/// <summary>
	/// Information about the utilization of a pool
	/// </summary>
	public interface IUtilizationData
	{
		/// <summary>
		/// Start time for the bucket
		/// </summary>
		public DateTime StartTime { get; }

		/// <summary>
		/// Finish time for this bucket
		/// </summary>
		public DateTime FinishTime { get; }

		/// <summary>
		/// Number of agents in total
		/// </summary>
		public int NumAgents { get; }

		/// <summary>
		/// Breakdown of utilization by pool
		/// </summary>
		public IReadOnlyList<IPoolUtilizationData> Pools { get; }

		/// <summary>
		/// Amount of time that agents were hibernating
		/// </summary>
		public double HibernatingTime { get; }

		/// <summary>
		/// Total time spent running administrative tasks (conform, etc...)
		/// </summary>
		public double AdminTime { get; }
	}

	/// <summary>
	/// Concrete implementation of <see cref="IUtilizationData"/>
	/// </summary>
	public sealed class UtilizationData : IUtilizationData
	{
		/// <inheritdoc/>
		public DateTime StartTime { get; set; }

		/// <inheritdoc/>
		public DateTime FinishTime { get; set; }

		/// <inheritdoc/>
		public int NumAgents { get; set; }

		/// <inheritdoc/>
		List<PoolUtilizationData> Pools { get; set; } = new List<PoolUtilizationData>();
		IReadOnlyList<IPoolUtilizationData> IUtilizationData.Pools => Pools;

		Dictionary<PoolId, PoolUtilizationData> PoolsLookup { get; set; } = new Dictionary<PoolId, PoolUtilizationData>();

		/// <inheritdoc/>
		public double HibernatingTime { get; set; }

		/// <inheritdoc/>
		public double AdminTime { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="startTime"></param>
		/// <param name="finishTime"></param>
		public UtilizationData(DateTime startTime, DateTime finishTime)
		{
			StartTime = startTime;
			FinishTime = finishTime;
		}

		/// <summary>
		/// Adds a new pool
		/// </summary>
		/// <param name="poolId">The pool id</param>
		/// <returns>Telemetry for the given pool</returns>
		public PoolUtilizationData FindOrAddPool(PoolId poolId)
		{
			PoolUtilizationData? pool;
			if (!PoolsLookup.TryGetValue(poolId, out pool))
			{
				pool = new PoolUtilizationData(poolId);
				Pools.Add(pool);
				PoolsLookup.Add(poolId, pool);
			}
			return pool;
		}
	}

	/// <summary>
	/// Information about the utilization of a pool
	/// </summary>
	public interface IPoolUtilizationData
	{
		/// <summary>
		/// Pool containing the work to execute
		/// </summary>
		public PoolId PoolId { get; }

		/// <summary>
		/// Number of agents in this pool
		/// </summary>
		public int NumAgents { get; }

		/// <summary>
		/// The stream executing work. If this is null, the time accounts for the machine executing work in another pool.
		/// </summary>
		public IReadOnlyList<IStreamUtilizationData> Streams { get; }

		/// <summary>
		/// Amount of time spent running 
		/// </summary>
		public double AdminTime { get; }

		/// <summary>
		/// Amount of time that agents were hibernating
		/// </summary>
		public double HibernatingTime { get; }

		/// <summary>
		/// Amount of time spent by agents in this pool servicing other pools
		/// </summary>
		public double OtherTime { get; }
	}

	/// <summary>
	/// Concrete implementation of <see cref="IPoolUtilizationData"/>
	/// </summary>
	public sealed class PoolUtilizationData : IPoolUtilizationData
	{
		/// <inheritdoc/>
		public PoolId PoolId { get; set; }

		/// <inheritdoc/>
		public int NumAgents { get; set; }

		/// <inheritdoc/>
		List<StreamUtilizationData> Streams { get; set; } = new List<StreamUtilizationData>();
		IReadOnlyList<IStreamUtilizationData> IPoolUtilizationData.Streams => Streams;

		Dictionary<StreamId, StreamUtilizationData> StreamLookup { get; set; } = new Dictionary<StreamId, StreamUtilizationData>();

		/// <inheritdoc/>
		public double AdminTime { get; set; }

		/// <inheritdoc/>
		public double HibernatingTime { get; set; }

		/// <inheritdoc/>
		public double OtherTime { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="poolId"></param>
		public PoolUtilizationData(PoolId poolId)
		{
			PoolId = poolId;
		}

		/// <summary>
		/// Adds a stream to the object
		/// </summary>
		/// <param name="streamId"></param>
		public StreamUtilizationData FindOrAddStream(StreamId streamId)
		{
			StreamUtilizationData? stream;
			if (!StreamLookup.TryGetValue(streamId, out stream))
			{
				stream = new StreamUtilizationData(streamId);
				Streams.Add(stream);
				StreamLookup.Add(streamId, stream);
			}
			return stream;
		}
	}

	/// <summary>
	/// Utilization of a pool for a particular stream
	/// </summary>
	public interface IStreamUtilizationData
	{
		/// <summary>
		/// The stream id
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// Number of machine hours spent executing work for this stream
		/// </summary>
		public double Time { get; }
	}

	/// <summary>
	/// Concrete implementation of <see cref="IStreamUtilizationData"/>
	/// </summary>
	public sealed class StreamUtilizationData : IStreamUtilizationData
	{
		/// <inheritdoc/>
		public StreamId StreamId { get; set; }

		/// <inheritdoc/>
		public double Time { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="streamId"></param>
		public StreamUtilizationData(StreamId streamId)
		{
			StreamId = streamId;
		}
	}
}
