// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Agents.Utilization
{
	/// <summary>
	/// Collection of utilization collection
	/// </summary>
	public interface IUtilizationDataCollection
	{
		/// <summary>
		/// Adds entries for the given utilization
		/// </summary>
		/// <param name="data">Telemetry data to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task AddUtilizationDataAsync(IUtilizationData data, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds utilization data matching the given criteria
		/// </summary>
		/// <param name="startTimeUtc">Start time to query utilization for</param>
		/// <param name="finishTimeUtc">Finish time to query utilization for</param>
		/// <param name="count">Maximum number of samples to return. If no time range is specified, this will return the most recent entries.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The utilization data</returns>
		Task<IReadOnlyList<IUtilizationData>> GetUtilizationDataAsync(DateTime? startTimeUtc = null, DateTime? finishTimeUtc = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds utilization data matching the given criteria
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The latest utilization data</returns>
		Task<IUtilizationData?> GetLatestUtilizationDataAsync(CancellationToken cancellationToken = default);
	}
}
