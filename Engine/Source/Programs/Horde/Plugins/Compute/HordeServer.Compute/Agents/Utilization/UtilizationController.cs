// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Telemetry;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Agents.Utilization
{
	/// <summary>
	/// Controller for the /api/v1/utilization endpoint.
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public sealed class UtilizationController : ControllerBase
	{
		readonly IUtilizationDataCollection _utilizationDataCollection;

		/// <summary>
		/// Constructor
		/// </summary>
		public UtilizationController(IUtilizationDataCollection utilizationDataCollection)
		{
			_utilizationDataCollection = utilizationDataCollection;
		}

		/// <summary>
		/// Gets utilization data for a time range.
		/// </summary>
		/// <param name="after">Start time for the search.</param>
		/// <param name="before">End time for the search.</param>
		/// <param name="count">Maximum number of samples to return. If the search range is open ended and no count is specified, the server will return a capped list of results.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Route("/api/v1/utilization")]
		public async Task<ActionResult<List<GetUtilizationDataResponse>>> GetUtilizationDataAsync([FromQuery] DateTime? after = null, [FromQuery] DateTime? before = null, [FromQuery] int? count = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<IUtilizationData> data = await _utilizationDataCollection.GetUtilizationDataAsync(after, before, count, cancellationToken);
			return data.ConvertAll(CreateTelemetryResponse);
		}

		/// <summary>
		/// Gets a collection of utilization data including an endpoint and a range
		/// </summary>
		[HttpGet]
		[Obsolete("Use the /api/v1/utilization endpoint instead.")]
		[Route("/api/v1/reports/utilization/{endDate}")]
		public async Task<ActionResult<List<GetUtilizationDataResponse>>> GetStreamUtilizationDataAsync(DateTime endDate, [FromQuery(Name = "Range")] int range, [FromQuery(Name = "TzOffset")] int? tzOffset, CancellationToken cancellationToken = default)
		{
			// Logic here is a bit messy. The client is always passing in a date at midnight
			// If user passes in 12/1/2020 into the date with range of 1, the range should be 12/1/2020:00:00:00 to 12/1/2020:23:59:59
			// Range 2 should be 11/30/2020:00:00:00 to 12/1/2020:23:59:59
			int offset = tzOffset ?? 0;
			DateTimeOffset endDateOffset = new DateTimeOffset(endDate, TimeSpan.FromHours(offset)).Add(new TimeSpan(23, 59, 59));
			DateTimeOffset startDateOffset = endDate.Subtract(new TimeSpan(range - 1, 0, 0, 0));

			IReadOnlyList<IUtilizationData> data = await _utilizationDataCollection.GetUtilizationDataAsync(startDateOffset.UtcDateTime, endDateOffset.UtcDateTime, cancellationToken: cancellationToken);

			return data.ConvertAll(CreateTelemetryResponse);
		}

		static GetUtilizationDataResponse CreateTelemetryResponse(IUtilizationData telemetry)
		{
			GetUtilizationDataResponse response = new GetUtilizationDataResponse();
			response.StartTime = telemetry.StartTime;
			response.FinishTime = telemetry.FinishTime;
			response.AdminTime = telemetry.AdminTime;
			response.HibernatingTime = telemetry.HibernatingTime;
			response.NumAgents = telemetry.NumAgents;

			response.Pools.AddRange(telemetry.Pools.Select(CreatePoolTelemetryResponse));
			return response;
		}

		static GetUtilizationPoolDataResponse CreatePoolTelemetryResponse(IPoolUtilizationData pool)
		{
			GetUtilizationPoolDataResponse response = new GetUtilizationPoolDataResponse();
			response.PoolId = pool.PoolId;
			response.NumAgents = pool.NumAgents;
			response.AdminTime = pool.AdminTime;
			response.HibernatingTime = pool.HibernatingTime;
			response.OtherTime = pool.OtherTime;
			response.Streams.AddRange(pool.Streams.Select(CreateStreamTelemetryResponse));
			return response;
		}

		static GetUtilizationStreamDataResponse CreateStreamTelemetryResponse(IStreamUtilizationData stream)
		{
			GetUtilizationStreamDataResponse response = new GetUtilizationStreamDataResponse();
			response.StreamId = stream.StreamId;
			response.Time = stream.Time;
			return response;
		}
	}
}
