// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Streams;
using HordeServer.Server;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Agents.Utilization
{
	/// <summary>
	/// Collection of utilization data
	/// </summary>
	public class UtilizationDataCollection : IUtilizationDataCollection
	{
		class UtilizationDocument : IUtilizationData
		{
			public DateTime StartTime { get; set; }
			public DateTime FinishTime { get; set; }

			public int NumAgents { get; set; }
			public List<PoolUtilizationDocument> Pools { get; set; }
			IReadOnlyList<IPoolUtilizationData> IUtilizationData.Pools => Pools;

			public double AdminTime { get; set; }
			public double HibernatingTime { get; set; }
			public int UpdateIndex { get; set; }

			[BsonConstructor]
			private UtilizationDocument()
			{
				Pools = new List<PoolUtilizationDocument>();
			}

			public UtilizationDocument(IUtilizationData other)
			{
				StartTime = other.StartTime;
				FinishTime = other.FinishTime;
				NumAgents = other.NumAgents;
				Pools = other.Pools.ConvertAll(x => new PoolUtilizationDocument(x));
				HibernatingTime = other.HibernatingTime;
				AdminTime = other.AdminTime;
			}
		}

		class PoolUtilizationDocument : IPoolUtilizationData
		{
			public PoolId PoolId { get; set; }
			public int NumAgents { get; set; }

			public List<StreamUtilizationDocument> Streams { get; set; }
			IReadOnlyList<IStreamUtilizationData> IPoolUtilizationData.Streams => Streams;

			public double AdminTime { get; set; }
			public double HibernatingTime { get; set; }
			public double OtherTime { get; set; }

			[BsonConstructor]
			private PoolUtilizationDocument()
			{
				Streams = new List<StreamUtilizationDocument>();
			}

			public PoolUtilizationDocument(IPoolUtilizationData other)
			{
				PoolId = other.PoolId;
				NumAgents = other.NumAgents;
				Streams = other.Streams.ConvertAll(x => new StreamUtilizationDocument(x));
				AdminTime = other.AdminTime;
				HibernatingTime = other.HibernatingTime;
				OtherTime = other.OtherTime;
			}
		}

		class StreamUtilizationDocument : IStreamUtilizationData
		{
			public StreamId StreamId { get; set; }
			public double Time { get; set; }

			[BsonConstructor]
			private StreamUtilizationDocument()
			{
			}

			public StreamUtilizationDocument(IStreamUtilizationData other)
			{
				StreamId = other.StreamId;
				Time = other.Time;
			}
		}

		readonly IMongoCollection<UtilizationDocument> _utilization;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="database"></param>
		public UtilizationDataCollection(IMongoService database)
		{
			_utilization = database.GetCollection<UtilizationDocument>("Utilization", keys => keys.Ascending(x => x.FinishTime).Ascending(x => x.StartTime));
		}

		/// <inheritdoc/>
		public async Task AddUtilizationDataAsync(IUtilizationData newTelemetry, CancellationToken cancellationToken)
		{
			await _utilization.InsertOneAsync(new UtilizationDocument(newTelemetry), (InsertOneOptions?)null, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IUtilizationData>> GetUtilizationDataAsync(DateTime? startTimeUtc, DateTime? finishTimeUtc, int? count, CancellationToken cancellationToken)
		{
			if (startTimeUtc == null || finishTimeUtc == null)
			{
				count ??= 10;
			}

			List<FilterDefinition<UtilizationDocument>> filters = new List<FilterDefinition<UtilizationDocument>>();
			if (startTimeUtc.HasValue)
			{
				filters.Add(Builders<UtilizationDocument>.Filter.Gte(x => x.FinishTime, startTimeUtc.Value));
			}
			if (finishTimeUtc.HasValue)
			{
				filters.Add(Builders<UtilizationDocument>.Filter.Lte(x => x.StartTime, finishTimeUtc.Value));
			}

			FilterDefinition<UtilizationDocument> filter = (filters.Count > 0) ? Builders<UtilizationDocument>.Filter.And(filters) : FilterDefinition<UtilizationDocument>.Empty;
			return await _utilization.Find(filter).SortByDescending(x => x.FinishTime).Limit(count).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUtilizationData?> GetLatestUtilizationDataAsync(CancellationToken cancellationToken)
		{
			return await _utilization.Find(FilterDefinition<UtilizationDocument>.Empty).SortByDescending(x => x.FinishTime).FirstOrDefaultAsync(cancellationToken);
		}
	}
}
