// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Agents.Utilization
{
	/// <summary>
	/// Service which updates telemetry periodically
	/// </summary>
	public sealed class UtilizationDataService : IHostedService, IAsyncDisposable
	{
		readonly IUtilizationDataCollection _utilizationDataCollection;
		readonly IAgentCollection _agentCollection;
		readonly ILeaseCollection _leaseCollection;
		readonly IPoolCollection _poolCollection;
		readonly IFleetManager _fleetManager;
		readonly IClock _clock;
		readonly ITicker _tick;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UtilizationDataService(IUtilizationDataCollection utilizationDataCollection, IAgentCollection agentCollection, ILeaseCollection leaseCollection, IPoolCollection poolCollection, IFleetManager fleetManager, IClock clock, ILogger<UtilizationDataService> logger)
		{
			_utilizationDataCollection = utilizationDataCollection;
			_agentCollection = agentCollection;
			_leaseCollection = leaseCollection;
			_poolCollection = poolCollection;
			_fleetManager = fleetManager;
			_clock = clock;
			_tick = clock.AddSharedTicker<UtilizationDataService>(TimeSpan.FromMinutes(10.0), TickLeaderAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _tick.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _tick.StopAsync();

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => _tick.DisposeAsync();

		/// <inheritdoc/>
		async ValueTask TickLeaderAsync(CancellationToken cancellationToken)
		{
			DateTime currentTime = _clock.UtcNow;

			// Find the last time that we need telemetry for
			DateTime maxTime = currentTime.Date + TimeSpan.FromHours(currentTime.Hour);

			// Get the latest telemetry data
			IUtilizationData? latest = await _utilizationDataCollection.GetLatestUtilizationDataAsync(cancellationToken);
			TimeSpan interval = TimeSpan.FromHours(1.0);
			int count = (latest == null) ? (7 * 24) : (int)Math.Round((maxTime - latest.FinishTime) / interval);
			DateTime minTime = maxTime - count * interval;

			// Query all the current data
			IReadOnlyList<IAgent> agents = await _agentCollection.FindAsync(cancellationToken: cancellationToken).ToListAsync(cancellationToken);
			IReadOnlyList<IPoolConfig> pools = await _poolCollection.GetConfigsAsync(cancellationToken);
			IReadOnlyList<ILease> leases = await _leaseCollection.FindLeasesAsync(minTime: minTime, cancellationToken: cancellationToken);

			// Remove any agents which are offline
			agents = agents.Where(x => x.Enabled && x.IsSessionValid(currentTime)).ToList();

			// Find all the agents
			Dictionary<AgentId, List<PoolId>> agentToPoolIds = agents.ToDictionary(x => x.Id, x => x.Pools.ToList());

			// Generate all the telemetry data
			DateTime bucketMinTime = minTime;
			for (int idx = 0; idx < count; idx++)
			{
				DateTime bucketMaxTime = bucketMinTime + interval;
				_logger.LogDebug("Calculating utilization data for {MinTime} to {MaxTime}", bucketMinTime, bucketMaxTime);

				UtilizationData telemetry = new UtilizationData(bucketMinTime, bucketMaxTime);
				telemetry.NumAgents = agents.Count;
				foreach (IPoolConfig pool in pools)
				{
					if (pool.EnableAutoscaling)
					{
						int numStoppedInstances = await _fleetManager.GetNumStoppedInstancesAsync(pool, cancellationToken);
						telemetry.NumAgents += numStoppedInstances;

						PoolUtilizationData poolTelemetry = telemetry.FindOrAddPool(pool.Id);
						poolTelemetry.NumAgents += numStoppedInstances;
						poolTelemetry.HibernatingTime += interval.TotalHours * numStoppedInstances;
					}
				}
				foreach (PoolId poolId in agentToPoolIds.Values.SelectMany(x => x))
				{
					telemetry.FindOrAddPool(poolId).NumAgents++;
				}
				foreach (ILease lease in leases)
				{
					if (lease.StartTime < bucketMaxTime && (!lease.FinishTime.HasValue || lease.FinishTime >= bucketMinTime))
					{
						List<PoolId>? leasePools;
						if (agentToPoolIds.TryGetValue(lease.AgentId, out leasePools))
						{
							DateTime finishTime = lease.FinishTime ?? bucketMaxTime;
							double time = new TimeSpan(Math.Min(finishTime.Ticks, bucketMaxTime.Ticks) - Math.Max(lease.StartTime.Ticks, bucketMinTime.Ticks)).TotalHours;

							foreach (PoolId poolId in leasePools)
							{
								PoolUtilizationData poolTelemetry = telemetry.FindOrAddPool(poolId);
								if (lease.PoolId == null || lease.StreamId == null)
								{
									poolTelemetry.AdminTime += time;
								}
								else if (poolId == lease.PoolId)
								{
									poolTelemetry.FindOrAddStream(lease.StreamId.Value).Time += time;
								}
								else
								{
									poolTelemetry.OtherTime += time;
								}
							}
						}
					}
				}
				await _utilizationDataCollection.AddUtilizationDataAsync(telemetry, cancellationToken);

				bucketMinTime = bucketMaxTime;
			}
		}
	}
}
