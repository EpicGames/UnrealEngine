// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using Amazon.CloudWatch;
using Amazon.CloudWatch.Model;
using HordeServer.Compute;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Aws;

/// <summary>
/// Exports metric inside Horde server to AWS CloudWatch
/// </summary>
public sealed class AwsCloudWatchMetricExporter : IHostedService, IAsyncDisposable
{
	private readonly ComputeService _computeService;
	private readonly IAmazonCloudWatch _cloudWatch;
	private readonly AsyncTaskQueue _asyncTaskQueue;
	private readonly ILogger<AwsCloudWatchMetricExporter> _logger;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="computeService"></param>
	/// <param name="cloudWatch"></param>
	/// <param name="logger"></param>
	public AwsCloudWatchMetricExporter(ComputeService computeService, IAmazonCloudWatch cloudWatch, ILogger<AwsCloudWatchMetricExporter> logger)
	{
		_computeService = computeService;
		_cloudWatch = cloudWatch;
		_asyncTaskQueue = new AsyncTaskQueue(logger);
		_logger = logger;
	}

	/// <inheritdoc/>
	public async ValueTask DisposeAsync()
	{
		await _asyncTaskQueue.DisposeAsync();
	}

	private void OnResourceNeedsUpdated(string clusterId, string poolId, string resourceName, int totalValue)
	{
		_asyncTaskQueue.Enqueue(cancellationToken => OnResourceNeedsUpdatedAsync(clusterId, poolId, resourceName, totalValue, cancellationToken));
	}

	private async Task OnResourceNeedsUpdatedAsync(string clusterId, string poolId, string resourceName, int totalValue, CancellationToken cancellationToken)
	{
		try
		{
			DateTime utcNow = DateTime.UtcNow;
			List<Dimension> dimensions = new()
			{
				new() { Name = "ComputeCluster", Value = clusterId },
				new() { Name = "Pool", Value = poolId },
				new() { Name = "Resource", Value = resourceName }
			};
			List<MetricDatum> metricDatums = new()
			{
				new ()
				{
					MetricName = "ResourceNeed",
					Dimensions = dimensions,
					Unit = StandardUnit.Count,
					Value = totalValue,
					TimestampUtc = utcNow
				}
			};

			PutMetricDataRequest request = new() { Namespace = "Horde", MetricData = metricDatums };
			PutMetricDataResponse response = await _cloudWatch.PutMetricDataAsync(request, cancellationToken);

			if (response.HttpStatusCode != HttpStatusCode.OK)
			{
				throw new HttpRequestException($"PutMetricData failed. Status code {response.HttpStatusCode}");
			}
		}
		catch (Exception e)
		{
			_logger.LogError(e, "Error while updating resource needs with AWS CloudWatch: {Message}", e.Message);
		}
	}

	/// <inheritdoc/>
	public Task StartAsync(CancellationToken cancellationToken)
	{
		_computeService.OnResourceNeedsUpdated += OnResourceNeedsUpdated;
		return Task.CompletedTask;
	}

	/// <inheritdoc/>
	public async Task StopAsync(CancellationToken cancellationToken)
	{
		_computeService.OnResourceNeedsUpdated -= OnResourceNeedsUpdated;
		await _asyncTaskQueue.FlushAsync(cancellationToken);
	}
}

