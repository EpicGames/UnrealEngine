// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using Amazon.CloudWatch;
using Amazon.CloudWatch.Model;
using Amazon.Runtime;
using Amazon.Runtime.Endpoints;

namespace HordeServer.Aws;

/// <summary>
/// Fan-out a single CloudWatch request to multiple clients (each AWS region requires a separate client) 
/// This is used for replicating metrics to more than one region while still implementing the same original interface.
/// Granted, only PutMetricDataAsync is implemented as that is what Horde uses.
/// </summary>
public class AwsCloudWatchMultiplexer : IAmazonCloudWatch
{
	private readonly List<IAmazonCloudWatch> _clients;
	
	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="clients">List of clients to send requests with</param>
	public AwsCloudWatchMultiplexer(List<IAmazonCloudWatch> clients)
	{
		if (clients.Count < 1)
		{
			throw new ArgumentException("At least one client must be specified", nameof(clients));
		}
		
		_clients = clients;
	}
	
	/// <inheritdoc/>
	public async Task<PutMetricDataResponse> PutMetricDataAsync(PutMetricDataRequest request, CancellationToken cancellationToken = default)
	{
		PutMetricDataResponse[] responses = await Task.WhenAll(_clients.Select(x => x.PutMetricDataAsync(request, cancellationToken)).ToArray());
		PutMetricDataResponse successfulResponse = new () { HttpStatusCode = HttpStatusCode.OK };
		foreach (PutMetricDataResponse res in responses)
		{
			if (res.HttpStatusCode != HttpStatusCode.OK)
			{
				return res;
			}
			successfulResponse = res;
		}
		
		return successfulResponse;
	}
	
	/// <inheritdoc/>
	public IClientConfig Config => _clients[0].Config;
	
	/// <inheritdoc/>
	public void Dispose()
	{
		Dispose(disposing: true);
		GC.SuppressFinalize(this);
	}
	
	/// <summary>
	/// Overridable dispose method
	/// </summary>
	protected virtual void Dispose(bool disposing)
	{
		if (disposing)
		{
			foreach (IAmazonCloudWatch client in _clients)
			{
				client.Dispose();
			}
		}
	}
	
#pragma warning disable CS1591
	public Task<DeleteAlarmsResponse> DeleteAlarmsAsync(DeleteAlarmsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DeleteAnomalyDetectorResponse> DeleteAnomalyDetectorAsync(DeleteAnomalyDetectorRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DeleteDashboardsResponse> DeleteDashboardsAsync(DeleteDashboardsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DeleteInsightRulesResponse> DeleteInsightRulesAsync(DeleteInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DeleteMetricStreamResponse> DeleteMetricStreamAsync(DeleteMetricStreamRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAlarmHistoryResponse> DescribeAlarmHistoryAsync(CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAlarmHistoryResponse> DescribeAlarmHistoryAsync(DescribeAlarmHistoryRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAlarmsResponse> DescribeAlarmsAsync(CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAlarmsResponse> DescribeAlarmsAsync(DescribeAlarmsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAlarmsForMetricResponse> DescribeAlarmsForMetricAsync(DescribeAlarmsForMetricRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeAnomalyDetectorsResponse> DescribeAnomalyDetectorsAsync(DescribeAnomalyDetectorsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DescribeInsightRulesResponse> DescribeInsightRulesAsync(DescribeInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DisableAlarmActionsResponse> DisableAlarmActionsAsync(DisableAlarmActionsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<DisableInsightRulesResponse> DisableInsightRulesAsync(DisableInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<EnableAlarmActionsResponse> EnableAlarmActionsAsync(EnableAlarmActionsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<EnableInsightRulesResponse> EnableInsightRulesAsync(EnableInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetDashboardResponse> GetDashboardAsync(GetDashboardRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetInsightRuleReportResponse> GetInsightRuleReportAsync(GetInsightRuleReportRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetMetricDataResponse> GetMetricDataAsync(GetMetricDataRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetMetricStatisticsResponse> GetMetricStatisticsAsync(GetMetricStatisticsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetMetricStreamResponse> GetMetricStreamAsync(GetMetricStreamRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<GetMetricWidgetImageResponse> GetMetricWidgetImageAsync(GetMetricWidgetImageRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListDashboardsResponse> ListDashboardsAsync(ListDashboardsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListManagedInsightRulesResponse> ListManagedInsightRulesAsync(ListManagedInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListMetricsResponse> ListMetricsAsync(CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListMetricsResponse> ListMetricsAsync(ListMetricsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListMetricStreamsResponse> ListMetricStreamsAsync(ListMetricStreamsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<ListTagsForResourceResponse> ListTagsForResourceAsync(ListTagsForResourceRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutAnomalyDetectorResponse> PutAnomalyDetectorAsync(PutAnomalyDetectorRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutCompositeAlarmResponse> PutCompositeAlarmAsync(PutCompositeAlarmRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutDashboardResponse> PutDashboardAsync(PutDashboardRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutInsightRuleResponse> PutInsightRuleAsync(PutInsightRuleRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutManagedInsightRulesResponse> PutManagedInsightRulesAsync(PutManagedInsightRulesRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutMetricAlarmResponse> PutMetricAlarmAsync(PutMetricAlarmRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<PutMetricStreamResponse> PutMetricStreamAsync(PutMetricStreamRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<SetAlarmStateResponse> SetAlarmStateAsync(SetAlarmStateRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<StartMetricStreamsResponse> StartMetricStreamsAsync(StartMetricStreamsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<StopMetricStreamsResponse> StopMetricStreamsAsync(StopMetricStreamsRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<TagResourceResponse> TagResourceAsync(TagResourceRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Task<UntagResourceResponse> UntagResourceAsync(UntagResourceRequest request, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
	public Endpoint DetermineServiceOperationEndpoint(AmazonWebServiceRequest request) { throw new NotImplementedException(); }
	public ICloudWatchPaginatorFactory Paginators => throw new NotImplementedException();
#pragma warning restore CS1591
}
