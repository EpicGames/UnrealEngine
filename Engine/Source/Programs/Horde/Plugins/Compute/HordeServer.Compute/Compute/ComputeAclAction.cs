// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Compute;

/// <summary>
/// ACL actions for compute service
/// </summary>
public static class ComputeAclAction
{
	/// <summary>
	/// User can add tasks to the compute cluster
	/// </summary>
	public static AclAction AddComputeTasks { get; } = new ("AddComputeTasks");

	/// <summary>
	/// User can get and list tasks from the compute cluster
	/// </summary>
	public static AclAction GetComputeTasks { get; } = new ("GetComputeTasks");
	
	/// <summary>
	/// User can read from Unreal Build Accelerator cache
	/// </summary>
	public static AclAction UbaCacheRead { get; } = new ("UbaCacheRead");
	
	/// <summary>
	/// User can write to Unreal Build Accelerator cache
	/// </summary>
	public static AclAction UbaCacheWrite { get; } = new ("UbaCacheWrite");
}
