// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Compute;
using HordeServer.Agents;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Compute
{
	/// <summary>
	/// Controller for the /api/v2/compute endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class ComputeController : HordeControllerBase
	{
		readonly ComputeService _computeService;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		readonly Tracer _tracer;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeController(ComputeService computeService, IOptionsSnapshot<ComputeConfig> computeConfig, Tracer tracer)
		{
			_computeService = computeService;
			_computeConfig = computeConfig;
			_tracer = tracer;
		}

		/// <summary>
		/// Find the most suitable cluster given a compute assignment request
		/// </summary>
		/// <param name="request">The request parameters</param>
		/// <returns></returns>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/compute/_cluster")] // Underscore to avoid clashing with endpoint for clusters below
		public ActionResult<GetClusterResponse> GetCluster([FromBody] AssignComputeRequest request)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeController)}.{nameof(GetCluster)}");
			IPAddress? requesterIp = null;
			try
			{
				string? forwardedForHeader = HttpContext.Request.Headers["X-Forwarded-For"].FirstOrDefault();
				IPAddress? clientIp = String.IsNullOrEmpty(forwardedForHeader)
					? HttpContext.Connection.RemoteIpAddress
					: IPAddress.Parse(forwardedForHeader);

				span.SetAttribute("clientIp", clientIp?.ToString());
				requesterIp = ComputeService.ResolveRequesterIp(clientIp, request.Connection?.PreferPublicIp, request.Connection?.ClientPublicIp);
				span.SetAttribute("requesterIp", requesterIp.ToString());
				ClusterId clusterId = ComputeService.FindBestComputeClusterId(_computeConfig.Value, requesterIp);
				span.SetAttribute("clusterId", clusterId.ToString());

				GetClusterResponse response = new() { ClusterId = clusterId };
				return response;
			}
			catch (ComputeServiceException)
			{
				return StatusCode((int)HttpStatusCode.NotFound, $"Unable to resolve a compute cluster ID for IP {requesterIp?.ToString() ?? "null"}");
			}
		}

		/// <summary>
		/// Add tasks to be executed remotely and auto-select appropriate compute cluster to use
		/// </summary>
		/// <param name="request">The request parameters</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/compute")]
		[Obsolete("Resolve cluster with get cluster endpoint instead")]
		public async Task<ActionResult<AssignComputeResponse>> AssignComputeResourceAsync([FromBody] AssignComputeRequest request, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeController)}.{nameof(AssignComputeResourceAsync)}");
			IPAddress requesterIp = ComputeService.ResolveRequesterIp(HttpContext.Connection.RemoteIpAddress, request.Connection?.PreferPublicIp, request.Connection?.ClientPublicIp);
			ClusterId clusterId = ComputeService.FindBestComputeClusterId(_computeConfig.Value, requesterIp);
			return await AssignComputeResourceInClusterAsync(clusterId, request, cancellationToken);
		}

		/// <summary>
		/// Add tasks to be executed remotely
		/// </summary>
		/// <param name="clusterId">Id of the compute cluster</param>
		/// <param name="request">The request parameters</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/compute/{clusterId}")]
		public async Task<ActionResult<AssignComputeResponse>> AssignComputeResourceInClusterAsync(ClusterId clusterId, [FromBody] AssignComputeRequest request, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.TryGetComputeCluster(clusterId, out ComputeClusterConfig? clusterConfig))
			{
				return NotFound(clusterId);
			}
			if (!clusterConfig.Authorize(ComputeAclAction.AddComputeTasks, User))
			{
				return Forbid(ComputeAclAction.AddComputeTasks, clusterId);
			}

			AllocateResourceParams arp = new(clusterId, User, (ComputeProtocol)request.Protocol, request.Requirements)
			{
				RequestId = request.RequestId,
				RequesterIp = HttpContext.Connection.RemoteIpAddress,
				ParentLeaseId = User.GetLeaseClaim(),
				Ports = request.Connection?.Ports ?? new Dictionary<string, int>(),
				ConnectionMode = request.Connection?.ModePreference,
				RequesterPublicIp = request.Connection?.ClientPublicIp,
				UsePublicIp = request.Connection?.PreferPublicIp,
				Encryption = ComputeService.ConvertEncryptionToProto(request.Connection?.Encryption),
				InactivityTimeoutMs = request.Connection?.InactivityTimeoutMs,
				UseUbaCache = request.UseUbaCache,
				UserId = User.GetUserId(),
			};

			ComputeResource computeResource;
			try
			{
				computeResource = await _computeService.TryAllocateResourceAsync(arp, cancellationToken);
			}
			catch (NoComputeResourcesException cse)
			{
				int statusCode = (int)HttpStatusCode.ServiceUnavailable;
				return cse.ShowToUser ? StatusCode(statusCode, cse.Message) : StatusCode(statusCode);
			}
			catch (ComputeServiceException cse)
			{
				int statusCode = (int)HttpStatusCode.InternalServerError;
				return cse.ShowToUser ? StatusCode(statusCode, cse.Message) : StatusCode(statusCode);
			}

			Dictionary<string, ConnectionMetadataPort> responsePorts = new();
			foreach ((string name, ComputeResourcePort crp) in computeResource.Ports)
			{
				responsePorts[name] = new ConnectionMetadataPort(crp.Port, crp.AgentPort);
			}

			AssignComputeResponse response = new()
			{
				Ip = computeResource.Ip.ToString(), Port = computeResource.Ports[ConnectionMetadataPort.ComputeId].Port, ConnectionMode = computeResource.ConnectionMode,
				ConnectionAddress = computeResource.ConnectionAddress,
				Ports = responsePorts,
				Encryption = ComputeService.ConvertEncryptionFromProto(computeResource.Task.Encryption),
				Nonce = StringUtils.FormatHexString(computeResource.Task.Nonce.Span),
				Key = StringUtils.FormatHexString(computeResource.Task.Key.Span),
				Certificate = StringUtils.FormatHexString(computeResource.Task.Certificate.Span),
				Uba = computeResource.Uba == null ? null : new UbaConfig(computeResource.Uba.CacheEndpoint, computeResource.Uba.CacheSessionKey, computeResource.Uba.WriteAccess),
				ClusterId = clusterId,
				AgentId = computeResource.AgentId,
				AgentVersion = computeResource.AgentVersion,
				LeaseId = computeResource.LeaseId,
				Properties = computeResource.Properties,
				Protocol = computeResource.Task.Protocol
			};
			
			foreach (KeyValuePair<string, int> pair in computeResource.Task.Resources)
			{
				response.AssignedResources.Add(pair.Key, pair.Value);
			}

			return response;
		}

		/// <summary>
		/// Request a UBA cache server session
		/// </summary>
		/// <param name="clusterId">ID of the compute cluster</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/compute/{clusterId}/uba-cache")]
		public async Task<ActionResult<UbaConfig>> AllocateUbaCacheServerAsync(ClusterId clusterId, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.TryGetComputeCluster(clusterId, out ComputeClusterConfig? clusterConfig))
			{
				return NotFound(clusterId);
			}
			
			UbaComputeResource ucr = await _computeService.AllocateUbaCacheServerAsync(User, clusterConfig, 120, cancellationToken);
			return new UbaConfig(ucr.CacheEndpoint, ucr.CacheSessionKey, ucr.WriteAccess);
		}

		/// <summary>
		/// Get current resource needs for active sessions
		/// </summary>
		/// <param name="clusterId">ID of the compute cluster</param>
		/// <returns>List of resource needs</returns>
		[HttpGet]
		[Authorize]
		[Route("/api/v2/compute/{clusterId}/resource-needs")]
		public async Task<ActionResult<GetResourceNeedsResponse>> GetResourceNeedsAsync(ClusterId clusterId)
		{
			if (!_computeConfig.Value.TryGetComputeCluster(clusterId, out ComputeClusterConfig? clusterConfig))
			{
				return NotFound(clusterId);
			}

			if (!clusterConfig.Authorize(ComputeAclAction.GetComputeTasks, User))
			{
				return Forbid(ComputeAclAction.GetComputeTasks, clusterId);
			}

			List<ResourceNeedsMessage> resourceNeeds =
				(await _computeService.GetResourceNeedsAsync())
				.Where(x => x.ClusterId == clusterId.ToString())
				.OrderBy(x => x.Timestamp)
				.Select(x => new ResourceNeedsMessage { SessionId = x.SessionId, Pool = x.Pool, ResourceNeeds = x.ResourceNeeds })
				.ToList();

			return new GetResourceNeedsResponse { ResourceNeeds = resourceNeeds };
		}

		/// <summary>
		/// Declare resource needs for a session to help server calculate current demand
		/// <see cref="KnownPropertyNames"/> for resource name property names
		/// </summary>
		/// <param name="clusterId">Id of the compute cluster</param>
		/// <param name="request">Resource needs request</param>
		/// <returns></returns>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/compute/{clusterId}/resource-needs")]
		public async Task<ActionResult<AssignComputeResponse>> SetResourceNeedsAsync(ClusterId clusterId, [FromBody] ResourceNeedsMessage request)
		{
			if (!_computeConfig.Value.TryGetComputeCluster(clusterId, out ComputeClusterConfig? clusterConfig))
			{
				return NotFound(clusterId);
			}

			if (!clusterConfig.Authorize(ComputeAclAction.AddComputeTasks, User))
			{
				return Forbid(ComputeAclAction.AddComputeTasks, clusterId);
			}

			await _computeService.SetResourceNeedsAsync(clusterId, request.SessionId, new PoolId(request.Pool).ToString(), request.ResourceNeeds);
			return Ok(new { message = "Resource needs set" });
		}
	}
}
