// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Compute;
using HordeCommon.Rpc.Tasks;

namespace HordeServer.Compute
{
	/// <summary>
	/// Request for a compute resource
	/// </summary>
	public class ComputeRequest
	{
		/// <summary>
		/// Requirements for the agent to satisfy this request
		/// </summary>
		public Requirements Requirements { get; }

		/// <summary>
		/// Connection settings for the channel
		/// </summary>
		public ComputeTask Task { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeRequest(Requirements requirements, ComputeTask task)
		{
			Requirements = requirements;
			Task = task;
		}
	}
}
