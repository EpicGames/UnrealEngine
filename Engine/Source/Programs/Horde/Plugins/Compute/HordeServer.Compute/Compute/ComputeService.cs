// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using System.Net;
using System.Security.Claims;
using System.Security.Cryptography;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Clients;
using EpicGames.Horde.Compute.Transports;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Users;
using Google.Protobuf;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Relay;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using StackExchange.Redis;
using Condition = EpicGames.Horde.Common.Condition;

namespace HordeServer.Compute
{
	/// <summary>
	/// Exceptions related to compute service
	/// </summary>
	public class ComputeServiceException : Exception
	{
		/// <summary>
		/// Whether this exception message can be shown to user or client
		/// </summary>
		public bool ShowToUser { get; init; } = false;

		/// <inheritdoc/>
		public ComputeServiceException(string? message, Exception? innerException = null, bool showToUser = false) : base(message, innerException)
		{
			ShowToUser = showToUser;
		}
	}
	
	/// <summary>
	/// Exception when no compute resources available
	/// </summary>
	public class NoComputeResourcesException(string? message, Exception? innerException = null) : ComputeServiceException(message, innerException, true)
	{
		internal const string AllResourcesInUse = "Matching resources available but all are in use";
		internal const string NoMatchingResources = "No matching resources available (in use or not)";
	}
	
	/// <summary>
	/// Outcome for a compute allocation request
	/// </summary>
	public enum AllocationOutcome
	{
		/// <summary>
		/// A lease was allocated
		/// </summary>
		Accepted,

		/// <summary>
		/// A lease was not allocated
		/// </summary>
		Denied
	}

	/// <summary>
	/// Internal parameters for allocating a resource
	/// </summary>
	public class AllocateResourceParams
	{
		/// <summary>
		/// Cluster ID
		/// </summary>
		public ClusterId ClusterId { get; }

		/// <summary>
		/// User allocating the resource
		/// </summary>
		public ClaimsPrincipal User { get; init; }

		/// <summary>
		/// User ID allocating the resource
		/// </summary>
		public UserId? UserId { get; init; }

		/// <summary>
		/// Desired protocol version
		/// </summary>
		public ComputeProtocol Protocol { get; }

		/// <summary>
		/// Criteria for selecting an agent
		/// </summary>
		public Requirements Requirements { get; }

		/// <summary>
		/// Unique ID of this allocation request. If the same allocation retried, the same request ID should be used
		/// </summary> 
		public string? RequestId { get; init; }

		/// <summary>
		/// Optional parent lease
		/// </summary>
		public LeaseId? ParentLeaseId { get; init; }

		/// <summary>
		/// IP address of the requester
		/// </summary>
		public IPAddress? RequesterIp { get; init; }

		/// <inheritdoc cref="ConnectionMetadataRequest.ClientPublicIp" />
		public string? RequesterPublicIp { get; init; }

		/// <inheritdoc cref="ConnectionMetadataRequest.Ports" />
		public Dictionary<string, int> Ports { get; init; } = new();

		/// <inheritdoc cref="ConnectionMetadataRequest.ModePreference" />
		public ConnectionMode? ConnectionMode { get; init; }

		/// <inheritdoc cref="ConnectionMetadataRequest.PreferPublicIp" />
		public bool? UsePublicIp { get; init; }

		/// <inheritdoc cref="ConnectionMetadataRequest.Encryption" />
		public ComputeEncryption Encryption { get; init; }
		
		/// <inheritdoc cref="ConnectionMetadataRequest.InactivityTimeoutMs" />
		public int? InactivityTimeoutMs { get; init; }
		
		/// <inheritdoc cref="AssignComputeResponse.Uba" />
		public bool? UseUbaCache { get; init; }

		/// <summary>
		/// Constructor
		/// </summary>
		public AllocateResourceParams(ClusterId clusterId, ClaimsPrincipal user, ComputeProtocol protocol, Requirements? requirements = null)
		{
			ClusterId = clusterId;
			User = user;
			Protocol = protocol;
			Requirements = requirements ?? new Requirements();
		}
	}

	/// <summary>
	/// Assigns compute leases to agents
	/// </summary>
	public sealed class ComputeService : IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Time-to-live for each bucket of request (currently grouped per minute)
		/// </summary>
		private readonly TimeSpan _requestLogTtl = TimeSpan.FromMinutes(5);

		/// <summary>
		/// How often the queue metric of allocation requests should be calculated
		/// </summary>
		private readonly TimeSpan _requestLogMetricInterval = TimeSpan.FromMinutes(1);

		/// <summary>
		/// How often to look for stale leases and relayed ports
		/// </summary>
		private readonly TimeSpan _relayPortCleanupInterval = TimeSpan.FromMinutes(1);

		/// <summary>
		/// Max age before discarding a reported resource need
		/// </summary>
		private readonly TimeSpan _resourceNeedsMaxAge = TimeSpan.FromMinutes(2);

		/// <summary>
		/// Delegate for resource need events
		/// </summary>
		public delegate void ResourceNeedEvent(string clusterId, string poolId, string resourceName, int total);

		/// <summary>
		/// Event triggered when resource needs have been calculated and updated
		/// </summary>
		public event ResourceNeedEvent? OnResourceNeedsUpdated;

		readonly IAgentCollection _agentCollection;
		readonly IAgentScheduler _agentScheduler;
		readonly ILogCollection _logCollection;
		readonly AgentService _agentService;
		readonly AgentRelayService _agentRelayService;
		readonly IRedisService _redisService;
		readonly HttpClient _httpClient;
		readonly IOptionsMonitor<ComputeServerConfig> _staticComputeConfig;
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly IClock _clock;
		readonly Tracer _tracer;
		readonly Counter<int> _allocationsAcceptedCount;
		readonly Counter<int> _allocationsDeniedCount;
		readonly ITicker _requestLogMetricTicker;
		readonly ITicker _relayPortCleanupTicker;
		readonly ILogger<ComputeService> _logger;

		List<Measurement<int>> _unservedMeasurements = new();
		List<Measurement<int>> _resourceNeedsMeasurements = new();

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeService(
			IAgentCollection agentCollection,
			IAgentScheduler agentScheduler,
			ILogCollection logCollection,
			AgentService agentService,
			AgentRelayService agentRelayService,
			IRedisService redisService,
			HttpClient httpClient,
			IOptionsMonitor<ComputeServerConfig> staticComputeConfig,
			IOptionsMonitor<ComputeConfig> computeConfig,
			IClock clock,
			Tracer tracer,
			Meter meter,
			ILogger<ComputeService> logger)
		{
			_agentCollection = agentCollection;
			_agentScheduler = agentScheduler;
			_logCollection = logCollection;
			_agentService = agentService;
			_agentRelayService = agentRelayService;
			_redisService = redisService;
			_httpClient = httpClient;
			_staticComputeConfig = staticComputeConfig;
			_computeConfig = computeConfig;
			_clock = clock;
			_tracer = tracer;
			_requestLogMetricTicker = clock.AddTicker($"{nameof(ComputeService)}.RequestLogMetric", _requestLogMetricInterval, RequestLogMetricTickAsync, logger);
			_relayPortCleanupTicker = clock.AddTicker($"{nameof(ComputeService)}.RelayPortCleanup", _relayPortCleanupInterval, RelayPortCleanupTickAsync, logger);
			_logger = logger;

			_allocationsAcceptedCount = meter.CreateCounter<int>("horde.compute.allocations.accepted");
			_allocationsDeniedCount = meter.CreateCounter<int>("horde.compute.allocations.denied");
			meter.CreateObservableGauge("horde.compute.allocations.unserved", () =>
			{
				List<Measurement<int>> temp = new(_unservedMeasurements);
				_unservedMeasurements.Clear();
				return temp;
			});

			meter.CreateObservableGauge("horde.compute.resourceNeeds", () =>
			{
				if (_resourceNeedsMeasurements.Count == 0)
				{
					// If no new measurements are available, send zero to prevent OpenTelemetry gauge from caching last observed value
					return new List<Measurement<int>> { new(0) };
				}
				List<Measurement<int>> copy = new(_resourceNeedsMeasurements);
				_resourceNeedsMeasurements.Clear();
				return copy;
			});
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _requestLogMetricTicker.StartAsync();
			await _relayPortCleanupTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _requestLogMetricTicker.StopAsync();
			await _relayPortCleanupTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _requestLogMetricTicker.DisposeAsync();
			await _relayPortCleanupTicker.DisposeAsync();
		}

		private async ValueTask RequestLogMetricTickAsync(CancellationToken stoppingToken)
		{
			_unservedMeasurements = await CalculateUnservedRequestsMetricAsync();
			_resourceNeedsMeasurements = await CalculateResourceNeedsAsync();
		}

		private async ValueTask RelayPortCleanupTickAsync(CancellationToken cancellationToken)
		{
			await CleanStaleRelayPortsAsync(cancellationToken);
		}

		/// <summary>
		/// Check for port mappings that either have no registered lease or possess an expired lease
		/// </summary>
		internal async Task CleanStaleRelayPortsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeService)}.{nameof(CleanStaleRelayPortsAsync)}");
			TimeSpan portRelayLeaseTimeout = TimeSpan.FromMinutes(10);
			ISet<ClusterId> clusterIds = await _agentRelayService.GetClustersAsync();
			foreach (ClusterId clusterId in clusterIds)
			{
				(int revision, List<PortMapping> portMappings) = await _agentRelayService.GetPortMappingsAsync(clusterId);
				foreach (PortMapping pm in portMappings)
				{
					bool hasPotentiallyExpired = pm.CreatedAt == null || pm.CreatedAt.ToDateTime() + portRelayLeaseTimeout < _clock.UtcNow;
					if (hasPotentiallyExpired)
					{
						LeaseId leaseId = LeaseId.Parse(pm.LeaseId);
						ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
						if (lease == null || lease.FinishTime != null)
						{
							_logger.LogInformation("Removing stale port mapping for lease {LeaseId}", leaseId);
							await _agentRelayService.RemovePortMappingAsync(clusterId, leaseId);
						}
					}
				}
			}
		}

		private async Task<List<Measurement<int>>> CalculateUnservedRequestsMetricAsync()
		{
			List<RequestInfo> unservedRequestIds = await GetUnservedRequestsAsync();
			Dictionary<string, int> poolsWithUnservedRequestCounts = GroupByPoolAndCount(unservedRequestIds);
			List<Measurement<int>> measurements = new();
			foreach ((string poolId, int reqCount) in poolsWithUnservedRequestCounts)
			{
				_logger.LogDebug("Unserved request count for {Pool}: {Count}", poolId, reqCount);
				measurements.Add(new Measurement<int>(reqCount, new KeyValuePair<string, object?>("pool", poolId)));
			}

			return measurements;
		}

		private record struct MetricKey(string ClusterId, string PoolId, string ResourceName);
		internal async Task<List<Measurement<int>>> CalculateResourceNeedsAsync()
		{
			List<SessionResourceNeeds> resourceNeeds = await GetResourceNeedsAsync();
			Dictionary<MetricKey, int> summedResourceValues = new();

			foreach (SessionResourceNeeds srn in resourceNeeds)
			{
				foreach ((string resource, int value) in srn.ResourceNeeds)
				{
					MetricKey key = new(srn.ClusterId, srn.Pool, resource);
					summedResourceValues.TryGetValue(key, out int currentValue);
					summedResourceValues[key] = currentValue + value;
				}
			}

			List<Measurement<int>> measurements = new();
			foreach ((MetricKey key, int totalValue) in summedResourceValues)
			{
				KeyValuePair<string, object?>[] kvp =
				{
					new ("cluster", key.ClusterId),
					new ("pool", key.PoolId),
					new ("resource", key.ResourceName)
				};
				measurements.Add(new Measurement<int>(totalValue, kvp));
				OnResourceNeedsUpdated?.Invoke(key.ClusterId, key.PoolId, key.ResourceName, totalValue);
			}

			return measurements;
		}

		/// <summary>
		/// Allocates a compute resource
		/// </summary>
		/// <param name="arp">Allocation parameters</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>A compute resource if successful</returns>
		public async Task<ComputeResource> TryAllocateResourceAsync(AllocateResourceParams arp, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeService)}.{nameof(TryAllocateResourceAsync)}");
			span.SetAttribute("clusterId", arp.ClusterId.ToString());
			span.SetAttribute("userId", arp.UserId?.ToString());
			span.SetAttribute("requestId", arp.RequestId);
			span.SetAttribute("requestIp", arp.RequesterIp?.ToString());
			span.SetAttribute("requesterPublicIp", arp.RequesterPublicIp);
			span.SetAttribute("connectionMode", arp.ConnectionMode?.ToString());
			span.SetAttribute("usePublicIp", arp.UsePublicIp.ToString());
			span.SetAttribute("encryption", arp.Encryption.ToString());
			span.SetAttribute("inactivityTimeout", arp.InactivityTimeoutMs);
			span.SetAttribute("useUbaCache", arp.UseUbaCache ?? false);
			span.SetAttribute("parentLeaseId", arp.ParentLeaseId?.ToString());
			span.SetAttribute("req.pool", arp.Requirements.Pool);
			span.SetAttribute("req.condition", arp.Requirements.Condition?.ToString());
			span.SetAttribute("req.exclusive", arp.Requirements.Exclusive);

			arp.Requirements.Pool = ResolvePoolId(arp.Requirements.Pool, arp.RequesterIp);
			span.SetAttribute("req.poolResolved", arp.Requirements.Pool);

			foreach ((string name, ResourceRequirements resReq) in arp.Requirements.Resources)
			{
				span.SetAttribute($"req.res.{name}.min", resReq.Min);
				span.SetAttribute($"req.res.{name}.max", resReq.Max);
			}

			if (!_computeConfig.CurrentValue.TryGetComputeCluster(arp.ClusterId, out ComputeClusterConfig? clusterConfig))
			{
				throw new ArgumentException($"Cluster '{arp.ClusterId}' not found");
			}

			byte[] certificate;
			using (TelemetrySpan _ = _tracer.StartActiveSpan("Generating certificate"))
			{
				certificate = GenerateCert(arp.Encryption); // A no-op if certificate is not required
			}

			List<Condition> conditions = [];
			if (arp.RequesterIp != null)
			{
				// Do not assign a compute resource hosted on the same machine as requester
				// This is often the case when a user is offering their local workstation to participate in a shared compute pool
				// With UnrealBuildAccelerator, it already makes full use of the local machine.
				// Normalize IPv6-mapped IPv4 addresses (e.g. ::ffff:192.168.1.1) to plain IPv4
				// to ensure consistent string comparison with agent ComputeIp values
				IPAddress requesterIp = arp.RequesterIp.IsIPv4MappedToIPv6
					? arp.RequesterIp.MapToIPv4()
					: arp.RequesterIp;
				conditions.Add(Condition.Parse($"{KnownPropertyNames.ComputeIp} != '{requesterIp}'"));
			}
			
			Task<UbaComputeResource>? ubaCache = null;
			if (arp.UseUbaCache ?? false)
			{
				// Allocate UBA cache session key in the background, await the task later
				ubaCache = AllocateUbaCacheServerAsync(arp.User, clusterConfig, 120, cancellationToken);
			}

			try
			{
				PoolId? poolId = arp.Requirements.Pool != null ? new PoolId(arp.Requirements.Pool) : null;
				int numMatchingAgents = 0;
				await foreach (IAgent agent in _agentCollection.FindAsync(poolId, consistentRead: false, cancellationToken: cancellationToken))
				{
					Dictionary<string, int> assignedResources = new Dictionary<string, int>();

					bool isMemberOfCluster = clusterConfig.Condition == null || agent.SatisfiesCondition(clusterConfig.Condition);
					if (isMemberOfCluster)
					{
						numMatchingAgents++;
					}
					bool match = isMemberOfCluster && agent.MeetsRequirements(arp.Requirements, assignedResources, conditions);
					if (match)
					{
						using TelemetrySpan matchSpan = _tracer.StartActiveSpan("Found match");

						ComputeProtocol protocol = ComputeProtocol.Initial;
						foreach (string value in agent.GetPropertyValues("ComputeProtocol"))
						{
							if (Int32.TryParse(value, out int versionInt))
							{
								protocol = (ComputeProtocol)Math.Min((int)arp.Protocol, versionInt);
								break;
							}
						}

						LeaseId leaseId = new LeaseId(BinaryIdUtils.CreateNew());
						ILog? log = await _logCollection.AddAsync(JobId.Empty, leaseId, agent.SessionId, LogType.Json, cancellationToken: cancellationToken);

						using TelemetrySpan createTaskSpan = _tracer.StartActiveSpan("CreateComputeTask");

						ComputeTask computeTask = CreateComputeTask(assignedResources, log?.Id, arp.Encryption, certificate, arp.ParentLeaseId, protocol, arp.InactivityTimeoutMs);

						CreateLeaseOptions createLeaseOptions = new CreateLeaseOptions(leaseId, arp.ParentLeaseId, "Compute task", null, null, log?.Id, assignedResources, arp.Requirements.Exclusive, computeTask);

						using TelemetrySpan assignSpan = _tracer.StartActiveSpan("TryAssignAsync");

						ComputeResource? resource = await TryAssignAsync(arp, agent, computeTask, protocol, leaseId);
						if (resource != null)
						{
							using TelemetrySpan addLeaseSpan = _tracer.StartActiveSpan("Adding lease");

							IAgent? newAgent = await agent.TryCreateLeaseAsync(createLeaseOptions, cancellationToken);
							if (newAgent != null)
							{
								span.SetAttribute("allocatedLeaseId", leaseId.ToString());
								span.SetAttribute("allocatedAgentId", newAgent.Id.ToString());
								span.SetAttribute("allocatedConnectionAddress", resource.ConnectionAddress);
								span.SetAttribute("allocatedIp", resource.Ip.ToString());
								
								if (ubaCache != null)
								{
									resource.Uba = await ubaCache;
								}

								await LogRequestAsync(AllocationOutcome.Accepted, arp.RequestId, arp.Requirements, arp.ParentLeaseId, span, cancellationToken);
								return resource;
							}
							else
							{
								if (resource.ConnectionMode == ConnectionMode.Relay)
								{
									await _agentRelayService.RemovePortMappingAsync(arp.ClusterId, leaseId);
								}
							}
						}
					}
				}

				await LogRequestAsync(AllocationOutcome.Denied, arp.RequestId, arp.Requirements, arp.ParentLeaseId, span, cancellationToken);
				
				if (numMatchingAgents == 0)
				{
					throw new NoComputeResourcesException(NoComputeResourcesException.NoMatchingResources);
				}
				throw new NoComputeResourcesException(NoComputeResourcesException.AllResourcesInUse);
			}
			catch (AgentRelayException are)
			{
				_logger.LogError(are, "Agent relay error while allocating compute resource for cluster {ClusterId}: {Message}", arp.ClusterId, are.Message);
				span.SetStatus(Status.Error);
				span.RecordException(are);
				throw new ComputeServiceException(are.Message, are) { ShowToUser = true };
			}
		}
		
		internal static bool VerifyUbaCache(ClaimsPrincipal user, ComputeClusterConfig clusterConfig)
		{
			if (clusterConfig.Uba?.CacheEndpoint == null || clusterConfig.Uba.CacheHttpEndpoint == null || clusterConfig.Uba.CacheHttpCrypto == null)
			{
				throw new ComputeServiceException($"No UBA cache server defined for compute cluster {clusterConfig.Id}") { ShowToUser = true };
			}
			
			bool hasReadAccess = clusterConfig.Authorize(ComputeAclAction.UbaCacheRead, user);
			bool hasWriteAccess = clusterConfig.Authorize(ComputeAclAction.UbaCacheWrite, user);
			bool hasAccess = hasReadAccess || hasWriteAccess;
			if (!hasAccess)
			{
				throw new ComputeServiceException($"User does not have UBA cache server access (check {ComputeAclAction.UbaCacheRead} or {ComputeAclAction.UbaCacheWrite})") { ShowToUser = true };
			}
			
			return hasWriteAccess;
		}
		
		private string? ResolvePoolId(string? poolId, IPAddress? ipAddress)
		{
			if (poolId == null)
			{
				return null;
			}

			_computeConfig.CurrentValue.TryGetNetworkConfig(ipAddress ?? IPAddress.Any, out NetworkConfig? networkConfig);
			string networkId = networkConfig?.Id ?? "default";
			string computeId = networkConfig?.ComputeId ?? "default";
			return poolId
				.Replace("%REQUESTER_NETWORK_ID%", networkId, StringComparison.InvariantCulture)
				.Replace("%REQUESTER_COMPUTE_ID%", computeId, StringComparison.InvariantCulture);
		}

		/// <summary>
		/// Declare resource needs for a session to help server calculate current demand
		/// Any previous declaration associated with the same session ID will be replaced.
		/// </summary>
		/// <param name="clusterId">Id of the compute cluster</param>
		/// <param name="sessionId">Unique session ID performing compute resource requests</param>
		/// <param name="pool">Pool of agents requesting resources from</param>
		/// <param name="resourceNeeds">Resource needs</param>
		public async Task SetResourceNeedsAsync(ClusterId clusterId, string sessionId, string pool, Dictionary<string, int> resourceNeeds)
		{
			SessionResourceNeeds needs = new(_clock.UtcNow, clusterId.ToString(), sessionId, pool, resourceNeeds);
			await _redisService.GetDatabase().HashSetAsync(RedisKeyResourceNeeds(), needs.GetRedisHashKey(), needs.Serialize());
		}

		internal async Task<List<SessionResourceNeeds>> GetResourceNeedsAsync()
		{
			IDatabase redis = _redisService.GetDatabase();
			HashEntry[] hashEntries = await redis.HashGetAllAsync(RedisKeyResourceNeeds());
			List<SessionResourceNeeds> validResourceNeeds = new();
			List<RedisValue> invalidKeys = new();

			foreach (HashEntry entry in hashEntries)
			{
				try
				{
					SessionResourceNeeds srn = SessionResourceNeeds.Deserialize(entry.Value.ToString());
					bool isOutdated = _clock.UtcNow > srn.Timestamp + _resourceNeedsMaxAge;
					if (isOutdated)
					{
						invalidKeys.Add(entry.Name.ToString());
					}
					else
					{
						validResourceNeeds.Add(srn);
					}
				}
				catch (JsonException je)
				{
					_logger.LogWarning(je, "Failed to deserialize compute resource needs (can be normal during version upgrades). Key={Key} Value={Value}", entry.Name, entry.Value);
					invalidKeys.Add(entry.Name.ToString());
				}
			}

			if (invalidKeys.Any())
			{
				await redis.HashDeleteAsync(RedisKeyResourceNeeds(), invalidKeys.ToArray());
			}

			return validResourceNeeds;
		}

		private static string RedisKeyResourceNeeds() => $"compute/resource-needs";

		/// <summary>
		/// Record describing the resource needs for a session at a particular point in time
		/// </summary>
		/// <param name="Timestamp">Timestamp</param>
		/// <param name="ClusterId">Cluster ID</param>
		/// <param name="SessionId">Session ID</param>
		/// <param name="Pool">Pool ID</param>
		/// <param name="ResourceNeeds">Resource needs</param>
		internal record struct SessionResourceNeeds(DateTime Timestamp, string ClusterId, string SessionId, string Pool, Dictionary<string, int> ResourceNeeds)
		{
			public string GetRedisHashKey() => $"{ClusterId}:{SessionId}:{Pool}";
			public string Serialize() => JsonSerializer.Serialize(this);
			public static SessionResourceNeeds Deserialize(string value) => JsonSerializer.Deserialize<SessionResourceNeeds>(value);
		}

		private static string RedisKeyComputeRequests(DateTimeOffset timestamp)
		{
			long unixTime = timestamp.ToUnixTimeSeconds();
			long unixTimeClosestMin = unixTime - unixTime % 60; // Round to closest starting minute (buckets per minute)
			return $"compute/requests/{unixTimeClosestMin}";
		}

		/// <summary>
		/// Record to be stored in Redis for representing a compute allocation request
		/// </summary>
		/// <param name="Timestamp"></param>
		/// <param name="Outcome"></param>
		/// <param name="RequestId"></param>
		/// <param name="Pool"></param>
		/// <param name="ParentLeaseId"></param>
		internal record RequestInfo(DateTimeOffset Timestamp, AllocationOutcome Outcome, string RequestId, string Pool, string? ParentLeaseId)
		{
			private const int Version = 1;

			public string Serialize()
			{
				StringBuilder sb = new(100);
				sb.Append(Version).Append('\t');
				sb.Append(Timestamp.ToUnixTimeSeconds()).Append('\t');
				sb.Append(Outcome).Append('\t');
				sb.Append(RequestId).Append('\t');
				sb.Append(Pool).Append('\t');
				sb.Append(ParentLeaseId);
				return sb.ToString();
			}

			public static RequestInfo? Deserialize(RedisValue value) { return Deserialize(value.ToString()); }
			public static RequestInfo? Deserialize(string value)
			{
				string[] parts = value.Split("\t");
				if (parts.Length != 6)
				{
					return null;
				}

				if (!Int32.TryParse(parts[0], out int version) || version != Version)
				{
					return null;
				}

				if (!Int64.TryParse(parts[1], out long unixTimeSec))
				{
					return null;
				}

				if (!System.Enum.TryParse(parts[2], out AllocationOutcome outcome))
				{
					return null;
				}

				return new RequestInfo(DateTimeOffset.FromUnixTimeSeconds(unixTimeSec), outcome, parts[3], parts[4], String.IsNullOrEmpty(parts[5]) ? null : parts[5]);
			}
		}

		internal async Task LogRequestAsync(AllocationOutcome outcome, string? requestId, Requirements requirements, LeaseId? parentLeaseId, TelemetrySpan currentSpan, CancellationToken cancellationToken = default)
		{
			int? numActiveLeases = await GetNumActiveLeasesAsync(parentLeaseId, cancellationToken);
			currentSpan.SetAttribute("numActiveLeases", numActiveLeases);

			KeyValuePair<string, object?> poolTag = new("pool", requirements.Pool);
			KeyValuePair<string, object?> activeLeasesTag = new("activeLeases", numActiveLeases?.ToString() ?? "null");

			if (requestId != null && requirements.Pool != null)
			{
				DateTimeOffset timestamp = new(_clock.UtcNow);
				string key = RedisKeyComputeRequests(timestamp);
				RequestInfo requestInfo = new(timestamp, outcome, requestId, requirements.Pool, parentLeaseId?.ToString());
				await _redisService.GetDatabase().ListRightPushAsync(key, requestInfo.Serialize());
				await _redisService.GetDatabase().KeyExpireAsync(key, _requestLogTtl);
			}

			switch (outcome)
			{
				case AllocationOutcome.Accepted: _allocationsAcceptedCount.Add(1, poolTag, activeLeasesTag); break;
				case AllocationOutcome.Denied: _allocationsDeniedCount.Add(1, poolTag, activeLeasesTag); break;
				default: throw new Exception("Invalid outcome " + outcome);
			}
		}

		internal async Task<List<RequestInfo>> GetUnservedRequestsAsync()
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeService)}.{nameof(GetUnservedRequestsAsync)}");
			IDatabase redis = _redisService.GetDatabase();
			DateTime utcNow = _clock.UtcNow;
			DateTimeOffset startTime = new(utcNow - TimeSpan.FromMinutes(1));
			DateTimeOffset endTime = new(utcNow);

			RedisValue[] lastMinValues = await redis.ListRangeAsync(RedisKeyComputeRequests(startTime));
			RedisValue[] currentMinValues = await redis.ListRangeAsync(RedisKeyComputeRequests(endTime));
			List<RedisValue> values = new(lastMinValues.Concat(currentMinValues));
			List<RequestInfo> requestInfos = values
				.Select(RequestInfo.Deserialize)
				.OfType<RequestInfo>()
				.Where(x => x.Timestamp >= startTime && x.Timestamp < endTime)
				.OrderBy(x => x.Timestamp)
				.ToList();

			Dictionary<string, RequestInfo> idToInfo = new();
			foreach (RequestInfo ri in requestInfos)
			{
				if (!idToInfo.TryGetValue(ri.RequestId, out RequestInfo? lastSeen) || lastSeen.Outcome == AllocationOutcome.Denied)
				{
					idToInfo[ri.RequestId] = ri;
				}
			}

			List<RequestInfo> results = idToInfo
				.Where(pair => pair.Value.Outcome == AllocationOutcome.Denied)
				.Select(pair => pair.Value)
				.ToList();

			span.SetAttribute("numCurrentMin", currentMinValues.Length);
			span.SetAttribute("numLastMin", lastMinValues.Length);
			span.SetAttribute("numResults", results.Count);
			return results;
		}

		internal static Dictionary<string, int> GroupByPoolAndCount(List<RequestInfo> requests)
		{
			return requests
				.GroupBy(ri => ri.Pool)
				.ToDictionary(group => group.Key, group => group.Count());
		}

		/// <summary>
		/// Get the number of currently active leases belonging to the given parent lease.
		/// Allows compute allocation requests metric to be broken down by lease.
		/// </summary>
		/// <param name="parentLeaseId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Number of active leases</returns>
		private async Task<int?> GetNumActiveLeasesAsync(LeaseId? parentLeaseId, CancellationToken cancellationToken)
		{
			if (parentLeaseId == null)
			{
				return null;
			}

			return await _agentScheduler.GetChildLeaseCountAsync(parentLeaseId.Value, cancellationToken);
		}

		private async Task<ComputeResource?> TryAssignAsync(AllocateResourceParams arp, IAgent agent, ComputeTask computeTask, ComputeProtocol protocol, LeaseId leaseId)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ComputeService)}.{nameof(TryAssignAsync)}");

			string? ipStr = agent.GetPropertyValues("ComputeIp").FirstOrDefault();
			if (ipStr == null || !IPAddress.TryParse(ipStr, out IPAddress? agentIp))
			{
				return null;
			}

			string? portStr = agent.GetPropertyValues("ComputePort").FirstOrDefault();
			if (portStr == null || !Int32.TryParse(portStr, out int computePort))
			{
				return null;
			}

			string? tunnelAddress = _staticComputeConfig.CurrentValue.ComputeTunnelAddress;

			if (arp.ConnectionMode is null or ConnectionMode.Direct)
			{
				// A direct connection with 1-to-1 mapped ports
				Dictionary<string, ComputeResourcePort> ports = new();
				ports[ConnectionMetadataPort.ComputeId] = new ComputeResourcePort(computePort, computePort);
				foreach ((string portId, int port) in arp.Ports)
				{
					ports[portId] = new ComputeResourcePort(port, port);
				}

				return new ComputeResource(ConnectionMode.Direct, agentIp, null, ports, computeTask, agent.Properties, agent.Id, agent.Version, protocol, leaseId);
			}
			else if (arp.ConnectionMode == ConnectionMode.Tunnel && tunnelAddress != null)
			{
				// A tunneled connection. Ports marked as -1 must be be tunneled via tunnel address
				Dictionary<string, ComputeResourcePort> ports = new();
				ports[ConnectionMetadataPort.ComputeId] = new ComputeResourcePort(-1, computePort);
				foreach ((string portId, int port) in arp.Ports)
				{
					ports[portId] = new ComputeResourcePort(-1, port);
				}

				return new ComputeResource(ConnectionMode.Tunnel, agentIp, tunnelAddress, ports, computeTask, agent.Properties, agent.Id, agent.Version, protocol, leaseId);
			}
			else if (arp.ConnectionMode == ConnectionMode.Relay)
			{
				Dictionary<string, int> portsWithComputePort = new(arp.Ports) { { ConnectionMetadataPort.ComputeId, computePort } };
				List<Port> relayPorts = portsWithComputePort
					.SelectMany(kvp => new List<Port>()
					{
						new () { RelayPort = -1, AgentPort = kvp.Value, Protocol = PortProtocol.Tcp },
						new () { RelayPort = -1, AgentPort = kvp.Value, Protocol = PortProtocol.Udp }
					})
					.OrderBy(x => x.AgentPort)
					.ToList();

				IPAddress? clientPublicIp = arp.RequesterPublicIp == null ? null : IPAddress.Parse(arp.RequesterPublicIp);
				PortMappingResult pmResult = await _agentRelayService.RequestPortMappingAsync(arp.ClusterId, leaseId, clientPublicIp, agentIp, relayPorts);

				IPAddress relayIp = FindBestRelayIp(arp.RequesterIp, null, pmResult.IpAddresses);
				Dictionary<string, ComputeResourcePort> ports = new();
				foreach (Port pmPort in pmResult.Ports.Where(x => x.Protocol == PortProtocol.Tcp))
				{
					foreach ((string portId, int agentPort) in portsWithComputePort)
					{
						if (agentPort == pmPort.AgentPort)
						{
							ports[portId] = new ComputeResourcePort(pmPort.RelayPort, pmPort.AgentPort);
							_logger.LogInformation("Port {PortId} mapped {RelayIp}:{RelayPort} -> {AgentPort} for lease ID {LeaseId}", portId, relayIp, pmPort.RelayPort, pmPort.AgentPort, leaseId);
						}
					}
				}

				return new ComputeResource(ConnectionMode.Relay, agentIp, relayIp.ToString(), ports, computeTask, agent.Properties, agent.Id, agent.Version, protocol, leaseId);
			}

			throw new Exception("Unable to resolve a suitable connection mode for compute task");
		}

		/// <summary>
		/// Resolve IP of the requester
		/// </summary>
		/// <param name="requesterIp"></param>
		/// <param name="usePublicIp"></param>
		/// <param name="requesterPublicIp"></param>
		/// <returns>IP address of request</returns>
		/// <exception cref="ComputeServiceException"></exception>
		public static IPAddress ResolveRequesterIp(IPAddress? requesterIp, bool? usePublicIp, string? requesterPublicIp)
		{
			if (usePublicIp is true && requesterPublicIp != null)
			{
				if (IPAddress.TryParse(requesterPublicIp, out IPAddress? publicIp))
				{
					requesterIp = publicIp;
				}
			}

			if (requesterIp == null)
			{
				throw new ComputeServiceException("Unable to determine IP of requester");
			}

			return requesterIp;
		}

		/// <summary>
		/// Find best compute cluster for given parameters
		/// </summary>
		/// <param name="computeConfig">Global config</param>
		/// <param name="requesterIp">IP of client (initiator) requesting a compute resource</param>
		/// <returns>Compute cluster ID</returns>
		/// <exception cref="ComputeServiceException"></exception>
		public static ClusterId FindBestComputeClusterId(ComputeConfig computeConfig, IPAddress? requesterIp)
		{
			if (requesterIp == null || !computeConfig.TryGetNetworkConfig(requesterIp, out NetworkConfig? networkConfig))
			{
				throw new ComputeServiceException("Unable to find a matching network config");
			}

			if (networkConfig.ComputeId == null)
			{
				throw new ComputeServiceException($"Network config '{networkConfig.Id}' has no compute ID set");
			}

			return new ClusterId(networkConfig.ComputeId);
		}

		private static IPAddress FindBestRelayIp(IPAddress? clientIp, IPAddress? publicClientIp, IEnumerable<string> relayIps)
		{
			return FindBestRelayIp(clientIp, publicClientIp, relayIps.Select(IPAddress.Parse).ToList());
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "To be implemented")]
		private static IPAddress FindBestRelayIp(IPAddress? clientIp, IPAddress? publicClientIp, List<IPAddress> relayIps)
		{
			// TODO: Implement proper lookup of best relay IP to use
			return relayIps[0];
		}
		
		/// <summary>
		/// Allocate a UBA cache server session
		/// </summary>
		/// <param name="user">Requesting user</param>
		/// <param name="clusterConfig">Compute cluster</param>
		/// <param name="ttl">Time-to-live in seconds for the session</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <param name="httpClient">Optional HTTP client override for tests</param>
		/// <returns>Settings for a UBA cache server session</returns>
		public async Task<UbaComputeResource> AllocateUbaCacheServerAsync(
			ClaimsPrincipal user, ComputeClusterConfig clusterConfig, int ttl, CancellationToken cancellationToken, HttpClient? httpClient = null)
		{
			string? cacheEndpoint = clusterConfig.Uba?.CacheEndpoint;
			string? httpEndpoint = clusterConfig.Uba?.CacheHttpEndpoint;
			string? httpCrypto = clusterConfig.Uba?.CacheHttpCrypto;
			if (cacheEndpoint == null || httpEndpoint == null || httpCrypto == null)
			{
				throw new ComputeServiceException($"No UBA cache server defined for compute cluster {clusterConfig.Id}") { ShowToUser = true };
			}
			
			bool hasReadAccess = clusterConfig.Authorize(ComputeAclAction.UbaCacheRead, user);
			bool hasWriteAccess = clusterConfig.Authorize(ComputeAclAction.UbaCacheWrite, user);
			bool hasAccess = hasReadAccess || hasWriteAccess;
			if (!hasAccess)
			{
				throw new ComputeServiceException($"User does not have UBA cache server access (check {ComputeAclAction.UbaCacheRead} or {ComputeAclAction.UbaCacheWrite})") { ShowToUser = true };
			}
			
			try
			{
				using CancellationTokenSource timeoutCts = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
				timeoutCts.CancelAfter(TimeSpan.FromSeconds(5));
				
				// Generate a session encryption key that we activate in the UBA cache server
				byte[] sessionKey = RandomNumberGenerator.GetBytes(16);
				string sessionKeyHex = Convert.ToHexString(sessionKey);
				byte[] cryptoKey = Convert.FromHexString(httpCrypto);
				if (cryptoKey.Length != 16)
				{
					throw new Exception($"Invalid crypto key length: {cryptoKey.Length}");
				}
				
				using UbaCrypto crypto = new(cryptoKey);
				string encryptedSessionKeyHex = Convert.ToHexString(crypto.Encrypt(sessionKey));
				string readOnlyStr = hasWriteAccess ? "false" : "true";
				Uri uri = new ($"http://{httpEndpoint}/addsession?crypto={encryptedSessionKeyHex},readonly={readOnlyStr},expiration={ttl}");
				HttpResponseMessage res = await (httpClient ?? _httpClient).GetAsync(uri, timeoutCts.Token);
				res.EnsureSuccessStatusCode();

				return new UbaComputeResource(cacheEndpoint, sessionKeyHex, hasWriteAccess);
			}
			catch (TaskCanceledException ex)
			{
				throw new ComputeServiceException("UBA cache server allocation timed out or was cancelled", ex);
			}
			catch (Exception ex)
			{
				throw new ComputeServiceException("Unexpected error during UBA cache server allocation", ex);
			}
		}

		static ComputeTask CreateComputeTask(Dictionary<string, int> assignedResources, LogId? logId, ComputeEncryption encryption,
			byte[] certificateData, LeaseId? parentLeaseId, ComputeProtocol protocol, int? inactivityTimeoutMs)
		{
			ComputeTask computeTask = new ComputeTask();
			computeTask.Encryption = encryption;
			computeTask.Nonce = UnsafeByteOperations.UnsafeWrap(RandomNumberGenerator.GetBytes(ServerComputeClient.NonceLength));
			computeTask.Key = UnsafeByteOperations.UnsafeWrap(AesTransport.CreateKey());
			computeTask.Certificate = UnsafeByteOperations.UnsafeWrap(certificateData);
			computeTask.Resources.Add(assignedResources);
			computeTask.LogId = logId?.ToString();
			computeTask.ParentLeaseId = parentLeaseId?.ToString() ?? String.Empty;
			computeTask.Protocol = (int)protocol;
			computeTask.InactivityTimeoutMs = inactivityTimeoutMs ?? 30000;
			return computeTask;
		}

		static byte[] GenerateCert(ComputeEncryption encryption)
		{
			return encryption switch
			{
				ComputeEncryption.SslRsa2048 => TcpSslTransport.GenerateCert(ConvertEncryptionFromProto(encryption)),
				ComputeEncryption.SslEcdsaP256 => TcpSslTransport.GenerateCert(ConvertEncryptionFromProto(encryption)),
				_ => Array.Empty<byte>()
			};
		}

		internal static Encryption ConvertEncryptionFromProto(ComputeEncryption proto)
		{
			return proto switch
			{
				ComputeEncryption.Aes => Encryption.Aes,
				ComputeEncryption.SslRsa2048 => Encryption.Ssl,
				ComputeEncryption.SslEcdsaP256 => Encryption.SslEcdsaP256,
				ComputeEncryption.None => Encryption.None,
				ComputeEncryption.Unspecified => Encryption.None,
				_ => throw new ArgumentOutOfRangeException(nameof(proto), proto, null)
			};
		}

		internal static ComputeEncryption ConvertEncryptionToProto(Encryption? json)
		{
			return json switch
			{
				Encryption.Aes => ComputeEncryption.Aes,
				Encryption.Ssl => ComputeEncryption.SslRsa2048,
				Encryption.SslEcdsaP256 => ComputeEncryption.SslEcdsaP256,
				Encryption.None => ComputeEncryption.None,
				_ => ComputeEncryption.None
			};
		}
	}
}
