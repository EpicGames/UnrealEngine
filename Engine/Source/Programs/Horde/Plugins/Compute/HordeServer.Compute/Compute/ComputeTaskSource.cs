// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Compute;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Relay;
using HordeServer.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Compute
{
	/// <inheritdoc cref="UbaConfig" />
	public class UbaComputeResource(string cacheEndpoint, string cacheSessionKey, bool writeAccess)
	{
		/// <inheritdoc cref="UbaConfig.CacheEndpoint" />
		public string CacheEndpoint { get; } = cacheEndpoint;
		
		/// <inheritdoc cref="UbaConfig.CacheSessionKey" />
		public string CacheSessionKey { get; } = cacheSessionKey;
		
		/// <inheritdoc cref="UbaConfig.WriteAccess" />
		public bool WriteAccess { get; } = writeAccess;
	}
	
	/// <summary>
	/// A TCP/IP port used by a compute resource, and how it is mapped externally -> internally
	/// </summary>
	public class ComputeResourcePort
	{
		/// <summary>
		/// Externally visible port that is mapped to agent port
		/// In direct connection mode, these two are identical.
		/// </summary>
		public int Port { get; }

		/// <summary>
		/// Port the local process on the agent is listening on
		/// </summary>
		public int AgentPort { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="port"></param>
		/// <param name="agentPort"></param>
		public ComputeResourcePort(int port, int agentPort)
		{
			Port = port;
			AgentPort = agentPort;
		}

		/// <inheritdoc cref="Equals(HordeServer.Compute.ComputeResourcePort)" />
		protected bool Equals(ComputeResourcePort other)
		{
			return Port == other.Port && AgentPort == other.AgentPort;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			return obj.GetType() == GetType() && Equals((ComputeResourcePort)obj);
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return HashCode.Combine(Port, AgentPort);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"Port={Port}, AgentPort={AgentPort}";
		}
	}

	/// <summary>
	/// Information about a compute resource
	/// </summary>
	public class ComputeResource
	{
		/// <inheritdoc cref="AssignComputeResponse.ConnectionMode" />
		public ConnectionMode ConnectionMode { get; }

		/// <summary>
		/// IP address of the agent
		/// </summary>
		public IPAddress Ip { get; }

		/// <inheritdoc cref="AssignComputeResponse.ConnectionAddress" />
		public string? ConnectionAddress { get; }

		/// <inheritdoc cref="AssignComputeResponse.Ports" />
		public IReadOnlyDictionary<string, ComputeResourcePort> Ports { get; }
		
		/// <inheritdoc cref="UbaComputeResource" />
		public UbaComputeResource? Uba { get; set;  }

		/// <summary>
		/// Information about the compute task
		/// </summary>
		public ComputeTask Task { get; }

		/// <summary>
		/// Properties of the assigned agent
		/// </summary>
		public IReadOnlyList<string> Properties { get; }

		/// <summary>
		/// Agent id on the remote machine
		/// </summary>
		public AgentId AgentId { get; }
		
		/// <summary>
		/// Agent version on the remote machine
		/// </summary>
		public string? AgentVersion { get; }
		
		/// <summary>
		/// Compute protocol this resource is using
		/// </summary>
		public ComputeProtocol Protocol { get; }

		/// <summary>
		/// Lease id on the remote machine
		/// </summary>
		public LeaseId LeaseId { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeResource(
			ConnectionMode connectionMode,
			IPAddress ip,
			string? connectionAddress,
			IReadOnlyDictionary<string, ComputeResourcePort> ports,
			ComputeTask task,
			IReadOnlyList<string> properties,
			AgentId agentId,
			string? agentVersion,
			ComputeProtocol protocol,
			LeaseId leaseId)
		{
			ConnectionMode = connectionMode;
			Ip = ip;
			ConnectionAddress = connectionAddress;
			Ports = ports;
			Task = task;
			Properties = properties;
			AgentId = agentId;
			AgentVersion = agentVersion;
			Protocol = protocol;
			LeaseId = leaseId;
		}
	}

	/// <summary>
	/// Dispatches requests for compute resources
	/// </summary>
	public class ComputeTaskSource : TaskSourceBase<ComputeTask>
	{
		class Waiter
		{
			public IAgent Agent { get; }
			public IPAddress Ip { get; }
			public int Port { get; }
			public TaskCompletionSource<CreateLeaseOptions?> Lease { get; } = new TaskCompletionSource<CreateLeaseOptions?>(TaskCreationOptions.RunContinuationsAsynchronously);

			public Waiter(IAgent agent, IPAddress ip, int port)
			{
				Agent = agent;
				Ip = ip;
				Port = port;
			}
		}

		/// <inheritdoc/>
		public override string Type => "Compute";

		/// <inheritdoc/>
		public override TaskSourceFlags Flags => TaskSourceFlags.None;

		readonly AgentRelayService _agentRelay;
		readonly ILeaseCollection _leaseCollection;
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly ILogger _logger;

		readonly object _lockObject = new object();
		readonly Dictionary<ClusterId, LinkedList<Waiter>> _waiters = new Dictionary<ClusterId, LinkedList<Waiter>>();

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeTaskSource(AgentRelayService agentRelay, ILeaseCollection leaseCollection, IOptionsMonitor<ComputeConfig> computeConfig, ILogger<ComputeTaskSource> logger)
		{
			_agentRelay = agentRelay;
			_leaseCollection = leaseCollection;
			_computeConfig = computeConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public override Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			return Task.FromResult(WaitInternalAsync(agent, cancellationToken));
		}

		/// <inheritdoc/>
		public override async Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, ComputeTask payload, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken)
		{
			await base.OnLeaseFinishedAsync(agent, leaseId, payload, outcome, output, logger, cancellationToken);

			// Remove any port mapping associated with this lease ID (as of now, only compute tasks can be relayed)
			await _agentRelay.RemovePortMappingAsync(leaseId);
		}

		async Task<CreateLeaseOptions?> WaitInternalAsync(IAgent agent, CancellationToken cancellationToken)
		{
			string? ipStr = agent.GetPropertyValues("ComputeIp").FirstOrDefault();
			if (ipStr == null || !IPAddress.TryParse(ipStr, out IPAddress? ip))
			{
				return null;
			}

			string? portStr = agent.GetPropertyValues("ComputePort").FirstOrDefault();
			if (portStr == null || !Int32.TryParse(portStr, out int port))
			{
				return null;
			}

			// Add it to the wait queue
			ComputeConfig computeConfig = _computeConfig.CurrentValue;
			Waiter? waiter = null;
			try
			{
				lock (_lockObject)
				{
					foreach (ComputeClusterConfig clusterConfig in computeConfig.Clusters)
					{
						if (clusterConfig.Condition == null || agent.SatisfiesCondition(clusterConfig.Condition))
						{
							LinkedList<Waiter>? list;
							if (!_waiters.TryGetValue(clusterConfig.Id, out list))
							{
								list = new LinkedList<Waiter>();
								_waiters.Add(clusterConfig.Id, list);
							}

							waiter ??= new Waiter(agent, ip, port);
							list.AddFirst(waiter);
						}
					}
				}

				if (waiter != null)
				{
					using IDisposable disposable = cancellationToken.Register(() => waiter.Lease.TrySetResult(null));
					CreateLeaseOptions? lease = await waiter.Lease.Task;
					if (lease != null)
					{
						_logger.LogInformation("Created compute lease for agent {AgentId}", agent.Id);
						return lease;
					}
				}
			}
			finally
			{
				lock (_lockObject)
				{
					if (waiter != null)
					{
						foreach (ComputeClusterConfig clusterConfig in computeConfig.Clusters)
						{
							if (_waiters.TryGetValue(clusterConfig.Id, out LinkedList<Waiter>? list))
							{
								list.Remove(waiter);
							}
						}
					}
				}
			}

			return null;
		}

		/// <inheritdoc/>
		public override async ValueTask GetLeaseDetailsAsync(Any payload, Dictionary<string, string> details, CancellationToken cancellationToken)
		{
			await base.GetLeaseDetailsAsync(payload, details, cancellationToken);

			ComputeTask message = payload.Unpack<ComputeTask>();
			if (!String.IsNullOrEmpty(message.ParentLeaseId) && LeaseId.TryParse(message.ParentLeaseId, out LeaseId parentLeaseId))
			{
				ILease? lease = await _leaseCollection.GetAsync(parentLeaseId, cancellationToken);
				if (lease != null)
				{
					details["parentLogId"] = lease.LogId.ToString() ?? String.Empty;
				}
			}
		}
	}
}
