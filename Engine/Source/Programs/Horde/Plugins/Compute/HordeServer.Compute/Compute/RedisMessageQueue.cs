// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Redis;
using StackExchange.Redis;

namespace HordeServer.Compute
{
	/// <summary>
	/// Interface for a distributed message queue
	/// </summary>
	/// <typeparam name="T">The message type</typeparam>
	interface IMessageQueue<T> where T : class
	{
		/// <summary>
		/// Adds messages to the queue
		/// </summary>
		/// <param name="channelId">The channel to post to</param>
		/// <param name="message">Message to post</param>
		Task PostAsync(string channelId, T message);

		/// <summary>
		/// Waits for a message to be available on the given channel
		/// </summary>
		/// <param name="channelId">The channel to read from</param>
		/// <returns>True if a message was available, false otherwise</returns>
		Task<List<T>> ReadMessagesAsync(string channelId);

		/// <summary>
		/// Waits for a message to be available on the given channel
		/// </summary>
		/// <param name="channelId">The channel to read from</param>
		/// <param name="cancellationToken">May be signalled to stop the wait, without throwing an exception</param>
		/// <returns>True if a message was available, false otherwise</returns>
		Task<List<T>> WaitForMessagesAsync(string channelId, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Implementation of a message queue using Redis
	/// </summary>
	/// <typeparam name="T">The message type. Must be serailizable by RedisSerializer.</typeparam>
	class RedisMessageQueue<T> : IMessageQueue<T>, IDisposable where T : class
	{
		readonly IDatabase _redis;
		readonly RedisKey _keyPrefix;
		readonly RedisChannel _updateChannel;
		readonly Dictionary<string, TaskCompletionSource<bool>> _channelWakeEvents = new Dictionary<string, TaskCompletionSource<bool>>();

		// Time after which entries should be removed
		TimeSpan ExpireTime { get; set; } = TimeSpan.FromSeconds(30);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="redis">The Redis database instance</param>
		/// <param name="keyPrefix">Prefix for keys to use in the database</param>
		public RedisMessageQueue(IDatabase redis, RedisKey keyPrefix)
		{
			_redis = redis;
			_keyPrefix = keyPrefix;
			_updateChannel = RedisChannel.Literal(keyPrefix.Append("updates").ToString());

			redis.Multiplexer.GetSubscriber().Subscribe(_updateChannel, OnChannelUpdate);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_redis.Multiplexer.GetSubscriber().Unsubscribe(_updateChannel, OnChannelUpdate);
		}

		/// <summary>
		/// Callback for a message being posted to a channel
		/// </summary>
		/// <param name="channel"></param>
		/// <param name="value"></param>
		void OnChannelUpdate(RedisChannel channel, RedisValue value)
		{
			TaskCompletionSource<bool>? completionSource;
			lock (_channelWakeEvents)
			{
				_channelWakeEvents.TryGetValue(value.ToString(), out completionSource);
			}
			if (completionSource != null)
			{
				completionSource.TrySetResult(true);
			}
		}

		/// <summary>
		/// Gets the set of messages for a channel
		/// </summary>
		/// <param name="channelId"></param>
		/// <returns></returns>
		RedisListKey<T> GetChannel(string channelId)
		{
			return new RedisListKey<T>(_keyPrefix.Append(channelId));
		}

		/// <inheritdoc/>
		public async Task PostAsync(string channelId, T message)
		{
			RedisListKey<T> channel = GetChannel(channelId);

			long length = await _redis.ListRightPushAsync(channel, message);
			if (length == 1)
			{
				await _redis.PublishAsync(_updateChannel, channelId, CommandFlags.FireAndForget);
			}
			await _redis.KeyExpireAsync(channel, ExpireTime, flags: CommandFlags.FireAndForget);
		}

		async Task<bool> ReadMessagesAsync(RedisListKey<T> list, List<T> messages)
		{
			T? message = await _redis.ListLeftPopAsync(list);
			while (message != null)
			{
				messages.Add(message);
				message = await _redis.ListLeftPopAsync(list);
			}
			return messages.Count > 0;
		}

		/// <inheritdoc/>
		public async Task<List<T>> ReadMessagesAsync(string channelId)
		{
			List<T> messages = new List<T>();
			await ReadMessagesAsync(GetChannel(channelId), messages);
			return messages;
		}

		/// <inheritdoc/>
		public async Task<List<T>> WaitForMessagesAsync(string channelId, CancellationToken cancellationToken)
		{
			List<T> messages = new List<T>();

			RedisListKey<T> channel = GetChannel(channelId);
			while (!cancellationToken.IsCancellationRequested && !await ReadMessagesAsync(channel, messages))
			{
				// Register for notifications on this channel
				TaskCompletionSource<bool>? completionSource;
				lock (_channelWakeEvents)
				{
					if (!_channelWakeEvents.TryGetValue(channelId, out completionSource))
					{
						completionSource = new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously);
						_channelWakeEvents.Add(channelId, completionSource);
					}
				}

				try
				{
					// Read the current queue state again, in case it was modified since we registered.
					if (await ReadMessagesAsync(channel, messages))
					{
						break;
					}

					// Wait for messages to be available
					using (cancellationToken.Register(() => completionSource.TrySetResult(false)))
					{
						await completionSource.Task;
					}
				}
				finally
				{
					lock (_channelWakeEvents)
					{
						_channelWakeEvents.Remove(channelId);
					}
				}
			}

			return messages;
		}
	}
}
