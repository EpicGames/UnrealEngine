// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Threading.Channels;
using EpicGames.Redis;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Compute
{
	using Condition = StackExchange.Redis.Condition;

	/// <summary>
	/// Interface for a queue of compute operations, each of which may have different requirements for the executing machines.
	/// </summary>
	/// <typeparam name="TQueueId">Type used to identify a particular queue</typeparam>
	/// <typeparam name="TTask">Type used to describe a task to be performed</typeparam>
	interface ITaskScheduler<TQueueId, TTask>
		where TQueueId : notnull
		where TTask : class
	{
		/// <summary>
		/// Adds a task to a queue
		/// </summary>
		/// <param name="queueId">The queue identifier</param>
		/// <param name="taskId">The task to add</param>
		/// <param name="atFront">Whether to insert at the front of the queue</param>
		Task EnqueueAsync(TQueueId queueId, TTask taskId, bool atFront);

		/// <summary>
		/// Dequeue any task from a particular queue
		/// </summary>
		/// <param name="queueId">The queue to remove a task from</param>
		/// <returns>Information about the task to be executed</returns>
		Task<TTask?> DequeueAsync(TQueueId queueId);

		/// <summary>
		/// Dequeues a task that the given agent can execute
		/// </summary>
		/// <param name="predicate">Predicate for determining which queues can be removed from</param>
		/// <param name="token">Cancellation token for the operation. Will return a null entry rather than throwing an exception.</param>
		/// <returns>Information about the task to be executed</returns>
		Task<Task<(TQueueId, TTask)?>> DequeueAsync(Func<TQueueId, ValueTask<bool>> predicate, CancellationToken token = default);

		/// <summary>
		/// Gets hashes of all the inactive task queues
		/// </summary>
		/// <returns></returns>
		Task<List<TQueueId>> GetInactiveQueuesAsync();

		/// <summary>
		/// Get number of tasks that a given pool/agent can execute
		/// A read-only operation and will not affect any queue.
		/// </summary>
		/// <param name="predicate">Predicate for determining which queues to read</param>
		/// <param name="token">Cancellation token for the operation</param>
		/// <returns>Number of tasks matching the supplied predicate</returns>
		Task<int> GetNumQueuedTasksAsync(Func<TQueueId, ValueTask<bool>> predicate, CancellationToken token = default);
	}

	/// <summary>
	/// Implementation of <see cref="ITaskScheduler{TQueue, TTask}"/> using Redis for storage
	/// </summary>
	/// <typeparam name="TQueueId">Type used to identify a particular queue</typeparam>
	/// <typeparam name="TTask">Type used to describe a task to be performed</typeparam>
	public sealed class RedisTaskScheduler<TQueueId, TTask> : ITaskScheduler<TQueueId, TTask>, IAsyncDisposable
		where TQueueId : notnull
		where TTask : class
	{
		class Listener
		{
			public Func<TQueueId, ValueTask<bool>> _predicate;
			public TaskCompletionSource<(TQueueId, TTask)?> CompletionSource { get; }

			public Listener(Func<TQueueId, ValueTask<bool>> predicate)
			{
				_predicate = predicate;
				CompletionSource = new TaskCompletionSource<(TQueueId, TTask)?>(TaskCreationOptions.RunContinuationsAsynchronously);
			}
		}

		readonly RedisConnectionPool _redisConnectionPool;
		readonly RedisKey _baseKey;
		readonly RedisSetKey<TQueueId> _queueIndex;
		readonly RedisHashKey<TQueueId, DateTime> _activeQueues; // Queues which are actively being dequeued from
		IReadOnlySet<TQueueId> _localActiveQueues = new HashSet<TQueueId>();
		readonly Stopwatch _resetActiveQueuesTimer = Stopwatch.StartNew();
		readonly List<Listener> _listeners = new List<Listener>();
		readonly RedisChannel<TQueueId> _newQueueChannel;
		readonly Task _queueUpdateTask;
		readonly CancellationTokenSource _cancellationSource = new CancellationTokenSource();
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="redisConnectionPool">The Redis connection pool</param>
		/// <param name="baseKey">Base key for all keys used by this scheduler</param>
		/// <param name="logger"></param>
		public RedisTaskScheduler(RedisConnectionPool redisConnectionPool, RedisKey baseKey, ILogger logger)
		{
			_redisConnectionPool = redisConnectionPool;
			_baseKey = baseKey;
			_queueIndex = new RedisSetKey<TQueueId>(baseKey.Append("index"));
			_activeQueues = new RedisHashKey<TQueueId, DateTime>(baseKey.Append("active"));
			_newQueueChannel = new RedisChannel<TQueueId>(RedisChannel.Literal(baseKey.Append("new_queues").ToString()));
			_logger = logger;

			_queueUpdateTask = Task.Run(() => UpdateQueuesAsync(_cancellationSource.Token));
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (!_queueUpdateTask.IsCompleted)
			{
				await _cancellationSource.CancelAsync();
				try
				{
					await _queueUpdateTask;
				}
				catch (OperationCanceledException)
				{
				}
			}
			_cancellationSource.Dispose();
		}

		/// <summary>
		/// Gets the key for a list of tasks for a particular queue
		/// </summary>
		/// <param name="queueId">The queue identifier</param>
		/// <returns></returns>
		RedisListKey<TTask> GetQueueKey(TQueueId queueId)
		{
			return new RedisListKey<TTask>(_baseKey.Append(RedisSerializer.Serialize(queueId).AsKey()));
		}

		/// <summary>
		/// Pushes a task onto either end of a queue
		/// </summary>
		static Task<long> PushTaskAsync(IDatabaseAsync database, RedisListKey<TTask> list, TTask task, When when, CommandFlags flags, bool atFront)
		{
			if (atFront)
			{
				return database.ListLeftPushAsync(list, task, when, flags);
			}
			else
			{
				return database.ListRightPushAsync(list, task, when, flags);
			}
		}

		/// <summary>
		/// Adds a task to a particular queue, creating and adding that queue to the index if necessary
		/// </summary>
		/// <param name="queueId">The queue to add the task to</param>
		/// <param name="task">Task to be scheduled</param>
		/// <param name="atFront">Whether to add to the front of the queue</param>
		public async Task EnqueueAsync(TQueueId queueId, TTask task, bool atFront)
		{
			RedisListKey<TTask> list = GetQueueKey(queueId);
			for (; ; )
			{
				IDatabase connection = _redisConnectionPool.GetDatabase();

				long newLength = await PushTaskAsync(connection, list, task, When.Exists, CommandFlags.None, atFront);
				if (newLength > 0)
				{
					_logger.LogInformation("Length of queue {QueueId} is {Length}", queueId, newLength);
					break;
				}

				ITransaction transaction = connection.CreateTransaction();
				_ = transaction.SetAddAsync(_queueIndex, queueId, CommandFlags.FireAndForget);
				_ = PushTaskAsync(transaction, list, task, When.Always, CommandFlags.FireAndForget, atFront);

				if (await transaction.ExecuteAsync())
				{
					_logger.LogInformation("Created queue {QueueId}", queueId);
					await connection.PublishAsync(_newQueueChannel, queueId);
					break;
				}

				_logger.LogDebug("EnqueueAsync() retrying...");
			}
		}

		/// <summary>
		/// Dequeues an item for execution by the given agent
		/// </summary>
		/// <param name="predicate">Predicate for queues that tasks can be removed from</param>
		/// <param name="token">Cancellation token for waiting for an item</param>
		/// <returns>The dequeued item, or null if no item is available</returns>
		public async Task<Task<(TQueueId, TTask)?>> DequeueAsync(Func<TQueueId, ValueTask<bool>> predicate, CancellationToken token = default)
		{
			// Compare against all the list of cached queues to see if we can dequeue something from any of them
			TQueueId[] queues = await _redisConnectionPool.GetDatabase().SetMembersAsync(_queueIndex);

			// Try to dequeue an item from the list
			(TQueueId, TTask)? entry = await TryAssignToLocalAgentAsync(queues, predicate);
			if (entry != null)
			{
				return Task.FromResult(entry);
			}

			// Otherwise create a new task to do the wait
			return WaitToDequeueAsync(queues, predicate, token);
		}

		private async Task<(TQueueId, TTask)?> WaitToDequeueAsync(TQueueId[] queues, Func<TQueueId, ValueTask<bool>> predicate, CancellationToken token)
		{
			Listener listener = new Listener(predicate);
			try
			{
				// Add the listener to the global list
				lock (_listeners)
				{
					_listeners.Add(listener);
				}

				// Try to dequeue an item from the list again. An item may have been made available 
				(TQueueId, TTask)? entry = await TryAssignToLocalAgentAsync(queues, predicate);
				if (entry != null)
				{
					if (!listener.CompletionSource.TrySetResult(entry.Value))
					{
						(TQueueId queue, TTask task) = entry.Value;
						await EnqueueAsync(queue, task, true);
					}
				}

				// Wait for an item to be available
				using (IDisposable registration = token.Register(() => listener.CompletionSource.TrySetResult(null)))
				{
					return await listener.CompletionSource.Task;
				}
			}
			finally
			{
				lock (_listeners)
				{
					_listeners.Remove(listener);
				}
			}
		}

		/// <summary>
		/// Attempts to dequeue a task from a set of queue
		/// </summary>
		/// <param name="queueIds">The current array of queues</param>
		/// <param name="predicate">Predicate for queues that tasks can be removed from</param>
		/// <returns>The dequeued item, or null if no item is available</returns>
		async Task<(TQueueId, TTask)?> TryAssignToLocalAgentAsync(TQueueId[] queueIds, Func<TQueueId, ValueTask<bool>> predicate)
		{
			foreach (TQueueId queueId in queueIds)
			{
				if (await predicate(queueId))
				{
					TTask? task = await DequeueAsync(queueId);
					if (task != null)
					{
						return (queueId, task);
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Dequeues the item at the front of a queue
		/// </summary>
		/// <param name="queueId">The queue to dequeue from</param>
		/// <returns>The dequeued item, or null if the queue is empty</returns>
		public async Task<TTask?> DequeueAsync(TQueueId queueId)
		{
			await AddActiveQueueAsync(queueId);

			IDatabase database = _redisConnectionPool.GetDatabase();

			TTask? item = await database.ListLeftPopAsync(GetQueueKey(queueId));
			if (item == null)
			{
				ITransaction transaction = database.CreateTransaction();
				transaction.AddCondition(Condition.KeyNotExists(GetQueueKey(queueId).Inner));
				Task<bool> wasRemoved = transaction.SetRemoveAsync(_queueIndex, queueId);

				if (await transaction.ExecuteAsync() && await wasRemoved)
				{
					_logger.LogInformation("Removed queue {QueueId} from index", queueId);
				}
			}

			return item;
		}

		/// <summary>
		/// Marks a queue key as being actively monitored, preventing it being returned by <see cref="GetInactiveQueuesAsync"/>
		/// </summary>
		/// <param name="queueId">The queue key</param>
		/// <returns></returns>
		async ValueTask AddActiveQueueAsync(TQueueId queueId)
		{
			// Periodically clear out the set of active keys
			TimeSpan resetTime = TimeSpan.FromSeconds(10.0);
			if (_resetActiveQueuesTimer.Elapsed > resetTime)
			{
				lock (_resetActiveQueuesTimer)
				{
					if (_resetActiveQueuesTimer.Elapsed > resetTime)
					{
						_localActiveQueues = new HashSet<TQueueId>();
						_resetActiveQueuesTimer.Restart();
					}
				}
			}

			// Check if the set of active keys already contains the key we're adding. In order to optimize the 
			// common case under heavy load where the key is in the set, updating it creates a full copy of it. Any
			// readers can thus access it without the need for any locking.
			for (; ; )
			{
				IReadOnlySet<TQueueId> localActiveQueuesCopy = _localActiveQueues;
				if (localActiveQueuesCopy.Contains(queueId))
				{
					break;
				}

				HashSet<TQueueId> newLocalActiveQueues = new HashSet<TQueueId>(localActiveQueuesCopy);
				newLocalActiveQueues.Add(queueId);

				if (Interlocked.CompareExchange(ref _localActiveQueues, newLocalActiveQueues, localActiveQueuesCopy) == localActiveQueuesCopy)
				{
					_logger.LogInformation("Refreshing active queue {QueueId}", queueId);
					await _redisConnectionPool.GetDatabase().HashSetAsync(_activeQueues, queueId, DateTime.UtcNow);
					break;
				}
			}
		}

		/// <summary>
		/// Find any inactive keys
		/// </summary>
		/// <returns></returns>
		public async Task<List<TQueueId>> GetInactiveQueuesAsync()
		{
			HashSet<TQueueId> keys = new HashSet<TQueueId>(await _redisConnectionPool.GetDatabase().SetMembersAsync(_queueIndex));
			HashSet<TQueueId> invalidKeys = new HashSet<TQueueId>();

			DateTime minTime = DateTime.UtcNow - TimeSpan.FromMinutes(10.0);

			HashEntry<TQueueId, DateTime>[] entries = await _redisConnectionPool.GetDatabase().HashGetAllAsync(_activeQueues);
			foreach (HashEntry<TQueueId, DateTime> entry in entries)
			{
				if (entry.Value < minTime)
				{
					invalidKeys.Add(entry.Name);
				}
				else
				{
					keys.Remove(entry.Name);
				}
			}

			if (invalidKeys.Count > 0)
			{
				await _redisConnectionPool.GetDatabase().HashDeleteAsync(_activeQueues, invalidKeys.ToArray());
			}

			return keys.ToList();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="predicate"></param>
		/// <param name="token"></param>
		/// <returns></returns>
		public async Task<int> GetNumQueuedTasksAsync(Func<TQueueId, ValueTask<bool>> predicate, CancellationToken token = default)
		{
			HashSet<TQueueId> queueIds = new(await _redisConnectionPool.GetDatabase().SetMembersAsync(_queueIndex));
			long totalTaskCount = 0;
			foreach (TQueueId queueId in queueIds)
			{
				if (await predicate(queueId))
				{
					totalTaskCount += await _redisConnectionPool.GetDatabase().ListLengthAsync(GetQueueKey(queueId));
				}
			}

			return (int)totalTaskCount;
		}

		async Task UpdateQueuesAsync(CancellationToken cancellationToken)
		{
			Channel<TQueueId> newQueues = Channel.CreateUnbounded<TQueueId>();

			await using RedisSubscription _ = await _redisConnectionPool.GetDatabase().Multiplexer.SubscribeAsync(_newQueueChannel, v => newQueues.Writer.TryWrite(v));

			while (await newQueues.Reader.WaitToReadAsync(cancellationToken))
			{
				HashSet<TQueueId> newQueueIds = new HashSet<TQueueId>();
				while (newQueues.Reader.TryRead(out TQueueId? queueId))
				{
					newQueueIds.Add(queueId);
				}
				foreach (TQueueId newQueueId in newQueueIds)
				{
					await TryDispatchToNewQueueAsync(newQueueId);
				}
			}
		}

		async Task<bool> TryDispatchToNewQueueAsync(TQueueId queueId)
		{
			RedisListKey<TTask> queue = GetQueueKey(queueId);

			// Find a local listener that can execute the work
			(TQueueId QueueId, TTask TaskId)? entry = null;
			try
			{
				for (; ; )
				{
					Listener? listener = null;

					// Look for a listener that can execute the task
					HashSet<Listener> checkedListeners = new HashSet<Listener>();
					while (listener == null)
					{
						// Find up to 10 listeners we haven't seen before
						List<Listener> newListeners = new List<Listener>();
						lock (_listeners)
						{
							newListeners.AddRange(_listeners.Where(x => !x.CompletionSource.Task.IsCompleted && checkedListeners.Add(x)).Take(10));
						}
						if (newListeners.Count == 0)
						{
							return false;
						}

						// Check predicates for each one against the new queue
						foreach (Listener newListener in newListeners)
						{
							if (await newListener._predicate(queueId))
							{
								listener = newListener;
								break;
							}
						}
					}

					// Pop an entry from the queue
					if (entry == null)
					{
						TTask? task = await DequeueAsync(queueId);
						if (task == null)
						{
							return false;
						}
						entry = (queueId, task);
					}

					// Assign it to the listener
					if (listener.CompletionSource.TrySetResult(entry.Value))
					{
						entry = null;
					}
				}
			}
			finally
			{
				if (entry != null)
				{
					await EnqueueAsync(entry.Value.QueueId, entry.Value.TaskId, true);
				}
			}
		}
	}
}
