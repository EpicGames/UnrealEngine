// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Sockets;
using EpicGames.Horde.Compute.Clients;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Compute
{
	/// <summary>
	/// Tunnels connections between Horde agents and compute initiators
	/// </summary>
	public sealed class TunnelService : IHostedService, IDisposable
	{
		private const int BufferSize = 4096;

		private readonly CancellationTokenSource _cancellationSource;
		private readonly ComputeServerConfig _staticComputeConfig;
		private readonly ILogger _logger;
		private Task? _serverTask;

		/// <summary>
		/// Accessor for the server task, for tests
		/// </summary>
		internal Task? ServerTask => _serverTask;

		/// <summary>
		/// Constructor
		/// </summary>
		public TunnelService(IOptions<ComputeServerConfig> staticComputeConfig, ILogger<TunnelService> logger)
		{
			_cancellationSource = new CancellationTokenSource();
			_staticComputeConfig = staticComputeConfig.Value;
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_cancellationSource.Dispose();
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			Start(IPAddress.Any);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Start the tunnel server
		/// </summary>
		/// <param name="address">Address to listen on</param>
		/// <returns>Task representing the TCP listener</returns>
		public void Start(IPAddress address)
		{
			_serverTask = RunTcpListenerAsync(address, _staticComputeConfig.ComputeTunnelPort, HandleClientAsync, _cancellationSource.Token);
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cancellationSource.CancelAsync();

			if (_serverTask != null)
			{
				try
				{
					await _serverTask.ConfigureAwait(false);
				}
				catch (OperationCanceledException)
				{
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Tunnel service task completed with an error");
				}
				_serverTask = null;
			}
		}

		async Task HandleClientAsync(TcpClient client, CancellationToken cancellationToken)
		{
			await using NetworkStream stream = client.GetStream();
			using StreamReader reader = new(stream);
			await using StreamWriter streamWriter = new(stream) { AutoFlush = true };

			string? requestStr = await reader.ReadLineAsync(cancellationToken);
			TunnelHandshakeRequest request = TunnelHandshakeRequest.Deserialize(requestStr);

			_logger.LogDebug("Connecting to target {TargetHostname}:{TargetPort}", request.Host, request.Port);

			using TcpClient targetClient = new();
			try
			{
				await targetClient.ConnectAsync(request.Host, request.Port, cancellationToken);
				await using NetworkStream targetStream = targetClient.GetStream();

				await streamWriter.WriteLineAsync(new TunnelHandshakeResponse(true, "Connected to target").Serialize());

				if (client.Client.RemoteEndPoint is IPEndPoint clientEndPoint)
				{
					_logger.LogDebug("Relaying streams between {ClientIp} <-> {TargetHostname}:{TargetPort}", clientEndPoint.Address, request.Host, request.Port);
				}

				Task t1 = RelayStreamsAsync(stream, targetStream, cancellationToken);
				Task t2 = RelayStreamsAsync(targetStream, stream, cancellationToken);
				await Task.WhenAll(t1, t2);
			}
			finally
			{
				if (targetClient.Connected)
				{
					targetClient.Close();
				}
			}
		}

		private static async Task RelayStreamsAsync(Stream input, Stream output, CancellationToken cancellationToken)
		{
			byte[] buffer = new byte[BufferSize];
			int bytesRead;
			while ((bytesRead = await input.ReadAsync(buffer, 0, buffer.Length, cancellationToken)) > 0)
			{
				await output.WriteAsync(buffer, 0, bytesRead, cancellationToken);
			}
		}

		async Task RunTcpListenerAsync(IPAddress address, int port, Func<TcpClient, CancellationToken, Task> handleConnectionAsync, CancellationToken cancellationToken)
		{
			if (port != 0)
			{
				TcpListener listener = new(address, port);
				listener.Start();

				List<Task> tasks = new();
				try
				{
					for (; ; )
					{
						TcpClient client = await listener.AcceptTcpClientAsync(cancellationToken);
						_logger.LogInformation("Received connection from {Remote}", client.Client.RemoteEndPoint);

						Task task = HandleConnectionGuardedAsync(client, handleConnectionAsync, cancellationToken);
						tasks.Add(task);
					}
				}
				finally
				{
					await Task.WhenAll(tasks);
					listener.Stop();
				}
			}
		}

		async Task HandleConnectionGuardedAsync(TcpClient tcpClient, Func<TcpClient, CancellationToken, Task> handleConnectionAsync, CancellationToken cancellationToken)
		{
			try
			{
				await handleConnectionAsync(tcpClient, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Exception while handling request: {Message}", ex.Message);
			}
			finally
			{
				tcpClient.Dispose();
			}
		}
	}
}
