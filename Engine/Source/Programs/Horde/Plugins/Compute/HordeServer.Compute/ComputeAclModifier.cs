// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Agents.Sessions;
using HordeServer.Agents.Software;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Utilities;

namespace HordeServer
{
	class ComputeAclModifier : IDefaultAclModifier
	{
		/// <inheritdoc/>
		public void Apply(DefaultAclBuilder acl)
		{
			acl.AddCustomRole(new AclClaimConfig(ClaimTypes.Role, "internal:AgentRegistration"), new[] { AgentAclAction.CreateAgent, SessionAclAction.CreateSession });
			acl.AddCustomRole(HordeClaims.AgentRegistrationClaim, new[] { AgentAclAction.CreateAgent, SessionAclAction.CreateSession, AgentAclAction.UpdateAgent, AgentSoftwareAclAction.DownloadSoftware, PoolAclAction.CreatePool, PoolAclAction.UpdatePool, PoolAclAction.ViewPool, PoolAclAction.DeletePool, PoolAclAction.ListPools, ServerAclAction.ViewCosts });
			acl.AddCustomRole(HordeClaims.DownloadSoftwareClaim, new[] { AgentSoftwareAclAction.DownloadSoftware });
			acl.AddCustomRole(HordeClaims.UploadToolsClaim, new[] { AgentSoftwareAclAction.UploadSoftware });

			acl.AddDefaultReadAction(AgentAclAction.ListAgents);
			acl.AddDefaultReadAction(AgentAclAction.ViewAgent);
			acl.AddDefaultReadAction(LogAclAction.ViewEvent);
			acl.AddDefaultReadAction(LogAclAction.ViewLog);
			acl.AddDefaultReadAction(PoolAclAction.ListPools);
			acl.AddDefaultReadAction(PoolAclAction.ViewPool);
		}
	}
}
