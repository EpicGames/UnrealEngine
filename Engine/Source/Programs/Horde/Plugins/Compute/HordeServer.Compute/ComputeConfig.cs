// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Net;
using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Tools;
using EpicGames.Serialization;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Agents.Sessions;
using HordeServer.Agents.Software;
using HordeServer.Compute;
using HordeServer.Configuration;
using HordeServer.Logs;
using HordeServer.Plugins;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Configuration for the compute system
	/// </summary>
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class ComputeConfig : IPluginConfig
	{
		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Macros within the compute config
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

		/// <summary>
		/// Inherited root acl
		/// </summary>
		public AclConfig Acl { get; private set; } = null!;

		/// <summary>
		/// Config version number
		/// </summary>
		public ConfigVersion VersionEnum { get; private set; }

		/// <summary>
		/// Telemetry store for lease and compute telemetry data
		/// </summary>
		public TelemetryStoreId TelemetryStoreId { get; set; } = TelemetryStoreId.Default;

		/// <summary>
		/// List of costs of a particular agent type
		/// </summary>
		public List<AgentRateConfig> Rates { get; set; } = new List<AgentRateConfig>();

		/// <summary>
		/// List of compute profiles
		/// </summary>
		public List<ComputeClusterConfig> Clusters { get; set; } = new List<ComputeClusterConfig>();

		/// <summary>
		/// List of pools
		/// </summary>
		public List<PoolConfig> Pools { get; set; } = new List<PoolConfig>();

		/// <summary>
		/// List of costs of a particular agent type
		/// </summary>
		public List<AgentSoftwareConfig> Software { get; set; } = new List<AgentSoftwareConfig>();

		/// <summary>
		/// List of networks
		/// </summary>
		public List<NetworkConfig> Networks { get; set; } = new List<NetworkConfig>();

		private readonly Dictionary<ClusterId, ComputeClusterConfig> _computeClusterLookup = new Dictionary<ClusterId, ComputeClusterConfig>();
		private readonly Dictionary<PoolId, PoolConfig> _poolLookup = new Dictionary<PoolId, PoolConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			Acl = configOptions.ParentAcl;
			VersionEnum = configOptions.Version;

			_computeClusterLookup.Clear();
			foreach (ComputeClusterConfig computeCluster in Clusters)
			{
				_computeClusterLookup.Add(computeCluster.Id, computeCluster);
				computeCluster.PostLoad(configOptions.ParentAcl);
			}

			_poolLookup.Clear();
			foreach (PoolConfig pool in Pools)
			{
				_poolLookup.Add(pool.Id, pool);
			}
			ConfigObject.MergeDefaults<string, PoolConfig>(Pools.Select(x => (x.Id.ToString(), x.Base?.ToString(), x)));
		}

		/// <summary>
		/// Authorize a user to perform a specific action
		/// </summary>
		public bool Authorize(AclAction action, ClaimsPrincipal principal)
			=> Acl.Authorize(action, principal);

		/// <summary>
		/// Attempts to get compute cluster configuration from this object
		/// </summary>
		/// <param name="clusterId">Compute cluster id</param>
		/// <param name="config">Receives the cluster configuration on success</param>
		/// <returns>True on success</returns>
		public bool TryGetComputeCluster(ClusterId clusterId, [NotNullWhen(true)] out ComputeClusterConfig? config)
			=> _computeClusterLookup.TryGetValue(clusterId, out config);

		/// <summary>
		/// Attempt to resolve an IP address to a network config
		/// </summary>
		/// <param name="ip">IP address to resolve</param>
		/// <param name="networkConfig">Config for the network</param>
		/// <returns>True if the IP address was resolved</returns>
		public bool TryGetNetworkConfig(IPAddress ip, [NotNullWhen(true)] out NetworkConfig? networkConfig)
		{
			foreach (NetworkConfig nc in Networks)
			{
				if (nc.Id != null && IsIpInBlock(ip, nc.CidrBlock))
				{
					networkConfig = nc;
					return true;
				}
			}

			networkConfig = null;
			return false;
		}

		private static bool IsIpInBlock(IPAddress ip, string? cidrBlock)
		{
			if (cidrBlock == null)
			{
				return false;
			}

			if (cidrBlock == "0.0.0.0/0")
			{
				return true;
			}

			string[] parts = cidrBlock.Split('/');
			if (parts.Length != 2 || !IPAddress.TryParse(parts[0], out IPAddress? address) || !Int32.TryParse(parts[1], out int maskBits))
			{
				return false;
			}

			byte[] networkPrefixBytes = address.GetAddressBytes();
			Array.Reverse(networkPrefixBytes);

			uint networkPrefix = BitConverter.ToUInt32(networkPrefixBytes, 0);
			uint subnetMask = 0xffffffff;
			subnetMask <<= 32 - maskBits;
			uint ipRangeStart = networkPrefix & subnetMask;
			uint ipRangeEnd = networkPrefix | (subnetMask ^ 0xffffffff);

			byte[] ipBytes = ip.GetAddressBytes();
			Array.Reverse(ipBytes);
			uint ipUint = BitConverter.ToUInt32(ipBytes, 0);
			return ipUint >= ipRangeStart && ipUint <= ipRangeEnd;
		}

		/// <summary>
		/// Attempts to get configuration for a pool from this object
		/// </summary>
		/// <param name="poolId">The pool identifier</param>
		/// <param name="config">Configuration for the pool</param>
		/// <returns>True if the pool configuration was found</returns>
		public bool TryGetPool(PoolId poolId, [NotNullWhen(true)] out PoolConfig? config) => _poolLookup.TryGetValue(poolId, out config);
	}

	/// <summary>
	/// Selects different agent software versions by evaluating a condition
	/// </summary>
	[DebuggerDisplay("{ToolId}")]
	public class AgentSoftwareConfig
	{
		/// <summary>
		/// Tool identifier
		/// </summary>
		public ToolId ToolId { get; set; }

		/// <summary>
		/// Condition for using this channel
		/// </summary>
		public Condition? Condition { get; set; }
	}

	/// <summary>
	/// Describes the monetary cost of agents matching a particular criteria
	/// </summary>
	public class AgentRateConfig
	{
		/// <summary>
		/// Condition string
		/// </summary>
		[CbField("c")]
		public Condition? Condition { get; set; }

		/// <summary>
		/// Rate for this agent
		/// </summary>
		[CbField("r")]
		public double Rate { get; set; }
	}

	/// <summary>
	/// Describes a network
	/// The ID describes any logical grouping, such as region, availability zone, rack or office location. 
	/// </summary>
	public class NetworkConfig
	{
		/// <summary>
		/// ID for this network
		/// </summary>
		[CbField("id")]
		public string? Id { get; set; }

		/// <summary>
		/// CIDR block
		/// </summary>
		[CbField("cb")]
		public string? CidrBlock { get; set; }

		/// <summary>
		/// Human-readable description
		/// </summary>
		[CbField("d")]
		public string? Description { get; set; }

		/// <summary>
		/// Compute ID for this network (used when allocating compute resources)
		/// </summary>
		[CbField("cid")]
		public string? ComputeId { get; set; }
	}
	
	/// <summary>
	/// Configuration settings for Unreal Build Accelerator (UBA) for a compute cluster
	/// Contains cache server connectivity and other UBA-related settings.
	/// Ensure all cache related fields are non-null to activate use of cache.
	/// </summary>
	public class UbaComputeClusterConfig
	{
		/// <summary>
		/// Host and port for connecting to the UBA cache server (host:port format)
		/// As specified with `-port=[PORT]` to UBA cache process.
		/// Serves all cache requests and must be accessible to both agents and initiators running UBA / Unreal Build Tool.
		/// </summary>
		public string? CacheEndpoint { get; set; }
		
		/// <summary>
		/// Host and port for HTTP management of UBA cache server (host:port format)
		/// As specified with `-http=[PORT]` to UBA cache process.
		/// This port is used by Horde to manage the UBA cache, such as adding new cache session keys for initiators
		/// For improved security, only allow Horde server to access this port
		/// </summary>
		public string? CacheHttpEndpoint { get; set; }
		
		/// <summary>
		/// AES encryption key used for secure Horde-to-cache-server communication
		/// As specified with `-httpcrypto=[KEY]` to UBA cache process.
		/// </summary>
		public string? CacheHttpCrypto { get; set; }
	}

	/// <summary>
	/// Profile for executing compute requests
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class ComputeClusterConfig
	{
		/// <summary>
		/// Name of the partition
		/// </summary>
		public ClusterId Id { get; set; } = new ClusterId("default");

		/// <summary>
		/// Name of the namespace to use
		/// </summary>
		public string NamespaceId { get; set; } = "horde.compute";

		/// <summary>
		/// Name of the input bucket
		/// </summary>
		public string RequestBucketId { get; set; } = "requests";

		/// <summary>
		/// Name of the output bucket
		/// </summary>
		public string ResponseBucketId { get; set; } = "responses";

		/// <summary>
		/// Filter for agents to include
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Access control list
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();
		
		/// <summary>
		/// Configuration settings for Unreal Build Accelerator (UBA) for a compute cluster
		/// </summary>
		public UbaComputeClusterConfig? Uba { get; set; }

		/// <summary>
		/// Callback post loading this config file
		/// </summary>
		/// <param name="parentAcl">The parent config instance</param>
		public void PostLoad(AclConfig parentAcl)
		{
			AclAction[] aclActions = AclConfig.GetActions(
				[
					typeof(AgentAclAction),
					typeof(AgentSoftwareAclAction),
					typeof(ComputeAclAction),
					typeof(LeaseAclAction),
					typeof(LogAclAction),
					typeof(PoolAclAction),
					typeof(SessionAclAction)
				]);
			Acl.PostLoad(parentAcl, $"compute:{Id}", aclActions);
		}

		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to validate</param>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);
	}
}
