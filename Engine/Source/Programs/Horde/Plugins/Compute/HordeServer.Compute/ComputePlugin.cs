// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon;
using Amazon.AutoScaling;
using Amazon.CloudWatch;
using Amazon.EC2;
using Amazon.Extensions.NETCore.Setup;
using Amazon.SQS;
using EpicGames.Horde.Agents;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Enrollment;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Agents.Relay;
using HordeServer.Agents.Telemetry;
using HordeServer.Auditing;
using HordeServer.Aws;
using HordeServer.Compute;
using HordeServer.Logs;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Tasks;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the storage plugin
	/// </summary>
	[Plugin("Compute", GlobalConfigType = typeof(ComputeConfig), ServerConfigType = typeof(ComputeServerConfig))]
	public class ComputePlugin : IPluginStartup
	{
		readonly IServerInfo _serverInfo;
		readonly ComputeServerConfig _staticComputeConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputePlugin(IServerInfo serverInfo, ComputeServerConfig staticComputeConfig)
		{
			_serverInfo = serverInfo;
			_staticComputeConfig = staticComputeConfig;
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
			app.UseEndpoints(endpoints =>
			{
				endpoints.MapGrpcService<RpcService>();
				endpoints.MapGrpcService<LogRpcService>();
				endpoints.MapGrpcService<AgentRelayService>();
				endpoints.MapGrpcService<EnrollmentRpc>();
			});
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<IAuditLog<AgentId>>(sp => sp.GetRequiredService<IAuditLogFactory<AgentId>>().Create("Agents.Log", "AgentId"));

			services.AddSingleton<IDefaultAclModifier, ComputeAclModifier>();
			services.AddSingleton<IPluginResponseFilter, ComputeResponseFilter>();

			services.AddSingleton<AgentCollection>();
			services.AddSingleton<IAgentCollection>(sp => sp.GetRequiredService<AgentCollection>());
			services.AddHostedService(sp => sp.GetRequiredService<AgentCollection>());

			services.AddSingleton<AgentScheduler>();
			services.AddSingleton<IAgentScheduler>(sp => sp.GetRequiredService<AgentScheduler>());
			services.AddHostedService(sp => sp.GetRequiredService<AgentScheduler>());

			services.AddSingleton<LeaseCollection>();
			services.AddSingleton<ILeaseCollection>(sp => sp.GetRequiredService<LeaseCollection>());

			services.AddSingleton<ILogCollection, LogCollection>();
			services.AddSingleton<IPoolCollection, PoolCollection>();

			services.AddSingleton<IAgentVersionProvider, AgentVersionProvider>();

			services.AddSingleton<AgentService>();
			services.AddHostedService<LeaseService>();
			services.AddSingleton(provider => new Lazy<AgentService>(provider.GetRequiredService<AgentService>));

			services.AddSingleton<AwsAutoScalingLifecycleService>();
			services.AddSingleton<FleetService>();
			services.AddSingleton<IFleetManagerFactory, FleetManagerFactory>();
			services.AddSingleton<IPoolSizeStrategyFactory, NoOpPoolSizeStrategyFactory>();
			services.AddSingleton<IPoolSizeStrategyFactory, LeaseUtilizationStrategyFactory>();

			// Associate IFleetManager interface with the default implementation from config for convenience
			// Though most fleet managers are created on a per-pool basis
			services.AddSingleton<IFleetManager>(ctx => ctx.GetRequiredService<IFleetManagerFactory>().CreateFleetManager(FleetManagerType.Default));

			if (_staticComputeConfig.ComputeTunnelPort != 0)
			{
				services.AddSingleton<TunnelService>();
				services.AddHostedService<TunnelService>(sp => sp.GetRequiredService<TunnelService>());
			}

			// Runs the agent relay service for all run modes to notify long-polling requests
			services.AddSingleton<AgentRelayService>();
			services.AddHostedService(provider => provider.GetRequiredService<AgentRelayService>());

			services.AddSingleton<SessionConflictService>();
			services.AddSingleton<ComputeService>();
			services.AddSingleton<EnrollmentService>();
			services.AddSingleton<LogTailService>();
			services.AddSingleton<PoolService>();

			// Always run tail service on workers, to receive tail notifications.
			services.AddHostedService(provider => provider.GetRequiredService<LogTailService>());

			// Always run agent service too; need to be able to listen to Redis for events on any server.
			services.AddHostedService(provider => provider.GetRequiredService<AgentService>());

			// Notifications to task sources about lease completion events
			services.AddHostedService<TaskSourceNotificationService>();

			// Create the agent telemetry collection, and register the hosted service so we can flush from any server.
			services.AddSingleton<AgentTelemetryCollection>();
			services.AddSingleton<IAgentTelemetryCollection>(sp => sp.GetRequiredService<AgentTelemetryCollection>());
			services.AddHostedService(provider => provider.GetRequiredService<AgentTelemetryCollection>());

			if (!_serverInfo.ReadOnlyMode)
			{
				services.AddSingleton<ComputeTaskSource>();
				services.AddSingleton<ITaskSource, ComputeTaskSource>(provider => provider.GetRequiredService<ComputeTaskSource>());

				services.AddSingleton<ITaskSource, UpgradeTaskSource>();
				services.AddSingleton<ITaskSource, ShutdownTaskSource>();
				services.AddSingleton<ITaskSource, RestartTaskSource>();

				if (_serverInfo.IsRunModeActive(RunMode.Worker))
				{
					services.AddHostedService(provider => provider.GetRequiredService<FleetService>());
					services.AddHostedService(provider => provider.GetRequiredService<ComputeService>());
					services.AddHostedService(provider => provider.GetRequiredService<EnrollmentService>());
				}
			}

			if (_staticComputeConfig.WithAws)
			{
				AWSOptions awsOptions = _serverInfo.Configuration.GetAWSOptions();
				services.AddDefaultAWSOptions(awsOptions);
				if (awsOptions.Region == null && Environment.GetEnvironmentVariable("AWS_REGION") == null)
				{
					awsOptions.Region = RegionEndpoint.USEast1;
				}
				
				HashSet<RegionEndpoint> regions = [];
				if (awsOptions.Region != null)
				{
					regions.Add(awsOptions.Region);	
				}
				foreach (string awsRegionStr in _staticComputeConfig.AwsRegions)
				{
					RegionEndpoint? region = RegionEndpoint.EnumerableAllRegions.FirstOrDefault(x => x.SystemName == awsRegionStr);
					if (region == null)
					{
						throw new Exception("Invalid AWS region: " + awsRegionStr);
					}
					regions.Add(region);
				}
				
				services.AddAWSService<IAmazonAutoScaling>();
				services.AddAWSService<IAmazonSQS>();
				services.AddAWSService<IAmazonEC2>();
				
				// Combine each CloudWatch client (one per region) under one that replicates requests to all
				services.AddSingleton<IAmazonCloudWatch>(_ =>
				{
					return new AwsCloudWatchMultiplexer(regions.Select(x => new AmazonCloudWatchClient(x)).ToList<IAmazonCloudWatch>());
				});
				
				services.AddSingleton<AwsCloudWatchMetricExporter>();

				services.AddSingleton<IPoolSizeStrategyFactory, LeaseUtilizationAwsMetricStrategyFactory>();
				services.AddSingleton<IPoolSizeStrategyFactory, ComputeQueueAwsMetricStrategyFactory>();

				if (_serverInfo.IsRunModeActive(RunMode.Worker))
				{
					services.AddHostedService(provider => provider.GetRequiredService<AwsAutoScalingLifecycleService>());
					services.AddHostedService(provider => provider.GetRequiredService<AwsCloudWatchMetricExporter>());
				}
			}
		}
	}

	/// <summary>
	/// Helper methods for compute config
	/// </summary>
	public static class ComputePluginExtensions
	{
		/// <summary>
		/// Configures the compute plugin
		/// </summary>
		public static void AddComputeConfig(this IDictionary<PluginName, IPluginConfig> dictionary, ComputeConfig computeConfig)
			=> dictionary[new PluginName("Compute")] = computeConfig;

		/// <summary>
		/// Get the compute plugin config
		/// </summary>
		public static ComputeConfig GetComputeConfig(this IDictionary<PluginName, IPluginConfig> plugins)
		{
			IPluginConfig pluginConfig = plugins[new PluginName("compute")];
			return (ComputeConfig)pluginConfig;
		}
	}
}
