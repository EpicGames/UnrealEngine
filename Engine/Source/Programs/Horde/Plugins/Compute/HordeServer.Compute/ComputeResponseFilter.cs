// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Options;

namespace HordeServer
{
	class ComputeResponseFilter : IPluginResponseFilter
	{
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;

		public ComputeResponseFilter(IOptionsMonitor<ComputeConfig> computeConfig)
		{
			_computeConfig = computeConfig;
		}

		public void Apply(HttpContext context, object response)
		{
			if (response is GetDashboardFeaturesResponse featuresResponse)
			{
				ComputeConfig computeConfig = _computeConfig.CurrentValue;

				bool hasDbPools = computeConfig.VersionEnum < ConfigVersion.PoolsInConfigFiles;
				featuresResponse.ShowPoolEditor = hasDbPools
					&& (computeConfig.Authorize(PoolAclAction.CreatePool, context.User)
						|| computeConfig.Authorize(PoolAclAction.UpdatePool, context.User));

				featuresResponse.ShowRemoteDesktop = computeConfig.Authorize(AgentAclAction.UpdateAgent, context.User);
			}
		}
	}
}
