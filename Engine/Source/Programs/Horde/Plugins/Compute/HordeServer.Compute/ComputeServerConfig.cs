// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Nodes;
using System.Text.Json.Serialization;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Pools;
using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Static configuration for the compute plugin
	/// </summary>
	public class ComputeServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Whether to enable the upgrade task source, always upgrading agents to the latest version
		/// </summary>
		public bool EnableUpgradeTasks { get; set; } = true;

		/// <summary>
		/// Whether to enable Amazon Web Services (AWS) specific features
		/// </summary>
		public bool WithAws { get; set; } = false;
		
		/// <summary>
		/// List of AWS regions for Horde to be aware of (e.g. us-east-1 or eu-central-1)
		/// Right now, this is only used for replicating CloudWatch metrics to multiple regions
		/// <see cref="Amazon.RegionEndpoint" />
		/// </summary>
		public string[] AwsRegions { get; set; } = [];
		
		/// <summary>
		/// AWS SQS queue URLs where lifecycle events from EC2 auto-scaling are received
		/// <see cref="AwsAutoScalingLifecycleService" />
		/// </summary>
		public string[] AwsAutoScalingQueueUrls { get; set; } = Array.Empty<string>();

		/// <summary>
		/// Default fleet manager to use (when not specified by pool)
		/// </summary>
		public FleetManagerType FleetManagerV2 { get; set; } = FleetManagerType.NoOp;

		/// <summary>
		/// Config for the fleet manager (serialized JSON)
		/// </summary>
		[JsonConverter(typeof(JsonObjectOrStringConverter))]
		public JsonObject? FleetManagerV2Config { get; set; }

		/// <summary>
		/// Whether to automatically enroll agents in the farm
		/// </summary>
		public bool AutoEnrollAgents { get; set; }

		/// <summary>
		/// Default agent pool sizing strategy for pools that doesn't have one explicitly configured
		/// </summary>
		public PoolSizeStrategy DefaultAgentPoolSizeStrategy { get; set; } = PoolSizeStrategy.LeaseUtilization;

		/// <summary>
		/// Scale-out cooldown for auto-scaling agent pools (in seconds). Can be overridden by per-pool settings.
		/// </summary>
		public int AgentPoolScaleOutCooldownSeconds { get; set; } = 60; // 1 min

		/// <summary>
		/// Scale-in cooldown for auto-scaling agent pools (in seconds). Can be overridden by per-pool settings.
		/// </summary>
		public int AgentPoolScaleInCooldownSeconds { get; set; } = 1200; // 20 mins

		/// <summary>
		/// Port to listen on for tunneling compute sockets to agents
		/// </summary>
		public int ComputeTunnelPort { get; set; }

		/// <summary>
		/// What address (host:port) clients should connect to for compute socket tunneling
		/// Port may differ from <see cref="ComputeTunnelPort" /> if Horde server is behind a reverse proxy/firewall
		/// </summary>
		public string? ComputeTunnelAddress { get; set; }
	}
}
