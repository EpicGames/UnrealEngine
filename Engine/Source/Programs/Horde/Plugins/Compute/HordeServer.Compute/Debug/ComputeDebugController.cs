// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Compute;
using Google.Protobuf;
using Horde.Common.Rpc;
using HordeServer.Agents.Relay;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using MongoDB.Bson;

namespace HordeServer.Debug
{
	/// <summary>
	/// Debug functionality for the compute system
	/// </summary>
	[ApiController]
	[Authorize]
	[DebugEndpoint]
	[Tags("Debug")]
	class ComputeDebugController : HordeControllerBase
	{
		readonly AgentRelayService _agentRelayService;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;

		public ComputeDebugController(AgentRelayService agentRelayService, IOptionsSnapshot<ComputeConfig> computeConfig)
		{
			_agentRelayService = agentRelayService;
			_computeConfig = computeConfig;
		}

		/// <summary>
		/// Add a port mapping for agent relay
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/relay/add-port")]
		public async Task<ActionResult<object>> AddPortMappingAsync([FromQuery] string? clientIpStr = null, [FromQuery] string? agentIpStr = null, [FromQuery] int? agentPort = null)
		{
			if (!_computeConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			if (clientIpStr == null || !IPAddress.TryParse(clientIpStr, out IPAddress? clientIp))
			{
				return BadRequest("Unable to read or convert query parameter 'clientIp'");
			}

			if (agentIpStr == null || !IPAddress.TryParse(agentIpStr, out IPAddress? agentIp))
			{
				return BadRequest("Unable to read or convert query parameter 'agentIp'");
			}

			if (agentPort == null)
			{
				return BadRequest("Bad query parameter 'agentPort'");
			}

			string bogusLeaseId = ObjectId.GenerateNewId().ToString();
			List<Port> ports = new()
			{
				new Port { RelayPort = -1, AgentPort = agentPort.Value, Protocol = PortProtocol.Tcp }
			};

			PortMapping portMapping = await _agentRelayService.AddPortMappingAsync(new ClusterId("default"), LeaseId.Parse(bogusLeaseId), clientIp, agentIp, ports);
			return JsonFormatter.Default.Format(portMapping);
		}

		/// <summary>
		/// Get the network ID for a given IP address
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/network-id")]
		public ActionResult<object> GetNetworkId([FromQuery] string? ipAddress = null)
		{
			if (!_computeConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			if (ipAddress == null || !IPAddress.TryParse(ipAddress, out IPAddress? ip))
			{
				return BadRequest("Unable to read or convert query parameter 'ipAddress'");
			}

			_computeConfig.Value.TryGetNetworkConfig(ip, out NetworkConfig? networkConfig);
			return networkConfig == null ? StatusCode(StatusCodes.Status500InternalServerError, "Unable to find a network config for the IP") : Ok(networkConfig);
		}
	}
}
