<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <Protobuf Include="Protos\horde\enrollment_rpc.proto" ProtoRoot="Protos" GrpcServices="Server" />
    <Protobuf Include="Protos\horde\enrollment_rpc_messages.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\horde_rpc.proto" ProtoRoot="Protos" GrpcServices="Server" />
    <Protobuf Include="Protos\horde\horde_rpc_messages.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\relay_rpc.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\common\agent_status.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\common\lease_outcome.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\common\lease_state.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\messages\agent_capabilities.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\messages\agent_requirements.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\messages\agent_workspace.proto" ProtoRoot="Protos" Access="Public" />
    <Protobuf Include="Protos\horde\messages\lease.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\messages\session.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\tasks\compute_task.proto" ProtoRoot="Protos" Access="Public" />
    <Protobuf Include="Protos\horde\tasks\conform_task.proto" ProtoRoot="Protos" Access="Public" />
    <Protobuf Include="Protos\horde\tasks\job_task.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\tasks\restart_task.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\tasks\shutdown_task.proto" ProtoRoot="Protos" />
    <Protobuf Include="Protos\horde\tasks\upgrade_task.proto" ProtoRoot="Protos" />
  </ItemGroup>

  <ItemGroup>
    <FrameworkReference Include="Microsoft.AspNetCore.App" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.AutoScaling" Version="3.7.301.9" />
    <PackageReference Include="AWSSDK.CloudWatch" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.EC2" Version="3.7.306.6" />
    <PackageReference Include="AWSSDK.Extensions.NETCore.Setup" Version="3.7.300" />
    <PackageReference Include="AWSSDK.S3" Version="3.7.304.1" />
    <PackageReference Include="AWSSDK.SimpleSystemsManagement" Version="3.7.304.24" />
    <PackageReference Include="AWSSDK.SQS" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.SecurityToken" Version="3.7.300.13" />
    <PackageReference Include="Azure.Storage.Blobs" Version="12.19.1" />
    <PackageReference Include="MongoDB.Driver" Version="2.22.0" />
    <PackageReference Include="Grpc.AspNetCore" Version="2.59.0" />
    <PackageReference Include="Grpc.AspNetCore.Server.Reflection" Version="2.59.0" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.Telemetry\EpicGames.Analytics.Telemetry.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\Tools\HordeServer.Tools\HordeServer.Tools.csproj" />
  </ItemGroup>

</Project>
