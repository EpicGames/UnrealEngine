// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using HordeServer.Agents;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;

namespace HordeServer.Logs
{
	/// <summary>
	/// Information about a log file
	/// </summary>
	public interface ILog
	{
		/// <summary>
		/// Identifier for the log. Randomly generated.
		/// </summary>
		LogId Id { get; }

		/// <summary>
		/// Unique id of the job containing this log
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The lease allowed to write to this log
		/// </summary>
		LeaseId? LeaseId { get; }

		/// <summary>
		/// The session allowed to write to this log
		/// </summary>
		SessionId? SessionId { get; }

		/// <summary>
		/// Type of data stored in this log 
		/// </summary>
		LogType Type { get; }

		/// <summary>
		/// Namespace containing the log data
		/// </summary>
		NamespaceId NamespaceId { get; }

		/// <summary>
		/// Name of the ref used to store data for this log
		/// </summary>
		RefName RefName { get; }

		/// <summary>
		/// Acl scope that owns this log
		/// </summary>
		AclScopeName AclScopeName { get; }

		/// <summary>
		/// Delete this log
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task DeleteAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Read a set of lines from the given log file
		/// </summary>
		/// <param name="index">Index of the first line to read</param>
		/// <param name="count">Maximum number of lines to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of lines</returns>
		Task<List<Utf8String>> ReadLinesAsync(int index, int count, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets metadata about the log file
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Metadata about the log file</returns>
		Task<LogMetadata> GetMetadataAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Updates the line count for a log file (v2 backend only)
		/// </summary>
		/// <param name="lineCount">New line count for the log file</param>
		/// <param name="complete">Flag indicating whether the log is complete, or can still be tailed for additional data</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>The updated log file document</returns>
		Task<ILog> UpdateLineCountAsync(int lineCount, bool complete, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets lines from the given log 
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Data for the requested range</returns>
		Task<Stream> OpenRawStreamAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets lines from the given log 
		/// </summary>
		/// <param name="offset"></param>
		/// <param name="length"></param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Data for the requested range</returns>
		Task<Stream> OpenRawStreamAsync(long offset, long length, CancellationToken cancellationToken);

		/// <summary>
		/// Parses a stream of json text and outputs plain text
		/// </summary>
		/// <param name="outputStream">Output stream to receive the text data</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async text</returns>
		Task CopyPlainTextStreamAsync(Stream outputStream, CancellationToken cancellationToken = default);

		/// <summary>
		/// Search for the specified text in a log file
		/// </summary>
		/// <param name="text">Text to search for</param>
		/// <param name="firstLine">Line to start search from</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="stats">Receives stats for the search</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>List of line numbers containing the given term</returns>
		Task<List<int>> SearchLogDataAsync(string text, int firstLine, int count, SearchStats stats, CancellationToken cancellationToken);

		#region Events

		/// <summary>
		/// Creates a new event
		/// </summary>
		/// <param name="newEvents">List of events to create</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task AddEventsAsync(List<NewLogEventData> newEvents, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds events within a log file
		/// </summary>
		/// <param name="spanId">Optional span to filter events by</param>
		/// <param name="index">Start index within the matching results</param>
		/// <param name="count">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of events matching the query</returns>
		Task<List<ILogAnchor>> GetAnchorsAsync(ObjectId? spanId = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		#endregion
	}

	/// <summary>
	/// Anchor within a log files
	/// </summary>
	public interface ILogAnchor
	{
		/// <summary>
		/// Unique id of the log containing this event
		/// </summary>
		public LogId LogId { get; }

		/// <summary>
		/// Severity of the event
		/// </summary>
		public LogEventSeverity Severity { get; }

		/// <summary>
		/// Index of the first line for this event
		/// </summary>
		public int LineIndex { get; }

		/// <summary>
		/// Number of lines in the event
		/// </summary>
		public int LineCount { get; }

		/// <summary>
		/// Span id for this log event
		/// </summary>
		public ObjectId? SpanId { get; }

		/// <summary>
		/// Gets the data for this log event
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<ILogEventData> GetDataAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Interface for event data
	/// </summary>
	public interface ILogEventData
	{
		/// <summary>
		/// The log level
		/// </summary>
		LogEventSeverity Severity { get; }

		/// <summary>
		/// The type of event
		/// </summary>
		EventId? EventId { get; }

		/// <summary>
		/// The complete rendered message, in plaintext
		/// </summary>
		string Message { get; }

		/// <summary>
		/// Gets this event data as a JSON objects
		/// </summary>
		IReadOnlyList<JsonLogEvent> Lines { get; }
	}

	/// <summary>
	/// Represents a node in the graph
	/// </summary>
	public class NewLogEventData
	{
		/// <summary>
		/// Severity of the event
		/// </summary>
		public LogEventSeverity Severity { get; set; }

		/// <summary>
		/// Index of the first line for this event
		/// </summary>
		public int LineIndex { get; set; }

		/// <summary>
		/// Number of lines in the event
		/// </summary>
		public int LineCount { get; set; }

		/// <summary>
		/// The span this this event belongs to
		/// </summary>
		public ObjectId? SpanId { get; set; }
	}

	/// <summary>
	/// Extensions for parsing log event properties
	/// </summary>
	public static class LogEventExtensions
	{
		/// <summary>
		/// Find all properties of the given type in a particular log line
		/// </summary>
		/// <param name="data">Line data</param>
		/// <param name="type">Type of property to return</param>
		/// <returns></returns>
		public static IEnumerable<JsonProperty> FindPropertiesOfType(this ILogEventData data, Utf8String type)
		{
			return data.Lines.SelectMany(x => x.FindPropertiesOfType(type));
		}
	}

	/// <summary>
	/// Metadata about a log file
	/// </summary>
	public class LogMetadata
	{
		/// <summary>
		/// Length of the log file
		/// </summary>
		public long Length { get; set; }

		/// <summary>
		/// Number of lines in the log file
		/// </summary>
		public int MaxLineIndex { get; set; }
	}

	/// <summary>
	/// Extension methods for log files
	/// </summary>
	public static class LogExtensions
	{
		/// <summary>
		/// Creates a new event
		/// </summary>
		/// <param name="log">Log to add an event to</param>
		/// <param name="newEvent">The new event to vreate</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AddEventAsync(this ILog log, NewLogEventData newEvent, CancellationToken cancellationToken = default)
		{
			return log.AddEventsAsync(new List<NewLogEventData> { newEvent }, cancellationToken);
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular template
		/// </summary>
		/// <param name="log">The template to check</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True if the action is authorized</returns>
		public static bool AuthorizeForSession(this ILog log, ClaimsPrincipal user)
		{
			if (log.SessionId != null && user.HasSessionClaim(log.SessionId.Value))
			{
				return true;
			}
			if (log.LeaseId != null && user.HasLeaseClaim(log.LeaseId.Value))
			{
				return true;
			}
			return false;
		}

		/// <summary>
		/// Parses a stream of json text and outputs plain text
		/// </summary>
		/// <param name="log">The log file to query</param>
		/// <param name="outputStream">Output stream to receive the text data</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async text</returns>
		public static async Task CopyRawStreamAsync(this ILog log, Stream outputStream, CancellationToken cancellationToken)
		{
			await using Stream stream = await log.OpenRawStreamAsync(cancellationToken);
			await stream.CopyToAsync(outputStream, cancellationToken);
		}
	}
}
