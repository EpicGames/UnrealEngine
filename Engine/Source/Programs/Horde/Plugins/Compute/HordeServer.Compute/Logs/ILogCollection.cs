// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using MongoDB.Bson;

namespace HordeServer.Logs
{
	/// <summary>
	/// Wrapper around the jobs collection in a mongo DB
	/// </summary>
	public interface ILogCollection
	{
		/// <summary>
		/// Adds a new log
		/// </summary>
		/// <param name="jobId">Unique id of the job that owns this log file</param>
		/// <param name="leaseId">Agent lease allowed to update the log</param>
		/// <param name="sessionId">Agent session allowed to update the log</param>
		/// <param name="type">Type of events to be stored in the log</param>
		/// <param name="logId">ID of the log file (optional)</param>
		/// <param name="aclScopeName">Name of the acl scope to use for authorizing access to this log</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>The new log file document</returns>
		Task<ILog> AddAsync(JobId jobId, LeaseId? leaseId, SessionId? sessionId, LogType type, LogId? logId = null, AclScopeName aclScopeName = default, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a log by ID
		/// </summary>
		/// <param name="logId">Unique id of the log file</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>The log instance</returns>
		Task<ILog?> GetAsync(LogId logId, CancellationToken cancellationToken = default);

		#region Events

		/// <summary>
		/// Finds a list of events for a set of spans
		/// </summary>
		/// <param name="spanIds">The span ids</param>
		/// <param name="logIds">List of log ids to query</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>List of events for this issue</returns>
		Task<IReadOnlyList<ILogAnchor>> FindAnchorsForSpansAsync(IEnumerable<ObjectId> spanIds, LogId[]? logIds = null, int index = 0, int count = 10, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update the span for an event
		/// </summary>
		/// <param name="events">The events to update</param>
		/// <param name="spanId">New span id</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async task</returns>
		Task AddSpanToEventsAsync(IEnumerable<ILogAnchor> events, ObjectId spanId, CancellationToken cancellationToken = default);

		#endregion
	}
}
