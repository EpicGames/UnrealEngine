// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;

namespace HordeServer.Logs
{
	/// <summary>
	/// Interface which allows 
	/// </summary>
	public interface ILogExtAuthProvider
	{
		/// <summary>
		/// Determine if the given principal can access a particular log
		/// </summary>
		/// <param name="log">The log to query</param>
		/// <param name="action">Action to perform</param>
		/// <param name="principal">The principal to authorize</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public ValueTask<bool> AuthorizeAsync(ILog log, AclAction action, ClaimsPrincipal principal, CancellationToken cancellationToken = default);
	}
}
