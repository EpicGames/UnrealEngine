// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Bson;

namespace HordeServer.Logs
{
	/// <summary>
	/// Interface which provides issue identifiers for event ids in log documents
	/// </summary>
	public interface ILogExtIssueProvider
	{
		/// <summary>
		/// Gets an issue it for the given span
		/// </summary>
		/// <param name="spanId">Identifier for the span</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Issue identifier</returns>
		public ValueTask<int?> GetIssueIdAsync(ObjectId spanId, CancellationToken cancellationToken = default);
	}
}
