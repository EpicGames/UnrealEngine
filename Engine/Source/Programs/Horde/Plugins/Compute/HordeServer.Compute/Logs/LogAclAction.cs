// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Logs
{
	/// <summary>
	/// ACL actions for logs
	/// </summary>
	public static class LogAclAction
	{
		/// <summary>
		/// Ability to create a log. Implicitly granted to agents.
		/// </summary>
		public static AclAction CreateLog { get; } = new AclAction("CreateLog");

		/// <summary>
		/// Ability to update log metadata
		/// </summary>
		public static AclAction UpdateLog { get; } = new AclAction("UpdateLog");

		/// <summary>
		/// Ability to view a log contents
		/// </summary>
		public static AclAction ViewLog { get; } = new AclAction("ViewLog");

		/// <summary>
		/// Ability to write log data
		/// </summary>
		public static AclAction WriteLogData { get; } = new AclAction("WriteLogData");

		/// <summary>
		/// Ability to create events
		/// </summary>
		public static AclAction CreateEvent { get; } = new AclAction("CreateEvent");

		/// <summary>
		/// Ability to view events
		/// </summary>
		public static AclAction ViewEvent { get; } = new AclAction("ViewEvent");
	}
}
