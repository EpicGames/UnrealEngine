// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.CompilerServices;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Logs
{
	/// <summary>
	/// Wrapper around the jobs collection in a mongo DB
	/// </summary>
	public sealed class LogCollection : ILogCollection, IDisposable
	{
		class Log : ILog
		{
			readonly LogCollection _collection;
			readonly LogDocument _document;

			LogId ILog.Id => _document.Id;
			JobId ILog.JobId => _document.JobId;
			LeaseId? ILog.LeaseId => _document.LeaseId;
			SessionId? ILog.SessionId => _document.SessionId;
			LogType ILog.Type => _document.Type;
			NamespaceId ILog.NamespaceId => _document.NamespaceId;
			RefName ILog.RefName => _document.RefName;
			AclScopeName ILog.AclScopeName => _document.AclScopeName;

			public Log(LogCollection collection, LogDocument document)
			{
				_collection = collection;
				_document = document;
			}

			public Task DeleteAsync(CancellationToken cancellationToken = default)
				=> _collection.DeleteAsync(_document, cancellationToken);

			public async Task<ILog> UpdateLineCountAsync(int lineCount, bool complete, CancellationToken cancellationToken = default)
			{
				LogDocument newDocument = await _collection.UpdateLineCountAsync(_document, lineCount, complete, cancellationToken);
				return new Log(_collection, newDocument);
			}

			public Task<List<Utf8String>> ReadLinesAsync(int index, int count, CancellationToken cancellationToken = default)
				=> _collection.ReadLinesAsync(_document, index, count, cancellationToken);

			public Task<LogMetadata> GetMetadataAsync(CancellationToken cancellationToken)
				=> _collection.GetMetadataAsync(_document, cancellationToken);

			public Task<Stream> OpenRawStreamAsync(CancellationToken cancellationToken = default)
				=> _collection.OpenRawStreamAsync(_document, 0, Int64.MaxValue, cancellationToken);

			public Task<Stream> OpenRawStreamAsync(long offset, long length, CancellationToken cancellationToken)
				=> _collection.OpenRawStreamAsync(_document, offset, length, cancellationToken);

			public Task CopyPlainTextStreamAsync(Stream outputStream, CancellationToken cancellationToken = default)
				=> _collection.CopyPlainTextStreamAsync(this, outputStream, cancellationToken);

			public Task<List<int>> SearchLogDataAsync(string text, int firstLine, int count, SearchStats stats, CancellationToken cancellationToken)
				=> _collection.SearchLogDataAsync(_document, text, firstLine, count, stats, cancellationToken);

			public Task AddEventsAsync(List<NewLogEventData> newEvents, CancellationToken cancellationToken = default)
				=> _collection.AddEventsAsync(_document.Id, newEvents, cancellationToken);

			public Task<List<ILogAnchor>> GetAnchorsAsync(ObjectId? spanId = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
				=> _collection.GetAnchorsAsync(this, spanId, index, count, cancellationToken);
		}

		class LogDocument
		{
			[BsonRequired, BsonId]
			public LogId Id { get; set; }

			[BsonRequired]
			public JobId JobId { get; set; }

			[BsonIgnoreIfNull]
			public LeaseId? LeaseId { get; set; }

			[BsonIgnoreIfNull]
			public SessionId? SessionId { get; set; }

			public LogType Type { get; set; }
			public bool UseNewStorageBackend { get; set; }

			[BsonIgnoreIfNull]
			public int? MaxLineIndex { get; set; }

			[BsonIgnoreIfNull]
			public long? IndexLength { get; set; }

			public List<LogChunkDocument> Chunks { get; set; } = new List<LogChunkDocument>();

			public int LineCount { get; set; }

			public NamespaceId NamespaceId { get; set; } = Namespace.Logs;
			public RefName RefName { get; set; }
			public AclScopeName AclScopeName { get; set; }

			[BsonIgnoreIfDefault]
			public bool Complete { get; set; }

			[BsonRequired]
			public int UpdateIndex { get; set; }

			[BsonConstructor]
			private LogDocument()
			{
			}

			public LogDocument(JobId jobId, LeaseId? leaseId, SessionId? sessionId, LogType type, LogId? logId, NamespaceId namespaceId, AclScopeName aclScopeName)
			{
				Id = logId ?? LogIdUtils.GenerateNewId();
				JobId = jobId;
				LeaseId = leaseId;
				SessionId = sessionId;
				Type = type;
				UseNewStorageBackend = true;
				MaxLineIndex = 0;
				NamespaceId = namespaceId;
				RefName = new RefName(Id.ToString());
				AclScopeName = aclScopeName;
			}
		}

		class LogChunkDocument
		{
			public long Offset { get; set; }
			public int Length { get; set; }
			public int LineIndex { get; set; }

			[BsonIgnoreIfNull]
			public string? Server { get; set; }

			[BsonConstructor]
			public LogChunkDocument()
			{
			}

			public LogChunkDocument(LogChunkDocument other)
			{
				Offset = other.Offset;
				Length = other.Length;
				LineIndex = other.LineIndex;
				Server = other.Server;
			}

			public LogChunkDocument Clone()
			{
				return (LogChunkDocument)MemberwiseClone();
			}
		}

		class LogAnchor : ILogAnchor
		{
			readonly ILog _log;
			readonly LogCollection _collection;
			readonly LogAnchorDocument _document;

			public LogAnchorDocument Document => _document;

			LogId ILogAnchor.LogId => _document.Id.LogId;
			LogEventSeverity ILogAnchor.Severity => _document.IsWarning ? LogEventSeverity.Warning : LogEventSeverity.Error;
			int ILogAnchor.LineIndex => _document.Id.LineIndex;
			int ILogAnchor.LineCount => _document.LineCount ?? 1;
			ObjectId? ILogAnchor.SpanId => _document.SpanId;

			public LogAnchor(ILog log, LogCollection collection, LogAnchorDocument document)
			{
				_log = log;
				_collection = collection;
				_document = document;
			}

			public async Task<ILogEventData> GetDataAsync(CancellationToken cancellationToken = default)
				=> await _collection.GetEventDataAsync(_log, _document.Id.LineIndex, _document.LineCount ?? 1, cancellationToken);
		}

		class LogAnchorId
		{
			[BsonElement("l")]
			public LogId LogId { get; set; }

			[BsonElement("n")]
			public int LineIndex { get; set; }
		}

		class LogAnchorDocument
		{
			[BsonId]
			public LogAnchorId Id { get; set; }

			[BsonElement("w"), BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool IsWarning { get; set; }

			[BsonElement("c"), BsonIgnoreIfNull]
			public int? LineCount { get; set; }

			[BsonElement("s")]
			public ObjectId? SpanId { get; set; }

			[BsonConstructor]
			public LogAnchorDocument()
			{
				Id = new LogAnchorId();
			}

			public LogAnchorDocument(LogId logId, LogEventSeverity severity, int lineIndex, int lineCount, ObjectId? spanId)
			{
				Id = new LogAnchorId { LogId = logId, LineIndex = lineIndex };
				IsWarning = severity == LogEventSeverity.Warning;
				LineCount = (lineCount > 1) ? (int?)lineCount : null;
				SpanId = spanId;
			}

			public LogAnchorDocument(LogId logId, NewLogEventData data)
				: this(logId, data.Severity, data.LineIndex, data.LineCount, data.SpanId)
			{
			}
		}

		class LegacyLogEventDocument
		{
			public ObjectId Id { get; set; }
			public DateTime Time { get; set; }
			public LogEventSeverity Severity { get; set; }
			public LogId LogId { get; set; }
			public int LineIndex { get; set; }
			public int LineCount { get; set; }

			public string? Message { get; set; }

			[BsonIgnoreIfNull, BsonElement("IssueId2")]
			public int? IssueId { get; set; }

			public BsonDocument? Data { get; set; }

			public int UpgradeVersion { get; set; }
		}

		readonly IMongoCollection<LogDocument> _logCollection;
		readonly IMongoCollection<LogAnchorDocument> _logEvents;
		readonly IMongoCollection<LegacyLogEventDocument> _legacyEvents;
		readonly LogTailService _logTailService;
		readonly StorageService _storageService;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly IMemoryCache _logCache;

		/// <summary>
		/// Constructor
		/// </summary>
		public LogCollection(IMongoService mongoService, LogTailService logTailService, StorageService storageService, Tracer tracer, ILogger<LogCollection> logger)
		{
			_logCollection = mongoService.GetCollection<LogDocument>("LogFiles");
			_logTailService = logTailService;
			_storageService = storageService;
			_tracer = tracer;
			_logger = logger;
			_logCache = new MemoryCache(new MemoryCacheOptions());

			List<MongoIndex<LogAnchorDocument>> logEventIndexes = new List<MongoIndex<LogAnchorDocument>>();
			logEventIndexes.Add(keys => keys.Ascending(x => x.Id.LogId));
			logEventIndexes.Add(keys => keys.Ascending(x => x.SpanId).Ascending(x => x.Id));
			_logEvents = mongoService.GetCollection<LogAnchorDocument>("LogEvents", logEventIndexes);

			_legacyEvents = mongoService.GetCollection<LegacyLogEventDocument>("Events", keys => keys.Ascending(x => x.LogId));
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_logCache.Dispose();
		}

		/// <inheritdoc/>
		public async Task<ILog> AddAsync(JobId jobId, LeaseId? leaseId, SessionId? sessionId, LogType type, LogId? logId, AclScopeName aclScopeName, CancellationToken cancellationToken)
		{
			LogDocument newLog = new LogDocument(jobId, leaseId, sessionId, type, logId, Namespace.Logs, aclScopeName);
			await _logCollection.InsertOneAsync(newLog, null, cancellationToken);
			return new Log(this, newLog);
		}

		/// <inheritdoc/>
		async Task DeleteAsync(LogDocument logDocument, CancellationToken cancellationToken)
		{
			await _logEvents.DeleteManyAsync(x => x.Id.LogId == logDocument.Id, cancellationToken);
			await _legacyEvents.DeleteManyAsync(x => x.LogId == logDocument.Id, cancellationToken);
			await _logCollection.DeleteOneAsync(x => x.Id == logDocument.Id, cancellationToken);

			if (logDocument.UseNewStorageBackend)
			{
				IStorageNamespace storageNamespace = _storageService.GetNamespace(logDocument.NamespaceId);
				await storageNamespace.RemoveRefAsync(logDocument.RefName, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<ILog?> GetAsync(LogId logId, CancellationToken cancellationToken)
		{
			LogDocument? logDocument = await _logCollection.Find<LogDocument>(x => x.Id == logId).FirstOrDefaultAsync(cancellationToken);
			if (logDocument == null || !logDocument.UseNewStorageBackend)
			{
				return null;
			}
			return new Log(this, logDocument);
		}

		/// <inheritdoc/>
		async Task<List<Utf8String>> ReadLinesAsync(LogDocument log, int index, int count, CancellationToken cancellationToken)
		{
			List<Utf8String> lines = new List<Utf8String>();

			IStorageNamespace storageNamespace = _storageService.GetNamespace(log.NamespaceId);

			int maxIndex = index + count;
			bool complete = log.Complete;

			LogNode? root = await storageNamespace.TryReadRefTargetAsync<LogNode>(log.RefName, cancellationToken: cancellationToken);
			if (root != null)
			{
				int chunkIdx = root.TextChunkRefs.GetChunkForLine(index);
				for (; index < maxIndex && chunkIdx < root.TextChunkRefs.Count; chunkIdx++)
				{
					LogChunkRef chunk = root.TextChunkRefs[chunkIdx];
					LogChunkNode chunkData = await chunk.Target.ReadBlobAsync(cancellationToken: cancellationToken);

					for (; index < maxIndex && index < chunk.LineIndex; index++)
					{
						lines.Add(new Utf8String($"Internal error; missing data for line {index}\n"));
					}

					for (; index < maxIndex && index < chunk.LineIndex + chunk.LineCount; index++)
					{
						lines.Add(chunkData.GetLine(index - chunk.LineIndex));
					}
				}
				complete |= root.Complete;
			}

			if (!complete)
			{
				await _logTailService.EnableTailingAsync(log.Id, root?.LineCount ?? 0, cancellationToken);
				if (index < maxIndex)
				{
					await _logTailService.ReadAsync(log.Id, index, maxIndex - index, lines);
				}
			}

			return lines;
		}

		/// <inheritdoc/>
		async Task<LogMetadata> GetMetadataAsync(LogDocument log, CancellationToken cancellationToken)
		{
			LogMetadata metadata = new LogMetadata();
			if (log.Complete)
			{
				metadata.MaxLineIndex = log.LineCount;
			}
			else
			{
				metadata.MaxLineIndex = await _logTailService.GetFullLineCountAsync(log.Id, log.LineCount, cancellationToken);
			}
			return metadata;
		}

		async Task<LogDocument> UpdateLineCountAsync(LogDocument log, int lineCount, bool complete, CancellationToken cancellationToken)
		{
			FilterDefinition<LogDocument> filter = Builders<LogDocument>.Filter.Eq(x => x.Id, log.Id);
			UpdateDefinition<LogDocument> update = Builders<LogDocument>.Update.Set(x => x.LineCount, lineCount).Set(x => x.Complete, complete).Inc(x => x.UpdateIndex, 1);
			return await _logCollection.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<LogDocument, LogDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}
		
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members")]
		async Task<(int, long)> GetLineOffsetAsync(LogDocument log, int lineIdx, CancellationToken cancellationToken)
		{
			IStorageNamespace storageNamespace = _storageService.GetNamespace(log.NamespaceId);

			LogNode? root = await storageNamespace.TryReadRefTargetAsync<LogNode>(log.RefName, cancellationToken: cancellationToken);
			if (root == null)
			{
				return (0, 0);
			}

			int chunkIdx = root.TextChunkRefs.GetChunkForLine(lineIdx);
			LogChunkRef chunk = root.TextChunkRefs[chunkIdx];
			LogChunkNode chunkData = await chunk.Target.ReadBlobAsync(cancellationToken: cancellationToken);

			if (lineIdx < chunk.LineIndex)
			{
				lineIdx = chunk.LineIndex;
			}

			int maxLineIndex = chunk.LineIndex + chunkData.LineCount;
			if (lineIdx >= maxLineIndex)
			{
				lineIdx = maxLineIndex;
			}

			long offset = chunk.Offset + chunkData.LineOffsets[lineIdx - chunk.LineIndex];
			return (lineIdx, offset);
		}

		/// <inheritdoc/>
		async Task<Stream> OpenRawStreamAsync(LogDocument log, long offset, long length, CancellationToken cancellationToken)
		{
			IStorageNamespace storageNamespace = _storageService.GetNamespace(log.NamespaceId);

			LogNode? root = await storageNamespace.TryReadRefTargetAsync<LogNode>(log.RefName, cancellationToken: cancellationToken);
			if (root == null || root.TextChunkRefs.Count == 0)
			{
				return new MemoryStream(Array.Empty<byte>(), false);
			}
			else
			{
				int lastChunkIdx = root.TextChunkRefs.Count - 1;

				// Clamp the length of the request
				LogChunkRef lastChunk = root.TextChunkRefs[lastChunkIdx];
				if (length > lastChunk.Offset)
				{
					long lastChunkLength = lastChunk.Length;
					if (lastChunkLength <= 0)
					{
						LogChunkNode lastChunkNode = await lastChunk.Target.ReadBlobAsync(cancellationToken: cancellationToken);
						lastChunkLength = lastChunkNode.Length;
					}
					length = Math.Min(length, (lastChunk.Offset + lastChunkLength) - offset);
				}

				// Create the new stream
				return new ResponseStream(root, offset, length);
			}
		}

		/// <inheritdoc/>
		async Task<List<int>> SearchLogDataAsync(LogDocument log, string text, int firstLine, int count, SearchStats searchStats, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(LogCollection)}.{nameof(SearchLogDataAsync)}");
			span.SetAttribute("logId", log.Id.ToString());
			span.SetAttribute("text", text);
			span.SetAttribute("count", count);

			List<int> results = new List<int>();
			if (count > 0)
			{
				IAsyncEnumerable<int> enumerable = SearchLogDataInternalNewAsync(log, text, firstLine, searchStats, cancellationToken);

				await using IAsyncEnumerator<int> enumerator = enumerable.GetAsyncEnumerator(cancellationToken);
				while (await enumerator.MoveNextAsync() && results.Count < count)
				{
					results.Add(enumerator.Current);
				}
			}

			_logger.LogDebug("Search for \"{SearchText}\" in log {LogId} found {NumResults}/{MaxResults} results, took {Time}ms ({@Stats})", text, log.Id, results.Count, count, timer.ElapsedMilliseconds, searchStats);
			return results;
		}

		async IAsyncEnumerable<int> SearchLogDataInternalNewAsync(LogDocument log, string text, int firstLine, SearchStats searchStats, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			SearchTerm searchText = new SearchTerm(text);
			IStorageNamespace storageNamespace = _storageService.GetNamespace(log.NamespaceId);

			// Search the index
			if (log.LineCount > 0)
			{
				LogNode? root = await storageNamespace.ReadRefTargetAsync<LogNode>(log.RefName, cancellationToken: cancellationToken);
				if (root != null)
				{
					LogIndexNode index = await root.IndexRef.ReadBlobAsync(cancellationToken: cancellationToken);
					await foreach (int lineIdx in index.SearchAsync(firstLine, searchText, searchStats, cancellationToken: cancellationToken))
					{
						yield return lineIdx;
					}
					if (root.Complete)
					{
						yield break;
					}
					firstLine = root.LineCount;
				}
			}

			// Search any tail data we have
			if (!log.Complete)
			{
				for (; ; )
				{
					Utf8String[] lines = await ReadTailAsync(log, firstLine, cancellationToken);
					if (lines.Length == 0)
					{
						break;
					}

					for (int idx = 0; idx < lines.Length; idx++)
					{
						if (SearchTerm.FindNextOcurrence(lines[idx].Span, 0, searchText) != -1)
						{
							yield return firstLine + idx;
						}
					}

					firstLine += lines.Length;
				}
			}
		}

		async Task<Utf8String[]> ReadTailAsync(LogDocument log, int index, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			const int BatchSize = 128;

			if (log.Complete)
			{
				return Array.Empty<Utf8String>();
			}

			string cacheKey = $"{log.Id}@{index}";
			if (_logCache.TryGetValue(cacheKey, out Utf8String[]? lines))
			{
				return lines!;
			}

			lines = (await _logTailService.ReadAsync(log.Id, index, BatchSize)).ToArray();
			if (log.Type == LogType.Json)
			{
				LogChunkBuilder builder = new LogChunkBuilder(lines.Sum(x => x.Length));
				foreach (Utf8String line in lines)
				{
					builder.AppendJsonAsPlainText(line.Span, _logger);
				}
				lines = lines.ToArray();
			}

			if (lines.Length == BatchSize)
			{
				int length = lines.Sum(x => x.Length);
				using (ICacheEntry entry = _logCache.CreateEntry(cacheKey))
				{
					entry.SetSlidingExpiration(TimeSpan.FromMinutes(1.0));
					entry.SetSize(length);
					entry.SetValue(lines);
				}
			}

			return lines;
		}

		/// <inheritdoc/>
		async Task CopyPlainTextStreamAsync(ILog log, Stream outputStream, CancellationToken cancellationToken)
		{
			long offset = 0;
			long length = Int64.MaxValue;

			using (Stream stream = await log.OpenRawStreamAsync(0, Int64.MaxValue, cancellationToken))
			{
				byte[] readBuffer = new byte[4096];
				int readBufferLength = 0;

				byte[] writeBuffer = new byte[4096];
				int writeBufferLength = 0;

				while (length > 0)
				{
					// Add more data to the buffer
					int readBytes = await stream.ReadAsync(readBuffer.AsMemory(readBufferLength, readBuffer.Length - readBufferLength), cancellationToken);
					readBufferLength += readBytes;

					// Copy as many lines as possible to the output
					int convertedBytes = 0;
					for (int endIdx = 1; endIdx < readBufferLength; endIdx++)
					{
						if (readBuffer[endIdx] == '\n')
						{
							writeBufferLength = GuardedConvertToPlainText(readBuffer.AsSpan(convertedBytes, endIdx - convertedBytes), writeBuffer, writeBufferLength, _logger);
							convertedBytes = endIdx + 1;
						}
					}

					// If there's anything in the write buffer, write it out
					if (writeBufferLength > 0)
					{
						if (offset < writeBufferLength)
						{
							int writeLength = (int)Math.Min((long)writeBufferLength - offset, length);
							await outputStream.WriteAsync(writeBuffer.AsMemory((int)offset, writeLength), cancellationToken);
							length -= writeLength;
						}
						offset = Math.Max(offset - writeBufferLength, 0);
						writeBufferLength = 0;
					}

					// If we were able to read something, shuffle down the rest of the buffer. Otherwise expand the read buffer.
					if (convertedBytes > 0)
					{
						Buffer.BlockCopy(readBuffer, convertedBytes, readBuffer, 0, readBufferLength - convertedBytes);
						readBufferLength -= convertedBytes;
					}
					else if (readBufferLength > 0)
					{
						Array.Resize(ref readBuffer, readBuffer.Length + 128);
						writeBuffer = new byte[readBuffer.Length];
					}

					// Exit if we didn't read anything in this iteration
					if (readBytes == 0)
					{
						break;
					}
				}
			}
		}

		/// <summary>
		/// Helper method for catching exceptions in <see cref="ConvertToPlainText(ReadOnlySpan{Byte}, Byte[], Int32)"/>
		/// </summary>
		public static int GuardedConvertToPlainText(ReadOnlySpan<byte> input, byte[] output, int outputOffset, ILogger logger)
		{
			try
			{
				return ConvertToPlainText(input, output, outputOffset);
			}
			catch (Exception ex)
			{
				logger.LogWarning(ex, "Unable to convert log line to plain text: {Line}", Encoding.UTF8.GetString(input));
				output[outputOffset] = (byte)'\n';
				return outputOffset + 1;
			}
		}

		/// <summary>
		/// Converts a JSON log line to plain text
		/// </summary>
		/// <param name="input">The JSON data</param>
		/// <param name="output">Output buffer for the converted line</param>
		/// <param name="outputOffset">Offset within the buffer to write the converted data</param>
		/// <returns></returns>
		public static int ConvertToPlainText(ReadOnlySpan<byte> input, byte[] output, int outputOffset)
		{
			if (IsEmptyOrWhitespace(input))
			{
				output[outputOffset] = (byte)'\n';
				return outputOffset + 1;
			}

			Utf8JsonReader reader = new Utf8JsonReader(input);
			if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					if (!reader.ValueTextEquals("message"))
					{
						reader.Skip();
						continue;
					}
					if (!reader.Read() || reader.TokenType != JsonTokenType.String)
					{
						reader.Skip();
						continue;
					}

					int unescapedLength = UnescapeUtf8(reader.ValueSpan, output.AsSpan(outputOffset));
					outputOffset += unescapedLength;

					output[outputOffset] = (byte)'\n';
					outputOffset++;

					break;
				}
			}
			return outputOffset;
		}

		/// <summary>
		/// Determines if the given line is empty
		/// </summary>
		/// <param name="input">The input data</param>
		/// <returns>True if the given text is empty</returns>
		static bool IsEmptyOrWhitespace(ReadOnlySpan<byte> input)
		{
			for (int idx = 0; idx < input.Length; idx++)
			{
				byte v = input[idx];
				if (v != (byte)'\n' && v != '\r' && v != ' ')
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Unescape a json utf8 string
		/// </summary>
		/// <param name="source">Source span of bytes</param>
		/// <param name="target">Target span of bytes</param>
		/// <returns>Length of the converted data</returns>
		static int UnescapeUtf8(ReadOnlySpan<byte> source, Span<byte> target)
		{
			int length = 0;
			for (; ; )
			{
				// Copy up to the next backslash
				int backslash = source.IndexOf((byte)'\\');
				if (backslash == -1)
				{
					source.CopyTo(target);
					length += source.Length;
					break;
				}
				else if (backslash > 0)
				{
					source.Slice(0, backslash).CopyTo(target);
					source = source.Slice(backslash);
					target = target.Slice(backslash);
					length += backslash;
				}

				// Check what the escape code is
				if (source[1] == 'u')
				{
					char[] chars = { (char)((StringUtils.ParseHexByte(source, 2) << 8) | StringUtils.ParseHexByte(source, 4)) };
					int encodedLength = Encoding.UTF8.GetBytes(chars.AsSpan(), target);
					source = source.Slice(6);
					target = target.Slice(encodedLength);
					length += encodedLength;
				}
				else
				{
					target[0] = source[1] switch
					{
						(byte)'\"' => (byte)'\"',
						(byte)'\\' => (byte)'\\',
						(byte)'b' => (byte)'\b',
						(byte)'f' => (byte)'\f',
						(byte)'n' => (byte)'\n',
						(byte)'r' => (byte)'\r',
						(byte)'t' => (byte)'\t',
						_ => source[1]
					};
					source = source.Slice(2);
					target = target.Slice(1);
					length++;
				}
			}
			return length;
		}

		#region Log events

		/// <inheritdoc/>
		Task AddEventsAsync(LogId logId, List<NewLogEventData> newEvents, CancellationToken cancellationToken)
		{
			return _logEvents.InsertManyAsync(newEvents.ConvertAll(x => new LogAnchorDocument(logId, x)), cancellationToken: cancellationToken);
		}

		/// <inheritdoc/>
		async Task<List<ILogAnchor>> GetAnchorsAsync(ILog log, ObjectId? spanId = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			_logger.LogInformation("Querying for log events for log {LogId}", log.Id);

			FilterDefinitionBuilder<LogAnchorDocument> builder = Builders<LogAnchorDocument>.Filter;

			FilterDefinition<LogAnchorDocument> filter = builder.Eq(x => x.Id.LogId, log.Id);
			if (spanId != null)
			{
				filter &= builder.Eq(x => x.SpanId, spanId.Value);
			}

			IFindFluent<LogAnchorDocument, LogAnchorDocument> results = _logEvents.Find(filter).SortBy(x => x.Id);
			if (index != null)
			{
				results = results.Skip(index.Value);
			}
			if (count != null)
			{
				results = results.Limit(count.Value);
			}

			List<LogAnchorDocument> logEventDocuments = await results.ToListAsync(cancellationToken);
			return logEventDocuments.ConvertAll<ILogAnchor>(x => new LogAnchor(log, this, x));
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ILogAnchor>> FindAnchorsForSpansAsync(IEnumerable<ObjectId> spanIds, LogId[]? logIds, int index, int count, CancellationToken cancellationToken)
		{
			FilterDefinition<LogAnchorDocument> filter = Builders<LogAnchorDocument>.Filter.In(x => x.SpanId, spanIds.Select<ObjectId, ObjectId?>(x => x));
			if (logIds != null && logIds.Length > 0)
			{
				filter &= Builders<LogAnchorDocument>.Filter.In(x => x.Id.LogId, logIds);
			}

			List<LogAnchor> logEvents = new List<LogAnchor>();

			List<LogAnchorDocument> logEventDocuments = await _logEvents.Find(filter).Skip(index).Limit(count).ToListAsync(cancellationToken);
			foreach (IGrouping<LogId, LogAnchorDocument> logEventGroup in logEventDocuments.GroupBy(x => x.Id.LogId))
			{
				ILog? log = await GetAsync(logEventGroup.Key, cancellationToken);
				if (log != null)
				{
					logEvents.AddRange(logEventGroup.Select(x => new LogAnchor(log, this, x)));
				}
			}

			return logEvents;
		}

		/// <inheritdoc/>
		public async Task AddSpanToEventsAsync(IEnumerable<ILogAnchor> events, ObjectId spanId, CancellationToken cancellationToken)
		{
			FilterDefinition<LogAnchorDocument> eventFilter = Builders<LogAnchorDocument>.Filter.In(x => x.Id, events.Select(x => ((LogAnchor)x).Document.Id));
			UpdateDefinition<LogAnchorDocument> eventUpdate = Builders<LogAnchorDocument>.Update.Set(x => x.SpanId, spanId);
			await _logEvents.UpdateManyAsync(eventFilter, eventUpdate, cancellationToken: cancellationToken);
		}

		class LogEventData : ILogEventData
		{
			public string? _message;
			public IReadOnlyList<JsonLogEvent> Lines { get; }

			EventId? ILogEventData.EventId => (Lines.Count > 0) ? Lines[0].EventId : null;
			LogEventSeverity ILogEventData.Severity => (Lines.Count == 0) ? LogEventSeverity.Information : (Lines[0].Level == LogLevel.Warning) ? LogEventSeverity.Warning : LogEventSeverity.Error;

			public LogEventData(IReadOnlyList<JsonLogEvent> lines)
			{
				Lines = lines;
			}

			string ILogEventData.Message
			{
				get
				{
					_message ??= String.Join("\n", Lines.Select(x => x.GetRenderedMessage().ToString()));
					return _message;
				}
			}
		}

		async Task<LogEventData> GetEventDataAsync(ILog log, int lineIndex, int lineCount, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(LogCollection)}.{nameof(GetEventDataAsync)}");
			span.SetAttribute("logId", log.Id.ToString());
			span.SetAttribute("lineIndex", lineIndex);
			span.SetAttribute("lineCount", lineCount);

			List<Utf8String> lines = await log.ReadLinesAsync(lineIndex, lineCount, cancellationToken);
			List<JsonLogEvent> jsonLines = new List<JsonLogEvent>(lines.Count);

			foreach (Utf8String line in lines)
			{
				try
				{
					jsonLines.Add(JsonLogEvent.Parse(line.Memory));
				}
				catch (JsonException ex)
				{
					_logger.LogWarning(ex, "Unable to parse line from log file: {Line}", line);
				}
			}

			return new LogEventData(jsonLines);
		}

		#endregion

		/// <summary>
		/// Streams log data to a caller
		/// </summary>
		class ResponseStream : Stream
		{
			readonly LogNode _rootNode;

			/// <summary>
			/// Starting offset within the file of the data to return 
			/// </summary>
			readonly long _responseOffset;

			/// <summary>
			/// Length of data to return
			/// </summary>
			readonly long _responseLength;

			/// <summary>
			/// Current offset within the stream
			/// </summary>
			long _currentOffset;

			/// <summary>
			/// The current chunk index
			/// </summary>
			int _chunkIdx;

			/// <summary>
			/// Buffer containing a message for missing data
			/// </summary>
			ReadOnlyMemory<byte> _sourceBuffer;

			/// <summary>
			/// Offset within the source buffer
			/// </summary>
			int _sourcePos;

			/// <summary>
			/// Length of the source buffer being copied from
			/// </summary>
			int _sourceEnd;

			/// <summary>
			/// Constructor
			/// </summary>
			public ResponseStream(LogNode rootNode, long offset, long length)
			{
				_rootNode = rootNode;

				_responseOffset = offset;
				_responseLength = length;

				_currentOffset = offset;

				_chunkIdx = rootNode.TextChunkRefs.GetChunkForOffset(offset);
				_sourceBuffer = null!;
			}

			/// <inheritdoc/>
			public override bool CanRead => true;

			/// <inheritdoc/>
			public override bool CanSeek => false;

			/// <inheritdoc/>
			public override bool CanWrite => false;

			/// <inheritdoc/>
			public override long Length => _responseLength;

			/// <inheritdoc/>
			public override long Position
			{
				get => _currentOffset - _responseOffset;
				set => throw new NotImplementedException();
			}

			/// <inheritdoc/>
			public override void Flush()
			{
			}

			/// <inheritdoc/>
			public override int Read(byte[] buffer, int offset, int count)
			{
#pragma warning disable VSTHRD002
				return ReadAsync(buffer, offset, count, CancellationToken.None).Result;
#pragma warning restore VSTHRD002
			}

			/// <inheritdoc/>
			public override async Task<int> ReadAsync(byte[] buffer, int offset, int length, CancellationToken cancellationToken)
			{
				return await ReadAsync(buffer.AsMemory(offset, length), cancellationToken);
			}

			/// <inheritdoc/>
			public override async ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken)
			{
				int readBytes = 0;
				while (readBytes < buffer.Length)
				{
					if (_sourcePos < _sourceEnd)
					{
						// Try to copy from the current buffer
						int blockSize = Math.Min(_sourceEnd - _sourcePos, buffer.Length - readBytes);
						_sourceBuffer.Slice(_sourcePos, blockSize).Span.CopyTo(buffer.Slice(readBytes).Span);
						_currentOffset += blockSize;
						readBytes += blockSize;
						_sourcePos += blockSize;
					}
					else if (_currentOffset < _responseOffset + _responseLength)
					{
						// Move to the right chunk
						while (_chunkIdx + 1 < _rootNode.TextChunkRefs.Count && _currentOffset >= _rootNode.TextChunkRefs[_chunkIdx + 1].Offset)
						{
							_chunkIdx++;
						}

						// Get the chunk data
						LogChunkRef chunk = _rootNode.TextChunkRefs[_chunkIdx];
						LogChunkNode chunkNode = await chunk.Target.ReadBlobAsync(cancellationToken: cancellationToken);

						// Get the source data
						_sourceBuffer = chunkNode.Data;
						_sourcePos = (int)(_currentOffset - chunk.Offset);
						_sourceEnd = (int)Math.Min(_sourceBuffer.Length, (_responseOffset + _responseLength) - chunk.Offset);
					}
					else
					{
						// End of the log
						break;
					}
				}
				return readBytes;
			}

			/// <inheritdoc/>
			public override long Seek(long offset, SeekOrigin origin) => throw new NotImplementedException();

			/// <inheritdoc/>
			public override void SetLength(long value) => throw new NotImplementedException();

			/// <inheritdoc/>
			public override void Write(byte[] buffer, int offset, int count) => throw new NotImplementedException();
		}
	}
}
