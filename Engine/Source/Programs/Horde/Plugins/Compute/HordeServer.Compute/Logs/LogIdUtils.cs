// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Logs;
using HordeServer.Utilities;

namespace HordeServer.Logs
{
	/// <summary>
	/// Server helper methods for <see cref="LogId"/>
	/// </summary>
	public static class LogIdUtils
	{
		/// <summary>
		/// Creates a new <see cref="LogId"/>
		/// </summary>
		public static LogId GenerateNewId() => new LogId(BinaryIdUtils.CreateNew());
	}
}
