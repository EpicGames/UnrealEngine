// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Logs
{
	/// <summary>
	/// Implements the Horde gRPC service for bots updating their status and dequeing work
	/// </summary>
	[Authorize]
	public class LogRpcService : LogRpc.LogRpcBase
	{
		// Exception message substrings for detecting closed gRPC response streams.
		// These come from Grpc.AspNetCore.Server (HttpContextServerCallContext) and ASP.NET Core's response body writer.
		// .NET Core/.NET 5+ exception messages are not localized (always English), so string matching is safe.
		const string RequestCompleteMessage = "Request is complete";
		const string WritingNotAllowedMessage = "Writing is not allowed";

		readonly ILogCollection _logCollection;
		readonly LogTailService _logTailService;
		readonly StorageService _storageService;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		readonly ILogger<LogRpcService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public LogRpcService(ILogCollection logCollection, LogTailService logTailService, StorageService storageService, IOptionsSnapshot<ComputeConfig> computeConfig, ILogger<LogRpcService> logger)
		{
			_logCollection = logCollection;
			_logTailService = logTailService;
			_storageService = storageService;
			_computeConfig = computeConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public override async Task<RpcUpdateLogResponse> UpdateLog(RpcUpdateLogRequest request, ServerCallContext context)
		{
			ILog? log = await _logCollection.GetAsync(LogId.Parse(request.LogId), context.CancellationToken);
			if (log == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Resource not found");
			}
			if (!log.AuthorizeForSession(context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Access denied");
			}

			IStorageNamespace store = _storageService.GetNamespace(Namespace.Logs);

			_logger.LogInformation("Updating {LogId} to node {RefTarget} (lines: {LineCount}, complete: {Complete})", request.LogId, request.TargetLocator, request.LineCount, request.Complete);

			IoHash hash;
			if (!IoHash.TryParse(request.TargetHash, out hash))
			{
				hash = IoHash.Zero;
			}

			IHashedBlobRef target = store.CreateBlobRef(hash, new BlobLocator(request.TargetLocator));
			await store.AddRefAsync(new RefName(request.LogId), target);

			await log.UpdateLineCountAsync(request.LineCount, request.Complete, context.CancellationToken);

			await _logTailService.FlushAsync(log.Id, request.LineCount);

			return new RpcUpdateLogResponse();
		}

		/// <inheritdoc/>
		public override async Task UpdateLogTail(IAsyncStreamReader<RpcUpdateLogTailRequest> requestStream, IServerStreamWriter<RpcUpdateLogTailResponse> responseStream, ServerCallContext context)
		{
			RpcUpdateLogTailResponse response = new () { TailNext = -1 };
			
			Task<bool> moveNextTask = requestStream.MoveNext();
			while (await moveNextTask)
			{
				RpcUpdateLogTailRequest request = requestStream.Current;
				LogId logId = LogId.Parse(request.LogId);
				_logger.LogDebug("Updating log tail for {LogId}: line {LineIdx}, size {Size}", logId, request.TailNext, request.TailData.Length);

				if (request.TailData.Length > 0)
				{
					await _logTailService.AppendAsync(logId, request.TailNext, request.TailData.Memory);
				}

				moveNextTask = requestStream.MoveNext();

				response.TailNext = await _logTailService.GetTailNextAsync(logId, context.CancellationToken);
				if (response.TailNext == -1)
				{
					using CancellationTokenSource cancellationSource = new ();
					_logger.LogDebug("Waiting for tail next on log {LogId}", logId);
					Task<int> waitTask = _logTailService.WaitForTailNextAsync(logId, cancellationSource.Token);
					Task completeTask = await Task.WhenAny(waitTask, moveNextTask);
					await cancellationSource.CancelAsync();

					try
					{
						response.TailNext = await waitTask;
						await completeTask; // Allow moveNextTask to throw if it finished due to cancellation, so we don't write to broken stream
					}
					catch (OperationCanceledException)
					{
						_logger.LogDebug("Log tail stream cancelled by client");
						break;
					}
					catch (IOException ex)
					{
						// Connection was aborted (client deadline exceeded, client process killed, etc.)
						_logger.LogDebug(ex, "Log tail stream connection aborted for {LogId}", logId);
						break;
					}
					catch (Exception ex)
					{
						_logger.LogWarning(ex, "Exception while waiting for tail next");
						break; // Don't continue on unknown errors - stream may be broken
					}
				}

				_logger.LogDebug("Return tail next for log {LogId} = {TailNext}", logId, response.TailNext);

				// Guard the write against connection closure
				if (!await TryWriteResponseAsync(responseStream, response))
				{
					break;
				}
			}

			// Don't write after loop - stream is closed when we exit the while loop
		}

		/// <summary>
		/// Attempts to write a response to the stream, returning false if the stream is already closed
		/// </summary>
		static async Task<bool> TryWriteResponseAsync(IServerStreamWriter<RpcUpdateLogTailResponse> responseStream, RpcUpdateLogTailResponse response)
		{
			try
			{
				await responseStream.WriteAsync(response);
				return true;
			}
			catch (InvalidOperationException ex) when (ex.Message.Contains(RequestCompleteMessage, StringComparison.OrdinalIgnoreCase) || ex.Message.Contains(WritingNotAllowedMessage, StringComparison.OrdinalIgnoreCase))
			{
				// Stream already closed by client - expected during shutdown
				return false;
			}
			catch (IOException)
			{
				// Connection aborted
				return false;
			}
		}

		/// <inheritdoc/>
		public override async Task<Empty> CreateLogEvents(RpcCreateLogEventsRequest request, ServerCallContext context)
		{
			if (!_computeConfig.Value.Authorize(LogAclAction.CreateEvent, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Access denied");
			}

			foreach (IGrouping<string, RpcCreateLogEventRequest> createEventGroup in request.Events.GroupBy(x => x.LogId))
			{
				ILog? log = await _logCollection.GetAsync(LogId.Parse(createEventGroup.Key), context.CancellationToken);
				if (log == null)
				{
					throw new StructuredRpcException(StatusCode.NotFound, "Log not found");
				}

				List<NewLogEventData> newEvents = new List<NewLogEventData>();
				foreach (RpcCreateLogEventRequest createEvent in createEventGroup)
				{
					NewLogEventData newEvent = new NewLogEventData();
					newEvent.Severity = (LogEventSeverity)createEvent.Severity;
					newEvent.LineIndex = createEvent.LineIndex;
					newEvent.LineCount = createEvent.LineCount;
					newEvents.Add(newEvent);
				}

				await log.AddEventsAsync(newEvents, context.CancellationToken);
			}

			return new Empty();
		}
	}
}
