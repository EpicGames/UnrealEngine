// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Globalization;
using EpicGames.Core;
using EpicGames.Horde.Logs;
using EpicGames.Redis;
using HordeServer.Server;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Logs
{
	/// <summary>
	/// Stores data for logs which has not yet been flushed to persistent storage.
	/// 
	/// * Once a log is retrieved, server adds an entry with this log id to a sorted set in Redis (see <see cref="s_expireQueue"/>, scored by 
	///   expiry time (defaulting to <see cref="ExpireAfter"/> from present) and broadcasts it to any other server instances.
	/// * If a log is being tailed, server keeps the total number of lines in a Redis key (<see cref="TailNextKey"/>, and appends chunks of log 
	///   data to a sorted set for that log scored by index of the first line (see <see cref="TailDataKey"/>).
	/// * Chunks are split on fixed boundaries, in order to allow older entries to be purged by score without having to count the number of lines
	///   they contain first.
	/// * Agent polls for requests to provide tail data via <see cref="LogRpcService.UpdateLogTail"/>, which calls <see cref="WaitForTailNextAsync"/>.
	///   <see cref="WaitForTailNextAsync"/> returns the index of the total line count for this log if it is being tailed (indicating the index of
	///   the next line that the agent should send to the server), or blocks until the log is being tailed.
	/// * Once data is flushed to persistent storage, the number of flushed lines is added to <see cref="s_trimQueue"/> and the line data is removed
	///   from Redis after <see cref="TrimAfter"/>.
	/// * <see cref="EpicGames.Horde.Logs.LogNode"/> contains a "complete" flag indicating whether it is necessary to check for tail data.
	///   
	/// </summary>
	public class LogTailService : IHostedService
	{
		/// <summary>
		/// Amount of time to keep tail data in the cache after it has been flushed to persistent storage. This gives some wiggle room with order of operations when sequencing persisted data with tail data.
		/// </summary>
		public static TimeSpan TrimAfter { get; } = TimeSpan.FromMinutes(0.5);

		/// <summary>
		/// Amount of time to continue tailing a log after the last time it was updated.
		/// </summary>
		public static TimeSpan ExpireAfter { get; } = TimeSpan.FromMinutes(2.0);

		readonly IRedisService _redisService;

		readonly ConcurrentDictionary<LogId, AsyncEvent> _notifyLogEvents = new ConcurrentDictionary<LogId, AsyncEvent>();
		readonly RedisChannel<LogId> _tailStartChannel;
		static readonly RedisSortedSetKey<string> s_trimQueue = "logs:trim";
		static readonly RedisSortedSetKey<LogId> s_expireQueue = "logs:expire";

		readonly IClock _clock;
		readonly ITicker _expireTailsTicker;

		readonly ILogger _logger;
		readonly IServerInfo _serverInfo;

		IAsyncDisposable? _tailStartSubscription;

		internal static RedisStringKey<int> TailNextKey(LogId logId) => new RedisStringKey<int>($"logs:{logId}:tail-next");
		internal static RedisSortedSetKey<ReadOnlyMemory<byte>> TailDataKey(LogId logId) => new RedisSortedSetKey<ReadOnlyMemory<byte>>($"logs:{logId}:tail-data");

		/// <summary>
		/// Boundary to split Redis chunks along. 
		/// </summary>
		public int ChunkLineCount { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public LogTailService(IRedisService redisService, IClock clock, IServerInfo serverInfo, ILogger<LogTailService> logger)
			: this(redisService, clock, 32, serverInfo, logger)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public LogTailService(IRedisService redisService, IClock clock, int chunkLineCount, IServerInfo serverInfo, ILogger<LogTailService> logger)
		{
			if ((chunkLineCount & (chunkLineCount - 1)) != 0)
			{
				throw new ArgumentException($"{nameof(ChunkLineCount)} must be a power of two", nameof(chunkLineCount));
			}

			ChunkLineCount = chunkLineCount;

			_redisService = redisService;
			_tailStartChannel = new RedisChannel<LogId>(RedisChannel.Literal("logs:notify"));

			_clock = clock;
			_expireTailsTicker = clock.AddSharedTicker<LogTailService>(TimeSpan.FromSeconds(30.0), ExpireTailsAsync, logger);
			_serverInfo = serverInfo;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_tailStartSubscription = await _redisService.SubscribeAsync(_tailStartChannel, OnTailStart);
			if (_serverInfo.IsRunModeActive(RunMode.Worker))
			{
				await _expireTailsTicker.StartAsync();
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			if (_serverInfo.IsRunModeActive(RunMode.Worker))
			{
				await _expireTailsTicker.StopAsync();
			}
			if (_tailStartSubscription != null)
			{
				await _tailStartSubscription.DisposeAsync();
			}
		}

		private async ValueTask ExpireTailsAsync(CancellationToken cancellationToken)
		{
			double maxScore = (_clock.UtcNow - DateTime.UnixEpoch).TotalSeconds;

			SortedSetEntry<string>[] trimEntries = await _redisService.GetDatabase().SortedSetRangeByScoreWithScoresAsync(s_trimQueue, stop: maxScore);
			if (trimEntries.Length > 0)
			{
				foreach (SortedSetEntry<string> trimEntry in trimEntries)
				{
					string[] values = trimEntry.Element.Split('=');

					LogId logId = LogId.Parse(values[0]);
					int lineCount = Int32.Parse(values[1], NumberStyles.None, CultureInfo.InvariantCulture);

					int minLineIndex = (lineCount & ~(ChunkLineCount - 1)) - 1;
					_ = await _redisService.GetDatabase().SortedSetRemoveRangeByScoreAsync(TailDataKey(logId), Double.NegativeInfinity, minLineIndex, Exclude.None, CommandFlags.FireAndForget);

					_logger.LogDebug("Removed tail data for log {LogId} at line {LineCount}", logId, lineCount);
				}
				_ = await _redisService.GetDatabase().SortedSetRemoveRangeByScoreAsync(s_trimQueue, Double.NegativeInfinity, maxScore, flags: CommandFlags.FireAndForget);
			}

			SortedSetEntry<LogId>[] expireEntries = await _redisService.GetDatabase().SortedSetRangeByScoreWithScoresAsync(s_expireQueue, stop: maxScore);
			if (expireEntries.Length > 0)
			{
				foreach (SortedSetEntry<LogId> expireEntry in expireEntries)
				{
					LogId logId = expireEntry.Element;

					ITransaction transaction = _redisService.GetDatabase().CreateTransaction();
					transaction.AddCondition(Condition.SortedSetEqual(s_expireQueue.Inner, expireEntry.ElementValue, expireEntry.Score));
					_ = transaction.KeyDeleteAsync(TailNextKey(logId), CommandFlags.FireAndForget);
					_ = transaction.KeyDeleteAsync(TailDataKey(logId), CommandFlags.FireAndForget);
					await transaction.ExecuteAsync();

					_logger.LogDebug("Expired tailing request for log {LogId}", logId);
				}
				await _redisService.GetDatabase().SortedSetRemoveRangeByScoreAsync(s_expireQueue, Double.NegativeInfinity, maxScore);

				long tailCount = await _redisService.GetDatabase().SortedSetLengthAsync(s_expireQueue);
				_logger.LogDebug("Currently tailing {NumLogs} logs", tailCount);
			}
		}

		private void OnTailStart(LogId logId)
		{
			if (_notifyLogEvents.TryGetValue(logId, out AsyncEvent? notifyEvent))
			{
				_logger.LogDebug("Latched notification for tail start on log {LogId}", logId);
				notifyEvent.Latch();
			}
		}

		/// <summary>
		/// Enable tailing for the given log file
		/// </summary>
		/// <param name="logId">Log file identifier</param>
		/// <param name="lineCount">The current flushed line count for the log file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask EnableTailingAsync(LogId logId, int lineCount, CancellationToken cancellationToken = default)
		{
			bool waitForTailData = false;
			if (await _redisService.GetDatabase().StringSetAsync(TailNextKey(logId), lineCount, when: When.NotExists))
			{
				_logger.LogDebug("Enabled tailing for log {LogId}", logId);
				waitForTailData = true;
			}

			await _redisService.GetDatabase().PublishAsync(_tailStartChannel, logId);

			double score = (_clock.UtcNow + ExpireAfter - DateTime.UnixEpoch).TotalSeconds;
			_ = _redisService.GetDatabase().SortedSetAddAsync(s_expireQueue, logId, score, flags: CommandFlags.FireAndForget);

			if (waitForTailData)
			{
				_logger.LogDebug("Waiting for tail data on log {LogId}", logId);

				// Bit hacky; poll for a while until we get some tail data through
				RedisSortedSetKey<ReadOnlyMemory<byte>> tailDataKey = TailDataKey(logId);
				for (int idx = 0; idx < 20; idx++)
				{
					if (await _redisService.GetDatabase().SortedSetLengthAsync(tailDataKey) > 0)
					{
						break;
					}
					else
					{
						await Task.Delay(100, cancellationToken);
					}
				}

				_logger.LogDebug("Redis key for tail data on {LogId} is set", logId);
			}
		}

		/// <summary>
		/// Appends data to the log tail, starting at the given line index
		/// </summary>
		/// <param name="logId">Log file identifier</param>
		/// <param name="tailNext">Index of the first line to append</param>
		/// <param name="tailData">Data for the log starting at the given line number</param>
		/// <returns></returns>
		public async Task AppendAsync(LogId logId, int tailNext, ReadOnlyMemory<byte> tailData)
		{
			List<SortedSetEntry<ReadOnlyMemory<byte>>> entries = new List<SortedSetEntry<ReadOnlyMemory<byte>>>();
			int newLineCount = SplitLogDataToChunks(tailNext, tailData, ChunkLineCount, entries);
			_logger.LogDebug("New line count = {NewLineCount}, entries = {EntryCount}", newLineCount, entries.Count);

			if (entries.Count > 0)
			{
				RedisStringKey<int> redisTailNext = TailNextKey(logId);
				RedisSortedSetKey<ReadOnlyMemory<byte>> redisTailData = TailDataKey(logId);

				ITransaction transaction = _redisService.GetDatabase().CreateTransaction();
				transaction.AddCondition(Condition.StringEqual(TailNextKey(logId).Inner, RedisSerializer.Serialize(tailNext)));
				_ = transaction.SortedSetAddAsync(redisTailData, entries.ToArray(), flags: CommandFlags.FireAndForget);
				_ = transaction.StringSetAsync(redisTailNext, newLineCount, flags: CommandFlags.FireAndForget);

				bool result = await transaction.ExecuteAsync();
				_logger.LogDebug("Tail data for {LogId}, line {TailNext} -> {NewTailNext}, result: {Result}", logId, tailNext, newLineCount, result);
			}
		}

		/// <summary>
		/// Reads data for the given log file, starting at the requested line index
		/// </summary>
		/// <param name="logId">The log to query</param>
		/// <param name="lineIndex">Index of the first line to return</param>
		/// <param name="lineCount">Maximum number of lines to return</param>
		/// <returns>List of lines for the file</returns>
		public async Task<List<Utf8String>> ReadAsync(LogId logId, int lineIndex, int lineCount)
		{
			List<Utf8String> lines = new List<Utf8String>();
			await ReadAsync(logId, lineIndex, lineCount, lines);
			return lines;
		}

		/// <summary>
		/// Reads data for the given log file, starting at the requested line index
		/// </summary>
		/// <param name="logId">The log to query</param>
		/// <param name="lineIndex">Index of the first line to return</param>
		/// <param name="lineCount">Maximum number of lines to return</param>
		/// <param name="lines">Buffer for the lines to be returned</param>
		/// <returns>List of lines for the file</returns>
		public async Task ReadAsync(LogId logId, int lineIndex, int lineCount, List<Utf8String> lines)
		{
			int lowerBound = lineIndex;
			int upperBound = lineIndex + lineCount;
			int chunkLowerBound = lowerBound & ~(ChunkLineCount - 1);
			int chunkUpperBound = (upperBound + (ChunkLineCount - 1)) & ~(ChunkLineCount - 1);

			int nextIndex = lineIndex;

			SortedSetEntry<ReadOnlyMemory<byte>>[] entries = await _redisService.GetDatabase().SortedSetRangeByScoreWithScoresAsync(TailDataKey(logId), chunkLowerBound, chunkUpperBound);
			foreach (SortedSetEntry<ReadOnlyMemory<byte>> entry in entries)
			{
				ReadOnlyMemory<byte> memory = entry.Element;

				int index = (int)entry.Score;
				if (index > nextIndex)
				{
					break;
				}
				for (; ; index++)
				{
					int lineLength = memory.Span.IndexOf((byte)'\n');
					if (lineLength == -1)
					{
						break;
					}
					if (index >= nextIndex)
					{
						lines.Add(new Utf8String(memory.Slice(0, lineLength)));
						nextIndex++;
					}
					memory = memory.Slice(lineLength + 1);
				}
			}
		}

		/// <summary>
		/// Flushes the given log file to the given number of lines
		/// </summary>
		/// <param name="logId">Log to flush</param>
		/// <param name="lineCount">Number of lines in the flushed log file</param>
		/// <returns></returns>
		public async ValueTask FlushAsync(LogId logId, int lineCount)
		{
			string trimEntry = $"{logId}={lineCount}";
			double trimTime = (_clock.UtcNow + TrimAfter - DateTime.UnixEpoch).TotalSeconds;

			ITransaction transaction = _redisService.GetDatabase().CreateTransaction();
			transaction.AddCondition(Condition.KeyExists(TailDataKey(logId).Inner));
			_ = transaction.SortedSetAddAsync(s_trimQueue, trimEntry, trimTime, flags: CommandFlags.FireAndForget);
			await transaction.ExecuteAsync(CommandFlags.FireAndForget);
		}

		/// <summary>
		/// Gets the total number of lines in a log
		/// </summary>
		/// <param name="logId">Log to query</param>
		/// <param name="flushedLineCount">Number of flushed lines</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Total number of lines in the file</returns>
		public async Task<int> GetFullLineCountAsync(LogId logId, int flushedLineCount, CancellationToken cancellationToken)
		{
			RedisStringKey<int> tailNextKey = TailNextKey(logId);

			int lineCount = await _redisService.GetDatabase().StringGetAsync(tailNextKey, -1);
			if (lineCount == -1)
			{
				await EnableTailingAsync(logId, flushedLineCount, cancellationToken);
			}

			return Math.Max(lineCount, flushedLineCount);
		}

		/// <summary>
		/// Gets the next requested tail line number, returning immediately if it is set
		/// </summary>
		/// <param name="logId">The log file to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Next requested line number, or -1 if tailing is not enabled.</returns>
		public async Task<int> GetTailNextAsync(LogId logId, CancellationToken cancellationToken)
		{
			_ = cancellationToken;
			RedisStringKey<int> tailNextKey = TailNextKey(logId);
			return await _redisService.GetDatabase().StringGetAsync(tailNextKey, -1);
		}

		/// <summary>
		/// Waits for a request to read tail data
		/// </summary>
		/// <param name="logId">The log file to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<int> WaitForTailNextAsync(LogId logId, CancellationToken cancellationToken)
		{
			AsyncEvent notifyEvent = _notifyLogEvents.GetOrAdd(logId, new AsyncEvent());
			try
			{
				using (IDisposable registration = cancellationToken.Register(() => notifyEvent.Latch()))
				{
					for (; ; )
					{
						Task notifyTask = notifyEvent.Task;

						int tailNext = await _redisService.GetDatabase().StringGetAsync(TailNextKey(logId), -1);
						if (tailNext != -1)
						{
							_logger.LogDebug("Released wait for tail on log {LogId} from line {Line}", logId, tailNext);
							return tailNext;
						}
						else if (cancellationToken.IsCancellationRequested)
						{
							return -1;
						}

						await notifyTask;
					}
				}
			}
			finally
			{
				_notifyLogEvents.TryRemove(logId, out _);
			}
		}

		/// <summary>
		/// Splits data into chunks aligned onto <see cref="ChunkLineCount"/> line boundaries.
		/// </summary>
		static int SplitLogDataToChunks(int lineIdx, ReadOnlyMemory<byte> data, int chunkLineCount, List<SortedSetEntry<ReadOnlyMemory<byte>>> entries)
		{
			for (; ; )
			{
				int minLineIdx = lineIdx;

				// Find the next chunk
				int length = 0;
				for (; ; )
				{
					// Add the next line to the buffer
					int lineLength = data.Span.Slice(length).IndexOf((byte)'\n') + 1;
					if (lineLength == 0)
					{
						break;
					}
					length += lineLength;

					// Move to the next line
					lineIdx++;

					// If this is a chunk boundary, break here
					if ((lineIdx & (chunkLineCount - 1)) == 0)
					{
						break;
					}
				}

				// If we couldn't find any more lines, bail out
				if (length == 0)
				{
					break;
				}

				// Add this entry
				ReadOnlyMemory<byte> chunk = data.Slice(0, length);
				entries.Add(new SortedSetEntry<ReadOnlyMemory<byte>>(chunk, minLineIdx));
				data = data.Slice(length);
			}
			return lineIdx;
		}
	}
}
