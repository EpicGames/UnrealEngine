// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using HordeServer.Agents;
using HordeServer.Agents.Sessions;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;

namespace HordeServer.Logs
{
	/// <summary>
	/// Format for the returned data
	/// </summary>
	public enum LogOutputFormat
	{
		/// <summary>
		/// Plain text
		/// </summary>
		Text,

		/// <summary>
		/// Raw output (text/json)
		/// </summary>
		Raw,
	}

	/// <summary>
	/// Controller for the /api/logs endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class LogsController : ControllerBase
	{
		private readonly ILogCollection _logCollection;
		private readonly StorageService _storageService;
		private readonly IEnumerable<ILogExtAuthProvider> _authProviders;
		private readonly IEnumerable<ILogExtIssueProvider> _issueProviders;
		private readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public LogsController(ILogCollection logCollection, StorageService storageService, IEnumerable<ILogExtAuthProvider> authProviders, IEnumerable<ILogExtIssueProvider> issueProviders, IOptionsSnapshot<ComputeConfig> computeConfig, ILogger<LogsController> logger)
		{
			_logCollection = logCollection;
			_storageService = storageService;
			_authProviders = authProviders;
			_issueProviders = issueProviders;
			_computeConfig = computeConfig;
			_logger = logger;
		}

		/// <summary>
		/// Retrieve metadata about a specific log file
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/logs/{logId}")]
		[ProducesResponseType(typeof(GetLogResponse), 200)]
		public async Task<ActionResult<object>> GetLogAsync(LogId logId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
			{
				return Forbid();
			}

			LogMetadata metadata = await log.GetMetadataAsync(cancellationToken);
			return CreateGetLogResponse(log, metadata).ApplyFilter(filter);
		}

		static GetLogResponse CreateGetLogResponse(ILog log, LogMetadata metadata)
		{
			GetLogResponse response = new GetLogResponse();
			response.Id = log.Id;
			response.JobId = log.JobId;
			response.LeaseId = log.LeaseId;
			response.SessionId = log.SessionId;
			response.Type = log.Type;
			response.LineCount = metadata.MaxLineIndex;
			return response;
		}

		/// <summary>
		/// Uploads a blob for a log file. See /api/v1/storage/XXX/blobs.
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="request">Request for the upload</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Information about the requested project</returns>
		[HttpPost]
		[Route("/api/v1/logs/{logId}/blobs")]
		[ProducesResponseType(typeof(WriteBlobResponse), 200)]
		public async Task<ActionResult<WriteBlobResponse>> WriteLogBlobAsync(LogId logId, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.WriteLogData, User, cancellationToken))
			{
				return Forbid();
			}
			if (!String.IsNullOrEmpty(request.Prefix))
			{
				return BadRequest("Cannot specify prefix for logs");
			}

			request.Prefix = $"{log.RefName}";

			IStorageBackend storageBackend = _storageService.CreateBackend(log.NamespaceId);
			return await StorageController.WriteBlobAsync(storageBackend, null, request, cancellationToken);
		}

		/// <summary>
		/// Uploads a blob for a log file. See /api/v1/storage/XXX/blobs.
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="locator">Locator for the new blob</param>
		/// <param name="request">Request for the upload</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Information about the requested project</returns>
		[HttpPut]
		[Route("/api/v1/logs/{logId}/blobs/{*locator}")]
		[ProducesResponseType(typeof(WriteBlobResponse), 200)]
		public async Task<ActionResult<WriteBlobResponse>> WriteLogBlobAsync(LogId logId, BlobLocator locator, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.WriteLogData, User, cancellationToken))
			{
				return Forbid();
			}
			if (!locator.ToString().StartsWith($"{logId}/", StringComparison.Ordinal))
			{
				return BadRequest($"Log locator must start with log id");
			}

			IStorageBackend storageBackend = _storageService.CreateBackend(log.NamespaceId);
			return await StorageController.WriteBlobAsync(storageBackend, locator, request, cancellationToken);
		}

		/// <summary>
		/// Retrieve raw data for a log file
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="format">Format for the returned data</param>
		/// <param name="fileName">Name of the default filename to download</param>
		/// <param name="download">Whether to download the file rather than display in the browser</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Raw log data for the requested range</returns>
		[HttpGet]
		[Route("/api/v1/logs/{logId}/data")]
		public async Task<ActionResult> GetLogDataAsync(
			LogId logId,
			[FromQuery] LogOutputFormat format = LogOutputFormat.Raw,
			[FromQuery] string? fileName = null,
			[FromQuery] bool download = false,
			CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
			{
				return Forbid();
			}

			Func<Stream, ActionContext, Task> copyTask;
			if (format == LogOutputFormat.Text && log.Type == LogType.Json)
			{
				copyTask = (outputStream, context) => log.CopyPlainTextStreamAsync(outputStream, cancellationToken);
			}
			else
			{
				copyTask = (outputStream, context) => log.CopyRawStreamAsync(outputStream, cancellationToken);
			}

			return new CustomFileCallbackResult(fileName ?? $"log-{logId}.txt", "text/plain", !download, copyTask);
		}

		/// <summary>
		/// Retrieve line data for a log
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="index">Index of the first line to retrieve</param>
		/// <param name="count">Number of lines to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/logs/{logId}/lines")]
		public async Task<ActionResult> GetLogLinesAsync(LogId logId, [FromQuery] int index = 0, [FromQuery] int count = 100, CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
			{
				return Forbid();
			}

			LogMetadata metadata = await log.GetMetadataAsync(cancellationToken);

			List<Utf8String> lines = await log.ReadLinesAsync(index, count, cancellationToken);
			using (MemoryStream stream = new MemoryStream(lines.Sum(x => x.Length) + (lines.Count * 20)))
			{
				stream.WriteByte((byte)'{');

				stream.Write(Encoding.UTF8.GetBytes($"\"index\":{index},"));
				stream.Write(Encoding.UTF8.GetBytes($"\"count\":{lines.Count},"));
				stream.Write(Encoding.UTF8.GetBytes($"\"maxLineIndex\":{Math.Max(metadata.MaxLineIndex, index + lines.Count)},"));
				stream.Write(Encoding.UTF8.GetBytes($"\"format\":{(log.Type == LogType.Json ? "\"JSON\"" : "\"TEXT\"")},"));

				stream.Write(Encoding.UTF8.GetBytes($"\"lines\":["));
				stream.WriteByte((byte)'\n');

				for (int lineIdx = 0; lineIdx < lines.Count; lineIdx++)
				{
					Utf8String line = lines[lineIdx];

					stream.WriteByte((byte)' ');
					stream.WriteByte((byte)' ');

					if (log.Type == LogType.Json)
					{
						await stream.WriteAsync(line.Memory, cancellationToken);
					}
					else
					{
						stream.WriteByte((byte)'\"');
						for (int idx = 0; idx < line.Length; idx++)
						{
							byte character = line[idx];
							if (character >= 32 && character <= 126 && character != '\\' && character != '\"')
							{
								stream.WriteByte(character);
							}
							else
							{
								stream.Write(Encoding.UTF8.GetBytes($"\\x{character:x2}"));
							}
						}
						stream.WriteByte((byte)'\"');
					}

					if (lineIdx + 1 < lines.Count)
					{
						stream.WriteByte((byte)',');
					}

					stream.WriteByte((byte)'\n');
				}

				if (log.Type == LogType.Json)
				{
					stream.Write(Encoding.UTF8.GetBytes($"]"));
				}

				stream.WriteByte((byte)'}');

				Response.ContentType = "application/json";
				Response.Headers.ContentLength = stream.Length;
				stream.Position = 0;
				await stream.CopyToAsync(Response.Body, cancellationToken);
			}
			return new EmptyResult();
		}

		/// <summary>
		/// Search log data
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="text">Text to search for</param>
		/// <param name="firstLine">First line to search from</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Raw log data for the requested range</returns>
		[HttpGet]
		[Route("/api/v1/logs/{logId}/search")]
		public async Task<ActionResult<SearchLogResponse>> SearchLogAsync(
			LogId logId,
			[FromQuery] string text,
			[FromQuery] int firstLine = 0,
			[FromQuery] int count = 5,
			CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
			{
				return Forbid();
			}

			SearchLogResponse response = new SearchLogResponse();
			response.Stats = new SearchStats();
			response.Lines = await log.SearchLogDataAsync(text, firstLine, count, response.Stats, cancellationToken);
			return response;
		}

		/// <summary>
		/// Retrieve events for a log
		/// </summary>
		/// <param name="logId">Id of the log file to get information about</param>
		/// <param name="index">Index of the first line to retrieve</param>
		/// <param name="count">Number of lines to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/logs/{logId}/events")]
		[ProducesResponseType(typeof(List<GetLogEventResponse>), 200)]
		public async Task<ActionResult<List<GetLogEventResponse>>> GetEventsAsync(LogId logId, [FromQuery] int? index = null, [FromQuery] int? count = null, CancellationToken cancellationToken = default)
		{
			ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
			if (log == null)
			{
				return NotFound();
			}
			if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
			{
				return Forbid();
			}

			List<ILogAnchor> anchors = await log.GetAnchorsAsync(null, index, count, cancellationToken);

			Dictionary<ObjectId, int?> spanIdToIssueId = new Dictionary<ObjectId, int?>();

			List<GetLogEventResponse> responses = new List<GetLogEventResponse>();
			foreach (ILogAnchor anchor in anchors)
			{
				ILogEventData logEventData;
				try
				{
					logEventData = await anchor.GetDataAsync(cancellationToken);
				}
				catch (StorageException ex)
				{
					_logger.LogWarning(ex, "Unable to read log event data for log {LogId} at line {LineIndex} (data may have been garbage collected)", logId, anchor.LineIndex);
					continue;
				}

				int? issueId = null;
				if (anchor.SpanId != null && !spanIdToIssueId.TryGetValue(anchor.SpanId.Value, out issueId))
				{
					foreach (ILogExtIssueProvider issueProvider in _issueProviders)
					{
						issueId ??= await issueProvider.GetIssueIdAsync(anchor.SpanId.Value, cancellationToken);
					}
					spanIdToIssueId[anchor.SpanId.Value] = issueId;
				}

				responses.Add(CreateGetLogEventResponse(anchor, logEventData, issueId));
			}
			return responses;
		}

		/// <summary>
		/// Retrieve events for multiple logs in a single request
		/// </summary>
		/// <param name="request">Request containing log ids to fetch events for</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Events grouped by log id</returns>
		[HttpPost]
		[Route("/api/v1/logs/events")]
		[ProducesResponseType(typeof(GetBatchLogEventsResponse), 200)]
		public async Task<ActionResult<GetBatchLogEventsResponse>> GetBatchEventsAsync([FromBody] GetBatchLogEventsRequest request, CancellationToken cancellationToken = default)
		{
			GetBatchLogEventsResponse response = new GetBatchLogEventsResponse();

			foreach (string logIdStr in request.LogIds)
			{
				LogId logId = LogId.Parse(logIdStr);
				ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
				if (log == null)
				{
					continue;
				}
				if (!await AuthorizeAsync(log, LogAclAction.ViewLog, User, cancellationToken))
				{
					continue;
				}

				List<ILogAnchor> anchors = await log.GetAnchorsAsync(null, null, null, cancellationToken);

				IEnumerable<ILogAnchor> matchingAnchors = anchors;
				if (request.Severity != null)
				{
					matchingAnchors = anchors.Where(a => a.Severity == request.Severity.Value);
				}

				List<ILogAnchor> matchingList = matchingAnchors.ToList();
				int total = matchingList.Count;

				if (total == 0)
				{
					continue;
				}

				IEnumerable<ILogAnchor> anchorsToFetch = request.Count != null ? matchingList.Take(request.Count.Value) : matchingList;

				Dictionary<ObjectId, int?> spanIdToIssueId = new Dictionary<ObjectId, int?>();
				List<GetLogEventResponse> events = new List<GetLogEventResponse>();

				foreach (ILogAnchor anchor in anchorsToFetch)
				{
					ILogEventData logEventData;
					try
					{
						logEventData = await anchor.GetDataAsync(cancellationToken);
					}
					catch (StorageException ex)
					{
						_logger.LogWarning(ex, "Unable to read log event data for log {LogId} at line {LineIndex} (data may have been garbage collected)", logId, anchor.LineIndex);
						continue;
					}

					int? issueId = null;
					if (anchor.SpanId != null && !spanIdToIssueId.TryGetValue(anchor.SpanId.Value, out issueId))
					{
						foreach (ILogExtIssueProvider issueProvider in _issueProviders)
						{
							issueId ??= await issueProvider.GetIssueIdAsync(anchor.SpanId.Value, cancellationToken);
						}
						spanIdToIssueId[anchor.SpanId.Value] = issueId;
					}

					events.Add(CreateGetLogEventResponse(anchor, logEventData, issueId));
				}

				response.LogEvents[logIdStr] = new BatchLogEventsEntry { Total = total, Events = events };
			}

			return response;
		}

		/// <summary>
		/// Create a log event response message
		/// </summary>
		/// <param name="anchor">The event to construct from</param>
		/// <param name="eventData">The event data</param>
		/// <param name="issueId">The issue for this event</param>
		public static GetLogEventResponse CreateGetLogEventResponse(ILogAnchor anchor, ILogEventData eventData, int? issueId)
		{
			GetLogEventResponse response = new GetLogEventResponse();
			response.Severity = anchor.Severity;
			response.LogId = anchor.LogId;
			response.LineIndex = anchor.LineIndex;
			response.LineCount = anchor.LineCount;
			response.IssueId = issueId;
			response.Lines.AddRange(eventData.Lines.Select(x => JsonDocument.Parse(x.Data).RootElement));
			return response;
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular template
		/// </summary>
		/// <param name="log">The template to check</param>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to authorize</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the action is authorized</returns>
		async Task<bool> AuthorizeAsync(ILog log, AclAction action, ClaimsPrincipal user, CancellationToken cancellationToken)
		{
			ComputeConfig computeConfig = _computeConfig.Value;
			if (user.HasAdminClaim())
			{
				return true;
			}
			if (log.LeaseId != null && user.HasLeaseClaim(log.LeaseId.Value))
			{
				return true;
			}
			if (log.SessionId != null && log.AuthorizeForSession(user))
			{
				return true;
			}
			if (action == LogAclAction.ViewLog && log.SessionId != null && computeConfig.Authorize(SessionAclAction.ViewSession, user))
			{
				return true;
			}
			foreach (ILogExtAuthProvider authProvider in _authProviders)
			{
				if (await authProvider.AuthorizeAsync(log, action, user, cancellationToken))
				{
					return true;
				}
			}
			if (computeConfig.Authorize(action, user))
			{
				return true;
			}
			return false;
		}
	}
}
