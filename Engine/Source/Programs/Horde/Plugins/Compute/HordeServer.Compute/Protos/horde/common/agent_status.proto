// Copyright Epic Games, Inc. All Rights Reserved.

//
// Derived from the Remote Workers API:
// https://github.com/googleapis/googleapis/blob/master/google/devtools/remoteworkers/v1test2/worker.proto
//

// Copyright 2019 Google LLC.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

syntax = "proto3";

package Horde.AgentStatusType;

option csharp_namespace = "HordeCommon.Rpc";

enum RpcAgentStatus {

	// Unspecified state.
	Unspecified = 0;

	// Agent is running normally.
	Ok = 1;

	// Agent is currently shutting down, and should not be assigned new leases.
	Stopping = 2;

	// Agent is in an unhealthy state and should not be assigned new leases.
	Unhealthy = 3;

	// Agent is currently stopped.
	Stopped = 4;

	// Agent is busy performing other work (eg. serving an interactive user)
	Busy = 5;
}
