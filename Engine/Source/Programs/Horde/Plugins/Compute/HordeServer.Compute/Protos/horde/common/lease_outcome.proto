// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

package Horde.LeaseOutcomeType;

option csharp_namespace = "HordeCommon.Rpc";

// Note: This enum mirrors the LeaseOutcome enum in C# code
enum RpcLeaseOutcome {

	// Default value.
	Unspecified = 0;

	// The lease was executed successfully
	Success = 1;

	// The lease was not executed succesfully, but cannot be run again.
	Failed = 2;

	// The lease was cancelled by request
	Cancelled = 4;
}

// Reason for an outcome of a lease. This enum mirrors the LeaseOutcomeReason in C# code
enum RpcLeaseOutcomeReason {
	// No specific reason
	None = 0;

	// Lease finished due to an unhandled or unknown error
	UnexpectedError = 1;

	// Lease finished with a non-zero exit code
	NonZeroExitCode = 2;

	// User initiated
	UserInitiated = 4;

	// Server initiated
	ServerInitiated = 8;

	// Agent initiated
	AgentInitiated = 16;

	// Agent's session was terminated
	AgentSessionTerminated = 32;

	// Agent became busy during lease (for example with non-Horde related work when running on a workstation)
	AgentBusy = 64;
	
	// Lease deemed to be a duplicate. For example two identical jobs executing or too many leases from a single user
	Duplicate = 128;

	// Lease execution timed out
	TimedOut = 256;

	// Lease no longer needed despite scheduled/started
	NoLongerNeeded = 512;

	// Lease was cancelled but no reason was reported by the agent
	NoReasonReported = 1024;
}