// Copyright Epic Games, Inc. All Rights Reserved.

//
// Derived from the Remote Workers API:
// https://github.com/googleapis/googleapis/blob/master/google/devtools/remoteworkers/v1test2/worker.proto
//

// Copyright 2019 Google LLC.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

syntax = "proto3";

package Horde.LeaseStateType;

option csharp_namespace = "HordeCommon.Rpc";

// Note: This enum mirrors the LeaseState enum in native code
enum RpcLeaseState {

	// Default value.
	Unspecified = 0;

	// Set by the server when waiting for an agent to accept the lease. Once processed, the agent should transition the lease state to active.
	Pending = 1;

	// The agent is actively working on this lease.
	Active = 2;

	// The agent has finished working on this lease.
	Completed = 3;

	// Set by the server to indicate that the lease should be cancelled.
	Cancelled = 4;
}
