// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "horde/enrollment_rpc_messages.proto";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

service EnrollmentRpc {

	// Enrolls an agent with the server. This call will not return a stream response until the agent is manually approved on the dashboard.
	rpc EnrollAgent(stream EnrollAgentRequest) returns (stream EnrollAgentResponse);
}
