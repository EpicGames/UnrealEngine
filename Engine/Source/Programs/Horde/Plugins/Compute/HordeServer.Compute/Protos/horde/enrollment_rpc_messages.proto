// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

message EnrollAgentRequest {

	// Unique key identifying the agent. Should be an unguessable unique id (eg. cryptographic random number). Does not need to be persisted.
	string Key = 1;

	// The machine hostname. This is typically used as a base for the agent identifier.
	string HostName = 2;

	// Description of the machine used to provide additional context to approvers.
	string Description = 3;
}

message EnrollAgentResponse {

	// Identifier for the new agent
	string Id = 1;

	// Token for creating a session
	string Token = 2;
}
