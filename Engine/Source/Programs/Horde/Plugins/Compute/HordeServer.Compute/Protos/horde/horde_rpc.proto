// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/empty.proto";

import "horde/horde_rpc_messages.proto";

package Horde;

option csharp_namespace = "HordeCommon.Rpc";

service HordeRpc {
	rpc QueryServerState(stream RpcQueryServerStateRequest) returns (stream RpcQueryServerStateResponse) {
		// Now using native connection handling/retries for rollovers
		option deprecated = true;
	}

	rpc QueryServerStateV2(stream RpcQueryServerStateRequest) returns (stream RpcQueryServerStateResponse) {
		// Now using native connection handling/retries for rollovers
		option deprecated = true;
	}

	rpc CreateAgent(RpcCreateAgentRequest) returns (RpcCreateAgentResponse);

	rpc CreateSession(RpcCreateSessionRequest) returns (RpcCreateSessionResponse);
	rpc UpdateSession(stream RpcUpdateSessionRequest) returns (stream RpcUpdateSessionResponse); // Not actually a streaming RPC, but streams are used to cancel the connection from either end

	rpc DownloadSoftware(RpcDownloadSoftwareRequest) returns (stream RpcDownloadSoftwareResponse);

	rpc UploadTelemetry(RpcUploadTelemetryRequest) returns (google.protobuf.Empty);
}

