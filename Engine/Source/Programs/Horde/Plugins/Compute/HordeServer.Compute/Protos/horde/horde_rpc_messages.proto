// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/timestamp.proto";

import "horde/common/agent_status.proto";

import "horde/messages/agent_capabilities.proto";
import "horde/messages/lease.proto";

package Horde;

option csharp_namespace = "HordeCommon.Rpc";

// Defines the operation mode of the agent
enum Mode {
	// For backwards compatibility
	UNSPECIFIED = 0;
	
	// Dedicated mode
	// - Trusted agent running in a controlled environment (e.g., build farm)
	// - Does not run any other non-Horde workloads
	// - Capable of executing all lease types
	DEDICATED = 1;

	// Workstation mode
	// - Low trust level, typically used for spare on-premise hardware like workstations
	// - Lease execution is inherently unreliable as these agents yield to non-Horde workloads
	// - Limited to compute leases only, primarily to support remote execution
	WORKSTATION = 2;  
}

////// Server State Operations //////

message RpcQueryServerStateRequest {

	// Host name to identify the agent
	string Name = 1;
}

message RpcQueryServerStateResponse {

	// Host name to identify the server
	string Name = 1;

	// Whether the server is currently shutting down
	bool Stopping = 2;
}

////// Agent Operations //////

message RpcCreateAgentRequest {

	// The machine hostname. This is typically used as a base for the agent identifier.
	string Name = 1;

	// Whether agent should register as being ephemeral.
	// Doing so will not persist any long-lived data on the server and once disconnected it's assumed to have been deleted permanently.
	// Ideal for short-lived agents, such as spot instances on AWS EC2.
	bool Ephemeral = 2;

	// The operation mode of the agent.
	Mode Mode = 3;
}

message RpcCreateAgentResponse {

	// Identifier for the new agent
	string Id = 1;

	// Token for creating a session
	string Token = 2;
}

////// Agent Session Operations //////

message RpcCreateSessionRequest {

	// The agent id
	string Id = 1;

	// The status of this agent (always assumes 'Ok')
	reserved 2;
	reserved "Status";

	// Capabilities of this agent
	RpcAgentCapabilities Capabilities = 3;

	// Version of the software running on this agent
	string Version = 4;
}

message RpcCreateSessionResponse {

	// The agent Id
	string AgentId = 1;

	// The new session name
	string SessionId = 2;

	// Time at which this session expires. The agent should renew the session before then.
	google.protobuf.Timestamp ExpiryTime = 3;

	// The current session token
	string Token = 4;
}

message RpcUpdateSessionRequest {

	// The unique agent id
	string AgentId = 1;

	// Unique id of the session for this agent
	string SessionId = 2;

	// The new agent status
	Horde.AgentStatusType.RpcAgentStatus Status = 3;

	// Updated capabilities for this agent. Optional.
	RpcAgentCapabilities Capabilities = 5;

	// Current leases for this agent
	repeated RpcLease Leases = 4;

	// Used for incremental updates to capabilities. Optional.
	// Unlike Capabilities that replaces the full set, these merge into existing values by key.
	RpcAgentCapabilities CapabilityOverrides = 6;
}

message RpcUpdateSessionResponse {

	// Time at which this session expires. The agent should renew the session before then.
	google.protobuf.Timestamp ExpiryTime = 1;

	// New leases for this agent
	repeated RpcLease Leases = 2;
	
	// Pool IDs calling agent is a member of
	repeated string PoolIds = 3;

	// The new agent status
	Horde.AgentStatusType.RpcAgentStatus Status = 4;
}

/// SOFTWARE

message RpcDownloadSoftwareRequest {

	// Unique id of the software to download
	string Version = 1;
}

message RpcDownloadSoftwareResponse {

	// Chunk of data
	bytes Data = 1;
}

/// EVENTS

message RpcUploadTelemetryRequest {

	// Timestamp when event was generated
	google.protobuf.Timestamp Timestamp = 1;
	
	// The agent identifier
	string AgentId = 2;
	
	// Percentage of time the CPU was busy executing code in user space
	float UserCpu = 3;

	// Percentage of time the CPU was busy executing code in kernel space
	float SystemCpu = 4;

	// Percentage of time the CPU was idling
	float IdleCpu = 5;

	// Total memory installed (megabytes)
	uint32 TotalRam = 6;

	// Available memory (megabytes)
	uint32 FreeRam = 7;

	// Used memory (megabytes)
	uint32 UsedRam = 8;

	// Agent free disk space (megabytes)
	uint64 WorkingFreeDisk = 9;

	// Agent disk size (megabytes)
	uint64 WorkingTotalDisk = 10;

	// System free disk space (megabytes)
	uint64 SystemFreeDisk = 11;

	// System disk size (megabytes)
	uint64 SystemTotalDisk = 12;
}
