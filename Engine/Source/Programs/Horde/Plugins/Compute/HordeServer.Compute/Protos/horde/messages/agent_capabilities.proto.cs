// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;

#pragma warning disable CS1591

namespace HordeCommon.Rpc.Messages
{
	partial class RpcAgentCapabilities
	{
		public RpcAgentCapabilities(IEnumerable<string> properties)
		{
			Properties.AddRange(properties);
		}

		public RpcAgentCapabilities MergeDevices()
		{
#pragma warning disable CS0612 // Type or member is obsolete
			if (Devices.Count == 0)
			{
				return this;
			}
			else
			{
				RpcAgentCapabilities other = new RpcAgentCapabilities();
				other.Properties.Add(Properties);
				other.Resources.Add(Resources);

				if (Devices.Count > 0)
				{
					RpcDeviceCapabilities? primaryDevice = Devices[0];
					other.Properties.Add(primaryDevice.Properties);
					other.Resources.Add(primaryDevice.Resources);
				}

				other.CopyPropertyToResource(KnownResourceNames.LogicalCores);
				other.CopyPropertyToResource(KnownResourceNames.Ram);

				return other;
			}
#pragma warning restore CS0612 // Type or member is obsolete
		}

		void CopyPropertyToResource(string name)
		{
			if (!Resources.ContainsKey(name))
			{
				foreach (string property in Properties)
				{
					if (property.Length > name.Length && property.StartsWith(name, StringComparison.OrdinalIgnoreCase) && property[name.Length] == '=')
					{
						int value;
						if (Int32.TryParse(property.AsSpan(name.Length + 1), out value))
						{
							Resources.Add(name, value);
						}
					}
				}
			}
		}
	}
}
