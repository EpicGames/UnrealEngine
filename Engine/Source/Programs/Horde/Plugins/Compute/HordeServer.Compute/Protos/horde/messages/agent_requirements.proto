// Copyright Epic Games, Inc. All Rights Reserved.

//
// Derived from the Remote Workers API:
// https://github.com/googleapis/googleapis/blob/master/google/devtools/remoteworkers/v1test2/worker.proto
//

// Copyright 2019 Google LLC.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Messages";

message RpcAgentRequirements {

	// Devices which are attached to this agent. The first device describes the host machine.
	reserved 1;
	reserved "Devices";

	// List of properties for this agent, in the form KEY=VALUE. Properties are attributes of a device which a work item can request.
	// Key names need not be unique; a device can advertise multiple values for the same key of different specificities.
	repeated string Properties = 2;

	// Resources for this device
	map<string, int32> Resources = 4;

	// Whether this agent can be shared
	bool Shared = 3;
}
