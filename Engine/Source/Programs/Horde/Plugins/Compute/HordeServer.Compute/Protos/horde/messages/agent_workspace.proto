// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/wrappers.proto";

option csharp_namespace = "HordeCommon.Rpc.Messages";

message RpcAgentWorkspace {

	// Name of the Perforce cluster that was used to configure this agent. 
	// This and ConfiguredUserName are used to reconstruct agent workspace definition and test whether config has changed during a conform loop.
	google.protobuf.StringValue ConfiguredCluster = 13;

	// Username that was set in the agent configuration, ignoring any default values that were applied.
	google.protobuf.StringValue ConfiguredUserName = 14;

	// The actual cluster being used
	string Cluster = 15;

	// The base perforce server and port (eg. perforce:1666)
	google.protobuf.StringValue BaseServerAndPort = 16;

	// The perforce server and port (eg. perforce:1666)
	google.protobuf.StringValue ServerAndPort = 7;

	// User to log into Perforce with (eg. buildmachine)
	google.protobuf.StringValue UserName = 8;

	// Password to use to log in to Perforce
	google.protobuf.StringValue Password = 9;

	// Login ticket to use for Perforce operations
	google.protobuf.StringValue Ticket = 19;

	// Identifier to distinguish this workspace from other workspaces
	string Identifier = 3;

	// The stream to sync
	string Stream = 4;

	// Custom view for the workspace
	repeated string View = 5;

	// Whether to use an incremental workspace
	bool Incremental = 6;

	// Method to use when syncing/materializing data from Perforce
	string Method = 17;

	// Whether to create a partitioned workspace
	bool Partitioned = 18;

	// Minimum disk space that must be available *after* syncing this workspace (in megabytes)
	int64 MinScratchSpace = 20;

	// Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk.
	int64 ConformDiskFreeSpace = 21;

	// Revision for hash-based revision lookups.
	google.protobuf.StringValue RevisionHash = 22;
}
