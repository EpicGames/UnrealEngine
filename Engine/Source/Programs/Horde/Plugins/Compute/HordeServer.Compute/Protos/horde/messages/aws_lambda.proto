// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Messages";

// Request sent to invoked AWS Lambda function that will run the worker service in hope of serving one lease.
message RpcAwsLambdaListenRequest {
	// Max waiting time for a new lease to arrive before finishing the AWS Lambda execution
	int32 MaxWaitTimeForLeaseMs = 1;
}

// Response sent for the above request
message RpcAwsLambdaListenResponse {
	// If a lease was accepted and executed
	bool DidAcceptLease = 1;
}