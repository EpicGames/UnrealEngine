// Copyright Epic Games, Inc. All Rights Reserved.

//
// Derived from the Remote Workers API:
// https://github.com/googleapis/googleapis/blob/master/google/devtools/remoteworkers/v1test2/worker.proto
//

// Copyright 2019 Google LLC.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

syntax = "proto3";

import "horde/common/lease_outcome.proto";
import "horde/common/lease_state.proto";
import "google/protobuf/any.proto";

option csharp_namespace = "HordeCommon.Rpc.Messages";

message RpcLeaseMetricCpuTime {
	// CPU time spent in user mode
	uint32 UserMs = 1;

	// CPU time spent in system/kernel mode
	uint32 SystemMs = 2;

	// CPU time spent idle
	uint32 IdleMs = 3;
}

message RpcLeaseMetrics {
	// Time taken to prepare for lease execution (e.g syncing VCS data)
	uint32 SetupTimeMs = 1;

	// Time taken to clean up after a lease execution (removing files created etc)
	uint32 TeardownTimeMs = 2;

	// CPU utilization in percent based on number of CPU seconds that was available during the lease
	// This is machine-wide so it assumes the lease has exclusive access to the agent and other background work is negligible.
	double GlobalCpuUtilization = 3;
	
	// Number of logical CPU cores that were available to the lease
	int32 CpuCount = 4;

	// Machine-wide CPU time spent during this lease
	RpcLeaseMetricCpuTime GlobalCpuTime = 5;
}

message RpcLease {

	// Unique id of this lease
	string IdString = 1;
	
	// The current state of the lease
	Horde.LeaseStateType.RpcLeaseState State = 2;

	// The work to be performed. Specified by the server when the lease state is pending.
	google.protobuf.Any Payload = 3;

	// Outcome of this lease. Note that this corresponds to the agent's *execution* of the lease, not the result of the actual work done.
	Horde.LeaseOutcomeType.RpcLeaseOutcome Outcome = 4;

	// Reasons leading to the outcome of this lease.
	repeated Horde.LeaseOutcomeType.RpcLeaseOutcomeReason OutcomeReasons = 6;

	// Metrics calculated at the end of lease
	RpcLeaseMetrics Metrics = 7;

	// Output from executing the lease. This is typically empty or a protobuf-encoded object of a type understood by the task source.
	bytes Output = 5;
}
