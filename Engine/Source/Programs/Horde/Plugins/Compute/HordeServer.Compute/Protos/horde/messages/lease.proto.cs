// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;

namespace HordeCommon.Rpc.Messages
{
	/// <summary>
	/// Additional methods for RpcLease
	/// </summary>
	partial class RpcLease
	{
		/// <summary>
		/// Typed accessor for the <see cref="RpcLease.IdString"/> field.
		/// </summary>
		public LeaseId Id
		{
			get => LeaseId.Parse(IdString);
			set => IdString = value.ToString();
		}

		/// <summary>
		/// Convert outcome reason(s) from Protobuf-format to C# enum
		/// </summary>
		public LeaseOutcomeReason ToNativeOutcomeReason()
		{
			return OutcomeReasons
				.Select(r => (LeaseOutcomeReason)(long)r)
				.Aggregate(LeaseOutcomeReason.None, (current, reason) => current | reason);
		}
		
		/// <summary>
		/// Convert C# enum with outcome reason(s) to Protobuf-format
		/// </summary>
		public static List<RpcLeaseOutcomeReason> ToProtobufOutcomeReasons(LeaseOutcomeReason reasons)
		{
			List<RpcLeaseOutcomeReason> result = [];
			foreach (RpcLeaseOutcomeReason rpcReason in Enum.GetValues(typeof(RpcLeaseOutcomeReason)))
			{
				if (rpcReason != RpcLeaseOutcomeReason.None && ((long)reasons & (long)rpcReason) != 0)
				{
					result.Add(rpcReason);
				}
			}
    
			return result;
		}
	}
}
