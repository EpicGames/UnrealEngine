// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "horde/common/agent_status.proto";
import "horde/common/lease_state.proto";
import "google/protobuf/any.proto";

option csharp_namespace = "HordeCommon.Rpc.Messages";

// State of an active session
message RpcSession {

	// The agent id
	string AgentIdValue = 1;

	// The session id
	string SessionIdValue = 2;

	// Time that this session was last updated, as C# DateTime ticks. This is also used as an update identifier for compare-swap operations.
	int64 UpdateTicks = 3;

	// Agent status
	Horde.AgentStatusType.RpcAgentStatus Status = 4;

	// List of leases on this agent
	repeated RpcSessionLease Leases = 5;

	// Hash of the capabilities for this agent
	string CapabilitiesHashValue = 6;

	// Last timestamp (in C# DateTime ticks) of the queue that was added
	int64 LastFilterUpdateTicks = 7;
}

// Lease reservation within the context of a session
message RpcSessionLease {

	// Unique id of this lease
	string IdValue = 1;
	
	// The parent lease id
	string ParentIdValue = 2;
	
	// The current state of the lease
	Horde.LeaseStateType.RpcLeaseState State = 3;

	// The work to be performed. Specified by the server when the lease state is pending.
	google.protobuf.Any Payload = 4;

	// Set of resources that have been allocated to this lease
	map<string, int32> Resources = 5;

	// Whether the lease reserves the agent exclusively
	bool Exclusive = 6;
}

