// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeServer.Agents;

#pragma warning disable CS1591

namespace HordeCommon.Rpc.Messages
{
	partial class RpcSession
	{
		public static TimeSpan ExpireAfterTime { get; } = TimeSpan.FromMinutes(5.0);

		public AgentId AgentId
		{
			get => new EpicGames.Horde.Agents.AgentId(AgentIdValue);
			set => AgentIdValue = value.ToString();
		}

		public SessionId SessionId
		{
			get => EpicGames.Horde.Agents.Sessions.SessionId.Parse(SessionIdValue);
			set => SessionIdValue = value.ToString();
		}

		public IoHash CapabilitiesHash
		{
			get => IoHash.Parse(CapabilitiesHashValue);
			set => CapabilitiesHashValue = value.ToString();
		}

		public DateTime ExpiryTime
			=> new DateTime(UpdateTicks + ((Status == RpcAgentStatus.Stopped)? 0 : ExpireAfterTime.Ticks), DateTimeKind.Utc);
	}

	partial class RpcSessionLease : IAgentLease
	{
		public LeaseId Id
		{
			get => LeaseId.Parse(IdValue);
			set => IdValue = value.ToString();
		}

		public LeaseId? ParentId
		{
			get => String.IsNullOrEmpty(ParentIdValue) ? null : LeaseId.Parse(ParentIdValue);
			set => ParentIdValue = value?.ToString() ?? String.Empty;
		}

		IReadOnlyDictionary<string, int>? IAgentLease.Resources => Resources;

		LeaseState IAgentLease.State => (LeaseState)State;
	}
}
