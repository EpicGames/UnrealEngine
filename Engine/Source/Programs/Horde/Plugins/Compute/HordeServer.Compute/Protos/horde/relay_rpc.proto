// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/timestamp.proto";

package Horde.Relay;

option csharp_namespace = "Horde.Common.Rpc";

service RelayRpc {
	// Get a list of port mappings to apply locally on the agent calling this service method
	rpc GetPortMappings(GetPortMappingsRequest) returns (stream GetPortMappingsResponse);
}

message GetPortMappingsRequest {
	// Cluster ID calling relay agent represents
	string ClusterId = 2;

	// Unique ID of the calling agent
	string AgentId = 3;

	// Change count of port mappings the client last knew about
	// Set to -1 to always long-poll for a result and skip revision check
	// Set to -2 to let server return latest result without long-polling 
	int32 RevisionCount = 4;
	
	// List of IP addresses current relay server can serve and listen to
	repeated string IpAddresses = 1;
}

message GetPortMappingsResponse {
	repeated PortMapping PortMappings = 1;

	// Change count of port mappings. For each modification to them, this value is incremented.
	int32 RevisionCount = 2;
}

enum PortProtocol {
	Tcp = 0;
	Udp = 1;
}

// A forwarded port
message Port {
	// Port that relay server will listen on
	int32 RelayPort = 1;

	// Port on agent where traffic will be forwarded to
	int32 AgentPort = 2;
	
	// Protocol to be forwarded
	PortProtocol Protocol = 3;
}

message PortMapping {
	// Which lease these port mapping apply to
	string LeaseId = 1;

	// Time when this port mapping was created
	google.protobuf.Timestamp CreatedAt = 5;

	// List of source IPs allowed to use this port mapping (usually IP of initiator/requester)
	// Empty list allows any source IP
	repeated string AllowedSourceIps = 4;
	
	// Destination IP where traffic will be forwarded
	string AgentIp = 2;
	
	// List of ports
	repeated Port Ports = 3;
}

// Internal message not part of the API. Used for Redis serialization.
message RelayAgentInfo {
	// Unique ID of agent
	string AgentId = 1;

	// Unix timestamp when this relay agent was last heard from
	google.protobuf.Timestamp LastUpdate = 2;
	
	// IP addresses relay agent is listening on and capable of serving
	repeated string IpAddresses = 3;
}