// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "horde/tasks/compute_task.proto";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

// A compute task specific for AWS Lambda invocations.
// Includes the task along with any extra metadata needed to execute.
message AwsLambdaComputeTaskMessage {
	// Actual compute task (re-used)
	ComputeTaskMessage Task = 1;
}

// A compute task result specific for AWS Lambda invocations.
// Includes the task along with any extra metadata needed to execute.
message AwsLambdaComputeTaskResultMessage {
	ComputeTaskResultMessage TaskResult = 1;
}