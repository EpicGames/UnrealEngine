// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

enum ComputeEncryption {
	COMPUTEENCRYPTION_UNSPECIFIED = 0;
	COMPUTEENCRYPTION_NONE = 1;
	COMPUTEENCRYPTION_AES = 2;
	COMPUTEENCRYPTION_SSL_RSA_2048 = 3;
	COMPUTEENCRYPTION_SSL_ECDSA_P256 = 4;
}

message ComputeTask {

	// Type of encryption to enable when communicating with the remote
	ComputeEncryption Encryption = 7;

	// Cryptographic nonce to identify the request
	bytes Nonce = 1;

	// The AES key for communicating with the remote
	bytes Key = 2;

	// X.509 certificate for communicating with the remote (used with SSL)
	bytes Certificate = 6;

	// Resources assigned to this lease
	map<string, int32> Resources = 3;

	// Log id for output
	string LogId = 4;

	// The parent lease id
	string ParentLeaseId = 5;

	// The protocol version number, from the ComputeProtocol enum.
	int32 Protocol = 8;

	// Maximum duration (in milliseconds) that communication can be inactive between
	// the compute client and task-running agent before the connection is terminated.
	int32 InactivityTimeoutMs = 9;
}
