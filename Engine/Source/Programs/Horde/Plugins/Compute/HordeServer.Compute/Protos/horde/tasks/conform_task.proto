// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "horde/messages/agent_workspace.proto";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

message ConformTask {

	reserved 4;

	// The log id for this task
	string LogId = 2;

	// Specifies the new workspaces to sync
	repeated RpcAgentWorkspace Workspaces = 1;

	// Whether to remove any untracked files from the workspace. Ignored for workspaces sharing a common root directory.
	bool RemoveUntrackedFiles = 3;
}
