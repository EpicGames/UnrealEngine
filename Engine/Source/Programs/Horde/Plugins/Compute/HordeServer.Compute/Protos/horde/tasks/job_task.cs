// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeCommon.Rpc.Tasks
{
	partial class RpcJobOptions
	{
	}
}
