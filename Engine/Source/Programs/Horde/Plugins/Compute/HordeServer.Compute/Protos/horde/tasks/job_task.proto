// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "horde/messages/agent_workspace.proto";
import "google/protobuf/wrappers.proto";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

// Task to execute a job. Passed as a payload in a new lease.
message ExecuteJobTask {

	// Unique id of the job to execute
	string JobId = 1;

	// Unique id of the batch to execute
	string BatchId = 2;

	// Unique id of the log to use for this task
	string LogId = 5;

	// Name of the job, for diagnostic purposes
	string JobName = 6;

	// The autosdk workspace to use for this job
	RpcAgentWorkspace AutoSdkWorkspace = 3;

	// The agent workspace to use for this job
	RpcAgentWorkspace Workspace = 4;

	// Options for executing the job
	RpcJobOptions JobOptions = 8;

	// Token for communicating with the server about this job
	string Token = 9;

	// Namespace for storing output data (DEPRECATED)
	reserved 10;
	reserved "NamespaceId";

	// Prefix for all things written to the above namespace (DEPRECATED)
	reserved 11;
	reserved "StoragePrefix";
}

// Options for executing a job
message RpcJobOptions
{
	// Outdated fields that have been removed
	reserved 3;
	reserved "UseNewTempStorage";

	reserved 7;
	reserved "CollectIbMonFilesAsArtifacts";

	reserved 9;
	reserved "BundleVersion";

	// Name of the executor to use
	google.protobuf.StringValue Executor = 1;

	// Whether to execute using Wine emulation on Linux
	google.protobuf.BoolValue UseWine = 4;

	// Executes the job lease in a separate process
	google.protobuf.BoolValue RunInSeparateProcess = 5;

	// What workspace materializer to use in WorkspaceExecutor. Will override any value from workspace config.
	google.protobuf.StringValue WorkspaceMaterializer = 6;

	// Options for executing a job inside a container
	RpcJobContainerOptions Container = 8;

	// Name of the driver to use
	google.protobuf.StringValue Driver = 10;
}

// Options for executing a job inside a container
message RpcJobContainerOptions
{
	// Whether to execute job inside a container
	google.protobuf.BoolValue Enabled = 1;
	
	// Image URL to container, such as "quay.io/podman/hello"
	google.protobuf.StringValue ImageUrl = 2;

	// Container engine executable (docker or with full path like /usr/bin/podman)
	google.protobuf.StringValue ContainerEngineExecutable = 3;

	// Additional arguments to pass to container engine
	google.protobuf.StringValue ExtraArguments = 4;
}