// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

message RestartTask {

	// The log id for this task
	string LogId = 1;
}
