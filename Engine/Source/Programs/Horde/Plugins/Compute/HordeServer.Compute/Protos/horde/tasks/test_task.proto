// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

// Used for tests, a no-op task
message TestTask {
}
