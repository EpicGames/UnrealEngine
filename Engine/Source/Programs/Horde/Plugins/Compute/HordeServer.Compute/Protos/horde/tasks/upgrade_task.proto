// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

option csharp_namespace = "HordeCommon.Rpc.Tasks";

// Task to update the agent software. Passed as a payload in a new lease.
message UpgradeTask {

	// Specifies the new software id
	string SoftwareId = 1;

	// Log id to use for this lease
	string LogId = 2;
}