// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Net;
using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Tools;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Sessions;
using HordeServer.Agents.Telemetry;
using HordeServer.Telemetry;
using HordeServer.Tools;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Server
{
	/// <summary>
	/// Implements the Horde gRPC service for bots updating their status and dequeing work
	/// </summary>
	[Authorize]
	public class RpcService : HordeRpc.HordeRpcBase
	{
		/// <summary>
		/// Number of session mismatches per agent that generate detailed (Error-level) structured exceptions.
		/// Beyond this threshold, mismatches are logged at Debug level to suppress spam.
		/// </summary>
		internal const int MaxDetailedConflictErrors = 20;

		/// <summary>
		/// Timeout before closing a long-polling request (client will retry again)
		/// </summary>
		internal TimeSpan _longPollTimeout = TimeSpan.FromMinutes(9);

		readonly AgentService _agentService;
		readonly ILifetimeService _lifetimeService;
		readonly IToolCollection _toolCollection;
		readonly IAgentTelemetryCollection _agentTelemetryCollection;
		readonly IAclService _aclService;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		readonly SessionConflictService _sessionConflictService;
		readonly ILogger _logger;
		readonly ITelemetryWriter _telemetryWriter;

		/// <summary>
		/// Constructor
		/// </summary>
		public RpcService(
			AgentService agentService,
			ILifetimeService lifetimeService,
			IToolCollection toolCollection,
			IAgentTelemetryCollection agentTelemetryCollection,
			IAclService aclService,
			IOptionsSnapshot<ComputeConfig> computeConfig,
			SessionConflictService sessionConflictService,
			ILogger<RpcService> logger,
			ITelemetryWriter telemetryWriter)
		{
			_agentService = agentService;
			_lifetimeService = lifetimeService;
			_toolCollection = toolCollection;
			_agentTelemetryCollection = agentTelemetryCollection;
			_aclService = aclService;
			_computeConfig = computeConfig;
			_sessionConflictService = sessionConflictService;
			_logger = logger;
			_telemetryWriter = telemetryWriter;
		}

		/// <summary>
		/// Waits until the server is terminating
		/// </summary>
		/// <param name="reader">Request reader</param>
		/// <param name="writer">Response writer</param>
		/// <param name="context">Context for the call</param>
		/// <returns>Response object</returns>
		[Obsolete("RPC connection management now done natively")]
		public override async Task QueryServerState(IAsyncStreamReader<RpcQueryServerStateRequest> reader, IServerStreamWriter<RpcQueryServerStateResponse> writer, ServerCallContext context)
		{
			if (await reader.MoveNext())
			{
				RpcQueryServerStateRequest request = reader.Current;
				_logger.LogInformation("Start server query for client {Name}", request.Name);

				// Return the current response
				RpcQueryServerStateResponse response = new RpcQueryServerStateResponse();
				response.Name = Dns.GetHostName();
				await writer.WriteAsync(response);

				// Move to the next request from the client. This should always be the end of the stream, but will not occur until the client stops requesting responses.
				Task<bool> moveNextTask = reader.MoveNext();

				// Wait for the client to close the stream or a shutdown to start
				Task longPollDelay = Task.Delay(_longPollTimeout);
				Task waitTask = await Task.WhenAny(moveNextTask, _lifetimeService.StoppingTask, longPollDelay);

				if (waitTask == moveNextTask)
				{
					throw new Exception("Unexpected request to QueryServerState posted from client.");
				}
				else if (waitTask == _lifetimeService.StoppingTask)
				{
					_logger.LogInformation("Notifying client {Name} of server shutdown", request.Name);
					await writer.WriteAsync(response);
				}
				else if (waitTask == longPollDelay)
				{
					// Send same response as server shutdown. In the agent perspective, they will be identical.
					await writer.WriteAsync(response);
				}
			}
		}

		/// <summary>
		/// Waits until the server is terminating
		/// </summary>
		/// <param name="reader">Request reader</param>
		/// <param name="writer">Response writer</param>
		/// <param name="context">Context for the call</param>
		/// <returns>Response object</returns>
		[Obsolete("RPC connection management now done natively")]
		public override async Task QueryServerStateV2(IAsyncStreamReader<RpcQueryServerStateRequest> reader, IServerStreamWriter<RpcQueryServerStateResponse> writer, ServerCallContext context)
		{
			if (await reader.MoveNext())
			{
				RpcQueryServerStateRequest request = reader.Current;
				_logger.LogDebug("Start server query for client {Name}", request.Name);

				try
				{
					// Return the current response
					RpcQueryServerStateResponse response = new RpcQueryServerStateResponse();
					response.Name = Dns.GetHostName();
					response.Stopping = _lifetimeService.IsStopping;
					await writer.WriteAsync(response);

					// Move to the next request from the client. This should always be the end of the stream, but will not occur until the client stops requesting responses.
					Task<bool> moveNextTask = reader.MoveNext();

					// Wait for the client to close the stream or a shutdown to start
					if (await Task.WhenAny(moveNextTask, _lifetimeService.StoppingTask) == _lifetimeService.StoppingTask)
					{
						response.Stopping = true;
						await writer.WriteAsync(response);
					}

					// Wait until the client has finished sending
					while (await moveNextTask)
					{
						moveNextTask = reader.MoveNext();
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception in QueryServerState for {Name}", request.Name);
					throw;
				}
			}
		}

		/// <summary>
		/// Creates a new agent
		/// The enrollment process is bypassed for all agent types created through this endpoint.
		/// </summary>
		/// <param name="request">Request to create a new agent</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>New agent's ID and authentication token (if applicable)</returns>
		public override async Task<RpcCreateAgentResponse> CreateAgent(RpcCreateAgentRequest request, ServerCallContext context)
		{
			using IDisposable? scope = _logger.BeginScope("CreateAgent({AgentId})", request.Name.ToString());
			
			switch (request.Mode)
			{
				case Mode.Unspecified or Mode.Dedicated:
					EnsurePermission(AgentAclAction.CreateAgent, "User is not authorized to create a dedicated agent", context);
					// Dedicated agents are assumed trusted
					CreateAgentOptions options = new(new AgentId(request.Name), AgentMode.Dedicated, request.Ephemeral, "", [$"{KnownPropertyNames.Trusted}=true"]);
					IAgent agent = await _agentService.CreateAgentAsync(options, context.CancellationToken);
					
					List<AclClaimConfig> claims = [new AclClaimConfig(HordeClaimTypes.Agent, agent.Id.ToString())];
					return new RpcCreateAgentResponse
					{
						Id = agent.Id.ToString(),
						Token = await _aclService.IssueBearerTokenAsync(claims, null, context.CancellationToken)
					};
				case Mode.Workstation:
					EnsurePermission(AgentAclAction.CreateWorkstationAgent, "User is not authorized to create a workstation agent", context);
					CreateAgentOptions workstationOptions = new(new AgentId(request.Name), AgentMode.Workstation, request.Ephemeral, "");
					IAgent workstationAgent = await _agentService.CreateAgentAsync(workstationOptions, context.CancellationToken);
					// Workstation-based agents always authenticate as the current user and should *not* be issued a token
					return new RpcCreateAgentResponse { Id = workstationAgent.Id.ToString(), Token = "" };
					
				default:
					throw new StructuredRpcException(StatusCode.InvalidArgument, $"Invalid agent mode: {request.Mode}");
			}
		}
		
		private void EnsurePermission(AclAction action, string errorMessage, ServerCallContext context)
		{
			if (!_computeConfig.Value.Authorize(action, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, errorMessage);
			}
		}

		/// <summary>
		/// Creates a new session
		/// </summary>
		/// <param name="request">Request to create a new agent</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcCreateSessionResponse> CreateSession(RpcCreateSessionRequest request, ServerCallContext context)
		{
			if (request.Capabilities == null)
			{
				throw new StructuredRpcException(StatusCode.InvalidArgument, "Capabilities may not be null");
			}

			AgentId agentId = new AgentId(request.Id);
			_logger.LogInformation("Attempting to create session for agent {AgentId}", agentId);
			using IDisposable? scope = _logger.BeginScope("CreateSession({AgentId})", agentId.ToString());

			ComputeConfig computeConfig = _computeConfig.Value;

			// Find the agent
			IAgent? agent = await _agentService.GetAgentAsync(agentId);
			if (agent == null)
			{
				if (!computeConfig.Authorize(AgentAclAction.CreateAgent, context.GetHttpContext().User))
				{
					throw new StructuredRpcException(StatusCode.PermissionDenied, "User is not authenticated to create new agents");
				}

				agent = await _agentService.CreateAgentAsync(new CreateAgentOptions(agentId, AgentMode.Dedicated, false, ""));
			}

			// Check the enrollment key in the user token matches
			string enrollmentKey = context.GetHttpContext().User.FindFirstValue(HordeClaimTypes.AgentEnrollmentKey) ?? String.Empty;
			if (!String.Equals(enrollmentKey, agent.EnrollmentKey, StringComparison.OrdinalIgnoreCase))
			{
				_logger.LogError("Enrollment key does not match for {AgentId} (was {OldKey}, now {NewKey})", agent.Id, agent.EnrollmentKey, enrollmentKey);
				throw new StructuredRpcException(StatusCode.PermissionDenied, $"Enrollment key does not match for {agent.Id}");
			}

			// Make sure we're allowed to create sessions on this agent
			ClaimsPrincipal user = context.GetHttpContext().User;
			if (!computeConfig.Authorize(SessionAclAction.CreateSession, user) && !user.HasAgentClaim(agentId))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "User is not authenticated to create session for {AgentId}", agentId);
			}

			// Create a new session
			agent = await _agentService.CreateSessionAsync(agent, request.Capabilities.MergeDevices(), request.Version, context.CancellationToken);
			if (agent == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Agent {AgentId} not found", agentId);
			}

			IPAddress? createRemoteIp = context.GetHttpContext().Connection?.RemoteIpAddress;
			_logger.LogDebug("Session {SessionId} created for agent {AgentId} from IP {RemoteIp}", agent.SessionId, agent.Id, createRemoteIp);

			// Create the response
			RpcCreateSessionResponse response = new RpcCreateSessionResponse();
			response.AgentId = agent.Id.ToString();
			response.SessionId = agent.SessionId.ToString();
			response.ExpiryTime = Timestamp.FromDateTime(agent.SessionExpiresAt!.Value);
			response.Token = await _agentService.IssueSessionTokenAsync(agent.Id, agent.SessionId!.Value, context.CancellationToken);
			return response;
		}

		/// <summary>
		/// Updates an agent session
		/// </summary>
		/// <param name="reader">Request to create a new agent</param>
		/// <param name="writer">Writer for response objects</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task UpdateSession(IAsyncStreamReader<RpcUpdateSessionRequest> reader, IServerStreamWriter<RpcUpdateSessionResponse> writer, ServerCallContext context)
		{
			// Read the request object
			Task<bool> nextRequestTask = reader.MoveNext();
			if (await nextRequestTask)
			{
				RpcUpdateSessionRequest request = reader.Current;
				using IDisposable? scope = _logger.BeginScope("UpdateSession for agent {AgentId}, session {SessionId}", request.AgentId, request.SessionId);

				_logger.LogDebug("Updating session for {AgentId}", request.AgentId);
				foreach (RpcLease lease in request.Leases)
				{
					_logger.LogDebug("Session {SessionId}, Lease {LeaseId} - State: {LeaseState}, Outcome: {LeaseOutcome}", request.SessionId, lease.Id, lease.State, lease.Outcome);
				}

				// Get a task for moving to the next item. This will only complete once the call has closed.
				using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(context.CancellationToken);
				nextRequestTask = reader.MoveNext(cancellationSource.Token);
				try
				{
					nextRequestTask = nextRequestTask.ContinueWith(task =>
					{
						cancellationSource.Cancel();
						return task.IsCanceled ? false : task.Result;
					}, TaskScheduler.Current);

					// Get the current agent state
					IAgent? agent = await _agentService.GetAgentAsync(new AgentId(request.AgentId));
					if (agent != null)
					{
						SessionId sessionId = SessionId.Parse(request.SessionId);

						// Check we're authorized to update it
						if (agent.SessionId != sessionId)
						{
							long conflictCount = await _sessionConflictService.RecordConflictAsync(agent.Id);
							if (conflictCount <= MaxDetailedConflictErrors)
							{
								throw new StructuredRpcException(StatusCode.PermissionDenied, "Agent {AgentId} has completed session {SessionId}; now executing session {NewSessionId}. Cannot update state.", request.AgentId, sessionId, agent.SessionId?.ToString() ?? "(None)");
							}
							else
							{
								_logger.LogDebug("Agent {AgentId} session mismatch (count: {ConflictCount}): expected {SessionId}, actual {NewSessionId}", request.AgentId, conflictCount, sessionId, agent.SessionId?.ToString() ?? "(None)");
								throw new RpcException(new Status(StatusCode.PermissionDenied, $"Agent {request.AgentId} session mismatch"));
							}
						}
						if (!_agentService.AuthorizeSession(agent, context.GetHttpContext().User, out string authReason))
						{
							throw new StructuredRpcException(StatusCode.PermissionDenied, "Not authenticated for {AgentId}. Reason {Reason}", request.AgentId, authReason);
						}

						// Update the session
						try
						{
							RpcAgentCapabilities? capabilityOverrides = request.CapabilityOverrides?.Properties.Count == 0 && request.CapabilityOverrides?.Resources.Count == 0 ? null : request.CapabilityOverrides;
							agent = await _agentService.UpdateSessionWithWaitAsync(agent, sessionId, (AgentStatus)request.Status, request.Capabilities?.MergeDevices(), request.Leases, capabilityOverrides, cancellationSource.Token);
						}
						catch (OperationCanceledException)
						{
							// Ignore cancellation due to a message having been received
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Swallowed exception while updating session for {AgentId}.", request.AgentId);
							throw new StructuredRpcException(StatusCode.Internal, "Failed updating session. Reason: {Reason}", ex.Message);
						}
					}

					// Handle the invalid agent case
					if (agent == null)
					{
						throw new StructuredRpcException(StatusCode.NotFound, "Invalid agent name '{AgentId}'", request.AgentId);
					}

					// Create the new session info
					if (!context.CancellationToken.IsCancellationRequested)
					{
						RpcUpdateSessionResponse response = new RpcUpdateSessionResponse();
						response.Leases.Add(agent.Leases.Select(x => x.ToRpcMessage()));
						response.ExpiryTime = (agent.SessionExpiresAt == null) ? new Timestamp() : Timestamp.FromDateTime(agent.SessionExpiresAt.Value);
						response.Status = (RpcAgentStatus)agent.Status;
						await writer.WriteAsync(response);
					}

					// Wait for the client to close the stream
					try
					{
						while (await nextRequestTask)
						{
							nextRequestTask = reader.MoveNext(cancellationSource.Token);
						}
					}
					catch (Exception ex)
					{
						_logger.LogDebug(ex, "Ignoring exception while finishing UpdateSession request.");
					}
				}
				finally
				{
					await cancellationSource.CancelAsync();

					try
					{
						await nextRequestTask;
					}
					catch (Exception ex) when (IsCancellationException(ex))
					{
						// Ignore cancellation exceptions
						// A more advanced check is used as it's sometimes wrapped as an AggregateException
					}
				}
			}
		}

		/// <summary>
		/// Downloads a new agent archive
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="responseStream">Writer for the output data</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task DownloadSoftware(RpcDownloadSoftwareRequest request, IServerStreamWriter<RpcDownloadSoftwareResponse> responseStream, ServerCallContext context)
		{
			int colonIdx = request.Version.IndexOf(':', StringComparison.Ordinal);
			ToolId toolId = new ToolId(request.Version.Substring(0, colonIdx));
			string version = request.Version.Substring(colonIdx + 1);

			ITool? tool = await _toolCollection.GetAsync(toolId, context.CancellationToken);
			if (tool == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, $"Missing tool {toolId}");
			}

			if (!tool.Public && !tool.Authorize(ToolAclAction.DownloadTool, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, $"User does not have DownloadTool entitlement for {toolId}");
			}

			IToolDeployment? deployment = tool.Deployments.LastOrDefault(x => x.Version.Equals(version, StringComparison.Ordinal));
			if (deployment == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, $"Missing tool version {version}");
			}

			try
			{
				await using Stream stream = await deployment.OpenZipStreamAsync(context.CancellationToken);
				using IMemoryOwner<byte> buffer = MemoryPool<byte>.Shared.Rent(128 * 1024);
				long totalWritten = 0;
				for (; ; )
				{
					int read = await stream.ReadAsync(buffer.Memory, context.CancellationToken);
					if (read == 0)
					{
						break;
					}

					RpcDownloadSoftwareResponse response = new RpcDownloadSoftwareResponse();
					response.Data = UnsafeByteOperations.UnsafeWrap(buffer.Memory.Slice(0, read));
					await responseStream.WriteAsync(response);

					totalWritten += response.Data.Length;
				}
				_logger.LogInformation("Agent software zip is {Size:n0} bytes", totalWritten);
			}
			catch (OperationCanceledException) when (context.CancellationToken.IsCancellationRequested)
			{
				_logger.LogDebug("DownloadSoftware stream for {ToolId} cancelled by client", toolId);
			}
		}

		/// <inheritdoc/>
		public override Task<Empty> UploadTelemetry(RpcUploadTelemetryRequest request, ServerCallContext context)
		{
			AgentId agentId = new AgentId(request.AgentId);

			_logger.LogDebug("Posting telemetry data for {AgentId}", agentId);

			NewAgentTelemetry telemetry = new NewAgentTelemetry(
				request.UserCpu, request.IdleCpu, request.SystemCpu, 
				(int)request.FreeRam, (int)request.UsedRam, (int)request.TotalRam, 
				(long)request.WorkingFreeDisk, (long)request.WorkingTotalDisk,
				(long)request.SystemFreeDisk, (long)request.SystemTotalDisk);
			_agentTelemetryCollection.Add(agentId, telemetry);

			if (_telemetryWriter.Enabled)
			{
				_logger.LogDebug("Writing telemetry event for {AgentId} to data sink(s).", agentId);

				try
				{
					_telemetryWriter.WriteEvent(
						TelemetryStoreId.Default,
						new AgentTelemetry(telemetry, agentId)
					);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Failed to publish agent telemetry.");
				}
			}

			return Task.FromResult(new Empty());
		}
		
		private static bool IsCancellationException(Exception ex)
		{
			return ex switch
			{
				OperationCanceledException => true,
				AggregateException aggEx => aggEx.InnerExceptions.All(IsCancellationException),
				_ => ex.InnerException != null && IsCancellationException(ex.InnerException)
			};
		}
	}
}
