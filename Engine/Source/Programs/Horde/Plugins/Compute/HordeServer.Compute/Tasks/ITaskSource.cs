// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Linq.Expressions;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using Google.Protobuf;
using Google.Protobuf.Reflection;
using Google.Protobuf.WellKnownTypes;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tasks
{
	/// <summary>
	/// Flags indicating when a task source is valid
	/// </summary>
	[Flags]
	public enum TaskSourceFlags
	{
		/// <summary>
		/// Normal behavior
		/// </summary>
		None = 0,

		/// <summary>
		/// Indicates that tasks from this source can run during downtime
		/// </summary>
		AllowDuringDowntime = 1,

		/// <summary>
		/// Allows this source to schedule tasks when the agent is disabled
		/// </summary>
		AllowWhenDisabled = 2,

		/// <summary>
		/// Allows this source to schedule tasks when the agent is busy executing external work
		/// </summary>
		AllowWhenBusy = 4,
	}

	/// <summary>
	/// Handler for a certain lease type
	/// </summary>
	public interface ITaskSource
	{
		/// <summary>
		/// The task type
		/// </summary>
		string Type { get; }

		/// <summary>
		/// Descriptor for this lease type
		/// </summary>
		MessageDescriptor Descriptor { get; }

		/// <summary>
		/// Flags controlling behavior of this source
		/// </summary>
		TaskSourceFlags Flags { get; }

		/// <summary>
		/// Assigns a lease or waits for one to be available.
		/// 
		/// This method returns a wrapped task object. Issuing of tasks is done in two phases; all task sources are queried sequentially for a lease with immediate 
		/// availability, and if one isn't found we wait for leases to become available in parallel. Task sources are specificially ordered to allow one to take precedence
		/// over another.
		/// 
		/// Performing the lease allocation in two steps allows for the following patterns:
		/// - A lease cannot be assigned immediately (due to dependencies on some limited external resource, or due to other leases already running on an agent), 
		///   but we should not assign any other leases to the agent either. In this situation, the method can just block immediately until the lease criteria are 
		///   satisfied.
		/// - The determination of a lease's availability requires some async setup, which can be done before the secondary task returns.
		/// </summary>
		/// <param name="agent">The agent to assign a lease to</param>
		/// <param name="cancellationToken">Cancellation token for the wait</param>
		/// <returns>Task returning a new lease object.</returns>
		Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken);

		/// <summary>
		/// Cancel a lease that was previously assigned to an agent, allowing it to be assigned out again
		/// </summary>
		/// <param name="agent">The agent that was assigned the lease</param>
		/// <param name="leaseId">The lease id</param>
		/// <param name="payload">Payload for the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task CancelLeaseAsync(IAgent agent, LeaseId leaseId, Any payload, CancellationToken cancellationToken);

		/// <summary>
		/// Notification that a lease has been started
		/// </summary>
		/// <param name="agent">The agent executing the lease</param>
		/// <param name="leaseId">The lease id</param>
		/// <param name="payload">Payload for the lease</param>
		/// <param name="logger">Logger for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task OnLeaseStartedAsync(IAgent agent, LeaseId leaseId, Any payload, ILogger logger, CancellationToken cancellationToken);

		/// <summary>
		/// Notification that a task has completed
		/// </summary>
		/// <param name="agent">The agent that was allocated to the lease</param>
		/// <param name="leaseId">The lease id</param>
		/// <param name="payload">The lease payload</param>
		/// <param name="outcome">Outcome of the lease</param>
		/// <param name="output">Output from the task</param>
		/// <param name="logger">Logger for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, Any payload, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken);

		/// <summary>
		/// Gets information to include for a lease in a lease info response
		/// </summary>
		/// <param name="payload">The lease payload</param>
		/// <param name="details">Properties for the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		ValueTask GetLeaseDetailsAsync(Any payload, Dictionary<string, string> details, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Base implementation of <see cref="ITaskSource"/>
	/// </summary>
	/// <typeparam name="TMessage"></typeparam>
	public abstract class TaskSourceBase<TMessage> : ITaskSource where TMessage : IMessage, new()
	{
		/// <summary>
		/// List of properties that can be output as log messages
		/// </summary>
		protected class PropertyList
		{
			internal StringBuilder _formatString = new StringBuilder();
			internal List<Func<TMessage, object>> _accessors = new List<Func<TMessage, object>>();
			internal List<(string, Func<TMessage, object>)> _jsonAccessors = new List<(string, Func<TMessage, object>)>();

			/// <summary>
			/// Adds a new property to the list
			/// </summary>
			/// <param name="expr">Accessor for the property</param>
			public PropertyList Add(Expression<Func<TMessage, object>> expr)
			{
				MemberInfo member = ((MemberExpression)expr.Body).Member;
				return Add(member.Name, expr.Compile());
			}

			/// <summary>
			/// Adds a new property to the list
			/// </summary>
			/// <param name="name">Name of the property</param>
			/// <param name="accessor">Accessor for the property</param>
			[System.Diagnostics.CodeAnalysis.SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase", Justification = "<Pending>")]
			public PropertyList Add(string name, Func<TMessage, object> accessor)
			{
				_formatString.Append(CultureInfo.InvariantCulture, $", {name}={{{name}}}");
				_accessors.Add(accessor);
				_jsonAccessors.Add((name[0..1].ToLower(CultureInfo.InvariantCulture) + name[1..], accessor));
				return this;
			}
		}

		static readonly TMessage s_message = new TMessage();

		/// <inheritdoc/>
		public abstract string Type { get; }

		/// <inheritdoc/>
		public abstract TaskSourceFlags Flags { get; }

		/// <inheritdoc/>
		protected virtual PropertyList OnLeaseStartedProperties { get; } = new PropertyList();

		/// <inheritdoc/>
		public MessageDescriptor Descriptor => s_message.Descriptor;

		/// <inheritdoc/>
		public abstract Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken);

		/// <inheritdoc/>
		public Task CancelLeaseAsync(IAgent agent, LeaseId leaseId, Any payload, CancellationToken cancellationToken) => CancelLeaseAsync(agent, leaseId, payload.Unpack<TMessage>(), cancellationToken);

		/// <inheritdoc/>
		public Task OnLeaseStartedAsync(IAgent agent, LeaseId leaseId, Any payload, ILogger logger, CancellationToken cancellationToken) => OnLeaseStartedAsync(agent, leaseId, payload.Unpack<TMessage>(), logger, cancellationToken);

		/// <inheritdoc/>
		public Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, Any payload, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken) => OnLeaseFinishedAsync(agent, leaseId, payload.Unpack<TMessage>(), outcome, output, logger, cancellationToken);

		/// <inheritdoc cref="ITaskSource.CancelLeaseAsync(IAgent, LeaseId, Any, CancellationToken)"/>
		public virtual Task CancelLeaseAsync(IAgent agent, LeaseId leaseId, TMessage payload, CancellationToken cancellationToken) => Task.CompletedTask;

		/// <inheritdoc cref="ITaskSource.OnLeaseStartedAsync(IAgent, LeaseId, Any, ILogger, CancellationToken)"/>
		public virtual Task OnLeaseStartedAsync(IAgent agent, LeaseId leaseId, TMessage payload, ILogger logger, CancellationToken cancellationToken)
		{
			object[] arguments = new object[2 + OnLeaseStartedProperties._accessors.Count];
			arguments[0] = leaseId;
			arguments[1] = Type;
			for (int idx = 0; idx < OnLeaseStartedProperties._accessors.Count; idx++)
			{
				arguments[idx + 2] = OnLeaseStartedProperties._accessors[idx](payload);
			}
#pragma warning disable CA2254 // Template should be a static expression
			logger.LogInformation($"Lease {{LeaseId}} started (Type={{Type}}{OnLeaseStartedProperties._formatString})", arguments);
#pragma warning restore CA2254 // Template should be a static expression
			return Task.CompletedTask;
		}

		/// <inheritdoc cref="ITaskSource.OnLeaseFinishedAsync(IAgent, LeaseId, Any, LeaseOutcome, ReadOnlyMemory{Byte}, ILogger, CancellationToken)"/>
		public virtual Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, TMessage payload, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("Lease {LeaseId} complete, outcome {LeaseOutcome}", leaseId, outcome);
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public virtual ValueTask GetLeaseDetailsAsync(Any payload, Dictionary<string, string> details, CancellationToken cancellationToken)
		{
			details["type"] = Type;

			if (OnLeaseStartedProperties._accessors.Count > 0)
			{
				TMessage message = payload.Unpack<TMessage>();
				foreach ((string name, Func<TMessage, object> getMethod) in OnLeaseStartedProperties._jsonAccessors)
				{
					details[name] = getMethod(message)?.ToString() ?? String.Empty;
				}
			}

			return default;
		}

		/// <summary>
		/// Creates a lease task which will wait until the given cancellation token is signalled.
		/// </summary>
		/// <param name="token">The cancellation token</param>
		/// <returns>Lease task</returns>
		protected static Task<CreateLeaseOptions?> SkipAsync(CancellationToken token)
		{
			_ = token;
			return Task.FromResult<CreateLeaseOptions?>(null);
		}

		/// <summary>
		/// Waits until the cancellation token is signalled, then return an cancelled lease task.
		/// </summary>
		/// <param name="token">The cancellation token</param>
		/// <returns>Lease task</returns>
		protected static async Task<Task<CreateLeaseOptions?>> DrainAsync(CancellationToken token)
		{
			await token.AsTask();
			return Task.FromResult<CreateLeaseOptions?>(null);
		}

		/// <summary>
		/// Creates a lease task from a given lease
		/// </summary>
		/// <param name="lease">Lease to create the task from</param>
		/// <returns></returns>
		protected static Task<CreateLeaseOptions?> LeaseAsync(CreateLeaseOptions lease)
		{
			return Task.FromResult<CreateLeaseOptions?>(lease);
		}
	}

	/// <summary>
	/// Service which dispatches lease complete notifications to task sources
	/// </summary>
	class TaskSourceNotificationService : IHostedService, IAsyncDisposable
	{
		readonly IAgentCollection _agentCollection;
		readonly LeaseCollection _leaseCollection;
		readonly ITaskSource[] _taskSources;
		readonly AsyncQueue _asyncQueue;
		readonly ILogger _logger;

		public TaskSourceNotificationService(IAgentCollection agentCollection, LeaseCollection leaseCollection, IEnumerable<ITaskSource> taskSources, ILogger<TaskSourceNotificationService> logger)
		{
			_agentCollection = agentCollection;
			_leaseCollection = leaseCollection;
			_taskSources = taskSources.ToArray();
			_asyncQueue = new AsyncQueue(1);
			_logger = logger;
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			_leaseCollection.OnLeaseComplete += OnLeaseComplete;
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			_leaseCollection.OnLeaseComplete -= OnLeaseComplete;
			await _asyncQueue.StopAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
			=> await _asyncQueue.DisposeAsync();

		void OnLeaseComplete(ILease lease)
		{
			if(lease.Payload.Length > 0)
			{
				Any payload = Any.Parser.ParseFrom(lease.Payload.Span);
				foreach (ITaskSource taskSource in _taskSources)
				{
					if (payload.Is(taskSource.Descriptor))
					{
						_asyncQueue.Enqueue(ctx => HandleCompleteLeaseAsync(taskSource, lease, payload, ctx));
						break;
					}
				}
			}
		}

		async Task HandleCompleteLeaseAsync(ITaskSource taskSource, ILease lease, Any payload, CancellationToken cancellationToken)
		{
			IAgent? agent = await _agentCollection.GetAsync(lease.AgentId, cancellationToken);
			if (agent != null)
			{
				_logger.LogInformation("Removing lease {LeaseId} ({LeaseType})", lease.Id, payload.TypeUrl);
				await taskSource.OnLeaseFinishedAsync(agent, lease.Id, payload, lease.Outcome, lease.Output, _agentCollection.GetLogger(agent.Id), cancellationToken);
			}
		}
	}
}
