// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Logs;
using HordeServer.Utilities;

namespace HordeServer.Tasks
{
	class RestartTaskSource : TaskSourceBase<RestartTask>
	{
		public override string Type => "Restart";

		public override TaskSourceFlags Flags => TaskSourceFlags.AllowWhenDisabled | TaskSourceFlags.AllowDuringDowntime;

		readonly ILogCollection _logCollection;

		public RestartTaskSource(ILogCollection logCollection)
		{
			_logCollection = logCollection;

			OnLeaseStartedProperties.Add(nameof(RestartTask.LogId), x => LogId.Parse(x.LogId));
		}

		public override async Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (agent.Mode == AgentMode.Workstation)
			{
				return SkipAsync(cancellationToken);
			}
			
			if (!agent.RequestForceRestart)
			{
				if (!agent.RequestRestart)
				{
					return SkipAsync(cancellationToken);
				}
				if (agent.Leases.Count > 0)
				{
					return await DrainAsync(cancellationToken);
				}
			}

			LeaseId leaseId = new LeaseId(BinaryIdUtils.CreateNew());
			ILog log = await _logCollection.AddAsync(JobId.Empty, leaseId, agent.SessionId, LogType.Json, cancellationToken: cancellationToken);

			RestartTask task = new RestartTask();
			task.LogId = log.Id.ToString();

			return LeaseAsync(new CreateLeaseOptions(leaseId, null, "Restart", null, null, log.Id, null, true, task));
		}
	}
}
