// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Logs;
using HordeServer.Utilities;

namespace HordeServer.Tasks
{
	class ShutdownTaskSource : TaskSourceBase<ShutdownTask>
	{
		public override string Type => "Shutdown";

		public override TaskSourceFlags Flags => TaskSourceFlags.AllowWhenDisabled | TaskSourceFlags.AllowDuringDowntime;

		readonly ILogCollection _logCollection;

		public ShutdownTaskSource(ILogCollection logCollection)
		{
			_logCollection = logCollection;
			OnLeaseStartedProperties.Add(nameof(ShutdownTask.LogId), x => LogId.Parse(x.LogId));
		}

		public override async Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (!agent.RequestShutdown || agent.Mode == AgentMode.Workstation)
			{
				return SkipAsync(cancellationToken);
			}
			if (agent.Leases.Count > 0)
			{
				return await DrainAsync(cancellationToken);
			}

			LeaseId leaseId = new LeaseId(BinaryIdUtils.CreateNew());
			ILog log = await _logCollection.AddAsync(JobId.Empty, leaseId, agent.SessionId, LogType.Json, cancellationToken: cancellationToken);

			ShutdownTask task = new ShutdownTask();
			task.LogId = log.Id.ToString();

			return LeaseAsync(new CreateLeaseOptions(leaseId, null, "Shutdown", null, null, log.Id, null, true, task));
		}
	}
}
