// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Tools;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Logs;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;

namespace HordeServer.Tasks
{
	class UpgradeTaskSource : TaskSourceBase<UpgradeTask>
	{
		public override string Type => "Upgrade";

		public override TaskSourceFlags Flags => TaskSourceFlags.AllowWhenDisabled | TaskSourceFlags.AllowDuringDowntime;

		readonly IToolCollection _toolCollection;
		readonly ILogCollection _logCollection;
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly IOptions<ComputeServerConfig> _staticComputeConfig;
		readonly IClock _clock;

		public UpgradeTaskSource(IToolCollection toolCollection, ILogCollection logCollection, IOptionsMonitor<ComputeConfig> computeConfig, IOptions<ComputeServerConfig> staticComputeConfig, IClock clock)
		{
			_toolCollection = toolCollection;
			_logCollection = logCollection;
			_computeConfig = computeConfig;
			_staticComputeConfig = staticComputeConfig;
			_clock = clock;

			OnLeaseStartedProperties.Add(nameof(UpgradeTask.LogId), x => LogId.Parse(x.LogId));
		}

		public override async Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (!_staticComputeConfig.Value.EnableUpgradeTasks)
			{
				return SkipAsync(cancellationToken);
			}
			
			if (!agent.IsAutoUpdateEnabled())
			{
				return SkipAsync(cancellationToken);
			}

			(ITool, IToolDeployment)? required = await GetRequiredSoftwareVersionAsync(agent, cancellationToken);
			if (required == null)
			{
				return SkipAsync(cancellationToken);
			}

			(ITool tool, IToolDeployment deployment) = required.Value;
			if (agent.Version == deployment.Version)
			{
				return SkipAsync(cancellationToken);
			}

			if (agent.Leases.Count > 0 || (agent.LastUpgradeTime != null && agent.LastUpgradeVersion == deployment.Version && _clock.UtcNow < agent.LastUpgradeTime.Value + TimeSpan.FromMinutes(5.0)))
			{
				return await DrainAsync(cancellationToken);
			}

			LeaseId leaseId = new LeaseId(BinaryIdUtils.CreateNew());
			ILog log = await _logCollection.AddAsync(JobId.Empty, leaseId, agent.SessionId, LogType.Json, cancellationToken: cancellationToken);

			UpgradeTask task = new UpgradeTask();
			task.SoftwareId = $"{tool.Id}:{deployment.Version}";
			task.LogId = log.Id.ToString();

			return LeaseAsync(new CreateLeaseOptions(leaseId, null, $"Upgrade to {tool.Id} {deployment.Version}", null, null, log.Id, null, true, task));
		}

		/// <summary>
		/// Determines the client software version that should be installed on an agent
		/// </summary>
		/// <param name="agent">The agent instance</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Unique id of the client version this agent should be running</returns>
		public async Task<(ITool, IToolDeployment)?> GetRequiredSoftwareVersionAsync(IAgent agent, CancellationToken cancellationToken)
		{
			ComputeConfig computeConfig = _computeConfig.CurrentValue;

			ToolId toolId = agent.GetSoftwareToolId(computeConfig);

			ITool? tool = await _toolCollection.GetAsync(toolId, cancellationToken);
			if (tool == null)
			{
				return null;
			}

			uint value = BuzHash.Add(0, Encoding.UTF8.GetBytes(agent.Id.ToString()));
			double phase = (value % 10000) / 10000.0;

			IToolDeployment? deployment = tool.GetCurrentDeployment(phase, _clock.UtcNow);
			if (deployment == null)
			{
				return null;
			}

			return (tool, deployment);
		}
	}
}
