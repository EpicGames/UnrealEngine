// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Tests;
using Microsoft.Extensions.DependencyInjection;
namespace HordeServer.Example.Plugin.Tests
{
	/// <summary>
	/// Handles set up of collections, services, fixtures etc during testing
	/// </summary>
	public class ExamplePluginTestSetup : BuildTestSetup
	{
		public ExamplePluginTestSetup()
		{
			AddPlugin<ExamplePlugin>();
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

		}
	}
}