// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using HordeServer.Tests;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using HordeServer.Server;

namespace HordeServer.Example.Plugin.Tests
{

	[TestClass]
	public class ExamplePluginTests : BuildTestSetup
	{
		private readonly string _greetingMessage = "Test Greeting";

		private ExamplePluginController GetExamplePluginController()
		{
			ExamplePluginController controller = ActivatorUtilities.CreateInstance<ExamplePluginController>(ServiceProvider);
			controller.ControllerContext = GetControllerContext();
			return controller;
		}

		private static ControllerContext GetControllerContext()
		{
			ControllerContext controllerContext = new ControllerContext();
			controllerContext.HttpContext = new DefaultHttpContext();
			controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
				new List<Claim> { HordeClaims.AdminClaim.ToClaim() }, "TestAuthType"));
			return controllerContext;
		}

		public ExamplePluginTests()
		{
			AddPlugin<ExamplePlugin>();

			ExamplePluginConfig examplePluginConfig = new ExamplePluginConfig();
			examplePluginConfig.GreetingMessage = _greetingMessage;

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddExamplePluginConfig(examplePluginConfig);

			SetConfigAsync(globalConfig).Wait();

		}

		[TestMethod]
		public void TestGreeting()
		{
			ExamplePluginController controller = GetExamplePluginController();

			ContentResult greeting = (controller.GetGreeting() as ContentResult)!;

			Assert.AreEqual(greeting.Content, _greetingMessage);
		}
	}
}

