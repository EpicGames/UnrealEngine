// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the ExamplePlugin
	/// </summary>
	[Plugin("ExamplePlugin", EnabledByDefault = false, GlobalConfigType = typeof(ExamplePluginConfig), ServerConfigType = typeof(ExamplePluginServerConfig))]
	public class ExamplePlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ExamplePlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			
		}
	}

	/// <summary>
	/// Helper methods for ExamplePlugin plugin config
	/// </summary>
	public static class ExamplePluginExtensions
	{
		/// <summary>
		/// Configures the ExamplePlugin plugin
		/// </summary>
		public static void AddExamplePluginConfig(this IDictionary<PluginName, IPluginConfig> dictionary, ExamplePluginConfig examplePluginConfig)
			=> dictionary[new PluginName("ExamplePlugin")] = examplePluginConfig;

		/// <summary>
		/// Gets configuration for the ExamplePlugin plugin
		/// </summary>
		public static ExamplePluginConfig GetExamplePluginConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (ExamplePluginConfig)dictionary[new PluginName("ExamplePlugin")];
	}
}
