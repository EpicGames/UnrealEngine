// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the ExamplePlugin plugin
	/// </summary>
	public class ExamplePluginConfig : IPluginConfig
	{
		/// <summary>
		/// The Greeting provided by the example plugin
		/// </summary>
		public string GreetingMessage { get; set; } = "Default Greeting";

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{			
		}
	}
}
