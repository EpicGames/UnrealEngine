// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer
{
	/// <summary>
	/// Controller example for ExamplePlugun
	/// </summary>
	
	[Authorize]
	[ApiController]	
	public class ExamplePluginController : HordeControllerBase
	{
		readonly IOptionsSnapshot<ExamplePluginConfig> _examplePluginConfig;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ExamplePluginController(IOptionsSnapshot<ExamplePluginConfig> examplePluginConfig, ILogger<ExamplePluginController> logger)
		{
			_examplePluginConfig = examplePluginConfig;
			_logger = logger;
		}

		/// <summary>
		/// Example plugin endpoint which returns message provided in 
		/// </summary>
		[HttpGet]
		[Route("/api/v1/exampleplugin/greeting")]
		public ActionResult GetGreeting()
		{
			string greeting = _examplePluginConfig.Value.GreetingMessage;
			_logger.LogInformation("Getting Greeting: {Greeting}", greeting);
			return Content(greeting, "text/plain");
		}
	}
}
