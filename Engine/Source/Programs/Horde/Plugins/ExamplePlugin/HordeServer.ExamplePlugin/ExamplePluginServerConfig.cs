// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Server configuration for the ExamplePlugin 
	/// </summary>
	public class ExamplePluginServerConfig : PluginServerConfig
	{
	}
}
