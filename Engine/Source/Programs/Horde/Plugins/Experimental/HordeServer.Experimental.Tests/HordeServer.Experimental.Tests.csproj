<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer.Tests</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Testing" Version="10.0.6" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Build\HordeServer.Build\HordeServer.Build.csproj" />
    <ProjectReference Include="..\..\Build\HordeServer.Build.Tests\HordeServer.Build.Tests.csproj" />
    <ProjectReference Include="..\HordeServer.Experimental\HordeServer.Experimental.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Tests.Shared\HordeServer.Tests.Shared.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>

</Project>
