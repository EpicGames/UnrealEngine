// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Experimental.Notifications;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.Notifications.Sinks;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Tests;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Moq;

namespace HordeServer.Experimental.Tests.Notifications
{
	class TemplateStub(ContentHash hash, string name) : ITemplate
	{
		public ContentHash Hash { get; } = hash;
		public string Name { get; } = name;
		public string? Description { get; init; } = null;
		public Priority? Priority { get; init; } = null;
		public bool AllowPreflights { get; init; } = false;
		public bool UpdateIssues { get; init; } = false;
		public bool PromoteIssuesByDefault { get; init; } = false;
		public string? InitialAgentType { get; init; } = null;
		public string? SubmitNewChange { get; init; } = null;
		public string? SubmitDescription { get; init; } = null;
		public IReadOnlyList<string> Arguments { get; init; } = [];
		public IReadOnlyList<ITemplateParameter> Parameters { get; init; } = [];
	}

	struct JobNotificationState
	{
		public JobNotificationState(JobId jobId, TemplateId templateId, string channel)
		{
			JobId = jobId;
			TemplateId = templateId;
			Channel = channel;
		}

		public JobId JobId { get; }
		public TemplateId TemplateId { get; }
		public string Channel { get; }
	}

	public class JobNotificationInformation
	{
		public IJob Job { get; }
		public IJobStep? Step { get; } = null;
		public List<ILogEventData>? JobStepEventData { get; } = null;
		public IEnumerable<IUser>? UsersToNotify { get; } = null;
		public JobNotificationInformation(IJob job, IJobStep? step = null, List<ILogEventData>? jobStepEventData = null, IEnumerable<IUser>? usersToNotify = null)
		{
			Job = job;
			Step = step;
			JobStepEventData = jobStepEventData;
			UsersToNotify = usersToNotify;
		}
	}

	public class FakeExperimentalNotificationSink : INotificationSink, IDisposable
	{
		readonly SlackNotificationProcessor _slackNotificationProcessor;

		public JobNotificationInformation? ProcessedJobNotification { get; set; } = null;

		public Task NotifyJobScheduledAsync(List<JobScheduledNotification> notifications, CancellationToken cancellationToken) { throw new NotImplementedException(); }

		public async Task NotifyJobStepCompleteAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			// Currently we only want to notify from the plugin's configuration which would not have this set
			if (usersToNotify is not null)
			{
				return;
			}

			await _slackNotificationProcessor.ProcessJobStepCompleteAsync(job, step, jobStepEventData, cancellationToken);
			ProcessedJobNotification = new JobNotificationInformation(job, step, jobStepEventData, usersToNotify);
		}

		public async Task NotifyJobStepAbortedAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			await _slackNotificationProcessor.ProcessJobStepAbortedAsync(job, step, cancellationToken);
			ProcessedJobNotification = new JobNotificationInformation(job, step, jobStepEventData, usersToNotify);
		}

		public async Task NotifyJobCompleteAsync(IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			await _slackNotificationProcessor.ProcessJobCompleteAsync(job, cancellationToken);
			ProcessedJobNotification = new JobNotificationInformation(job, null, null, null);
		}

		public Task NotifyJobCompleteAsync(IUser user, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			// Not handled, but we don't want to throw an exception
			return Task.CompletedTask;
		}

		public Task NotifyLabelCompleteAsync(IUser user, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> stepData, CancellationToken cancellationToken) { throw new NotImplementedException(); }
		public Task NotifyIssueUpdatedAsync(IIssue issue, CancellationToken cancellationToken) { throw new NotImplementedException(); }
		public Task NotifyConfigUpdateAsync(ConfigUpdateInfo info, CancellationToken cancellationToken) => Task.CompletedTask;
		public Task NotifyConfigUpdateFailureAsync(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
		public Task NotifyDeviceServiceAsync(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? stream = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
		public Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task SendSessionConflictReportAsync(IReadOnlyList<(AgentId Id, int Count)> conflicts, CancellationToken cancellationToken) => Task.CompletedTask;
		public Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken = default) => throw new NotImplementedException();
		public Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken = default) => throw new NotImplementedException();
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				_slackNotificationProcessor.Dispose();
			}
		}

		public FakeExperimentalNotificationSink(IOptions<BuildServerConfig> buildServerConfig,
			IOptionsMonitor<BuildConfig> buildConfig,
			IOptionsMonitor<ExperimentalConfig> experimentalConfig,
			IJobNotificationCollection jobNotificationCollection,
			IJobCollection jobCollection,
			IUserCollection userCollection,
			IServerInfo serverInfo,
			ILogger<FakeExperimentalNotificationSink> logger)
		{
			_slackNotificationProcessor = new SlackNotificationProcessor(buildServerConfig, buildConfig, experimentalConfig, jobNotificationCollection, jobCollection, userCollection, serverInfo, logger);
		}
	}

	[TestClass]
	public class ExperimentalNotificationTests : ExperimentalTestSetup
	{
		const string MainStreamName = "//UE5/Main";
		readonly StreamId _mainStreamId = new StreamId(StringId.Sanitize(MainStreamName));

		readonly TemplateId _buildAndTestTemplateId = new TemplateId("test-build-test-job-template");
		readonly TemplateId _buildTemplateId = new TemplateId("test-build-job-template");
		readonly TemplateId _spawnedTestTemplateId = new TemplateId("test-spawned-job-template");

		readonly StreamConfig _mainStreamConfig;

		readonly TemplateRefConfig _testBuildTestJobTemplate;
		readonly TemplateRefConfig _testBuildJobTemplate;
		readonly TemplateRefConfig _testSpawnedJobTemplate;

		const string ChannelId = "TEST_CHANNEL";
		const string SpawnJobTestGroup = "Spawn Test Job";
		const string ClientBootTestGroup = "Client Boot Test";
		const string ClientProjectTestGroup = "Client Project Tests";
		const string EditorBootTestGroup = "Editor Boot Test";
		const string EditorProjectTestGroup = "Editor Project Tests";

		/// <summary>
		/// Slack emoji used to emphasize state
		/// </summary>
		const string PendingBadge = ":large_blue_square:";
		const string WarningBadge = ":large_yellow_square:";
		const string FailureBadge = ":large_red_square:";
		const string SkippedBadge = ":black_square:";
		const string PassingBadge = ":large_green_square:";

		readonly List<JobNotificationState> _testJobNotifications = new List<JobNotificationState>();

		public ExperimentalNotificationTests()
		{
			_testBuildTestJobTemplate = new TemplateRefConfig { Id = _buildAndTestTemplateId };
			_testBuildJobTemplate = new TemplateRefConfig { Id = _buildTemplateId };
			_testSpawnedJobTemplate = new TemplateRefConfig { Id = _spawnedTestTemplateId };

			_mainStreamConfig = CreateStream(_mainStreamId, MainStreamName);

			ProjectConfig projectConfig = new ProjectConfig { Id = new ProjectId("ue5"), Name = "UE5" };
			projectConfig.Streams.Add(_mainStreamConfig);

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			SetConfigAsync(globalConfig).Wait();
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			// Replace the ExperimentalSlackNotificationSink with the one used here for our tests
			// Otherwise, we'll be adding duplicate records to our database
			ServiceDescriptor descriptor = services.First(x => x.ImplementationType == typeof(ExperimentalSlackNotificationSink));
			services.Remove(descriptor);

			services.AddSingleton<FakeExperimentalNotificationSink>();
			services.AddSingleton<INotificationSink>(sp => sp.GetRequiredService<FakeExperimentalNotificationSink>());
		}

		StreamConfig CreateStream(StreamId streamId, string streamName)
		{
			StreamConfig streamConfig = new StreamConfig { Id = streamId, Name = streamName };
			streamConfig.StreamTags.Add(new StreamTag { Name = "NotificationEnabledTag", Enabled = true });
			streamConfig.StreamTags.Add(new StreamTag { Name = "DisabledTag", Enabled = false });
			streamConfig.Tabs.Add(new TabConfig { Title = "General", Templates = new List<TemplateId> { _buildAndTestTemplateId, _buildTemplateId, _spawnedTestTemplateId } });
			streamConfig.Templates.Add(_testBuildTestJobTemplate);
			streamConfig.Templates.Add(_testBuildJobTemplate);
			streamConfig.Templates.Add(_testSpawnedJobTemplate);
			return streamConfig;
		}

		void UpdateGlobalConfig(ExperimentalConfig experimentalConfig)
		{
			UpdateConfigAsync(globalConfig =>
			{
				globalConfig.Plugins.AddExperimentalConfig(experimentalConfig);
			}).Wait();
		}

		ExperimentalConfig CreateExperimentalNotificationConfig()
		{
			NotificationFormatConfig editorNotificationFormatConfig = CreateNotificationFormatConfig(EditorBootTestGroup, "^Editor BootTest\\s");
			NotificationFormatConfig clientNotificationFormatConfig = CreateNotificationFormatConfig(ClientBootTestGroup, "^Client BootTest\\s");

			JobNotificationConfig jobNotificationConfig = new JobNotificationConfig();
			jobNotificationConfig.Enabled = true;
			jobNotificationConfig.Template = _buildAndTestTemplateId;
			jobNotificationConfig.NamePattern = "Platform1|Platform3";
			jobNotificationConfig.NotificationFormats.Add(editorNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientNotificationFormatConfig);
			jobNotificationConfig.Channels.Add(ChannelId);

			NotificationStreamConfig notificationStreamConfig = new NotificationStreamConfig();
			notificationStreamConfig.Streams.Add(_mainStreamId);
			notificationStreamConfig.JobNotifications.Add(jobNotificationConfig);

			NotificationConfig notificationConfig = new NotificationConfig();
			notificationConfig.Id = new NotificationConfigId("test-notification");
			notificationConfig.NotificationStreams.Add(notificationStreamConfig);

			ExperimentalConfig experimentalConfig = new ExperimentalConfig();
			experimentalConfig.Notifications.Add(notificationConfig);

			return experimentalConfig;
		}

		static NotificationFormatConfig CreateNotificationFormatConfig(string group, string stepPattern, string? alternateNamePattern = null)
		{
			NotificationFormatConfig notificationFormatConfig = new NotificationFormatConfig();
			notificationFormatConfig.Group = group;
			notificationFormatConfig.StepPattern = stepPattern;
			notificationFormatConfig.AlternativeNamePattern = alternateNamePattern;
			return notificationFormatConfig;
		}

		static NewGroup AddGroup(List<NewGroup> groups, string agentType = "win64")
		{
			NewGroup group = new NewGroup(agentType, new List<NewNode>());
			groups.Add(group);
			return group;
		}

		static void AddNode(NewGroup group, string name, string[]? inputDependencies, Action<NewNode>? action = null)
		{
			NewNode node = new NewNode(name, inputDependencies: inputDependencies?.ToList(), orderDependencies: inputDependencies?.ToList());
			action?.Invoke(node);
			group.Nodes.Add(node);
		}

		static void AddPlatformTestSteps(NewGroup group, string platform, List<string> targetArguments, string[]? editorInputDependencies = null, string[]? clientInputDependencies = null)
		{
			AddNode(group, $"Editor BootTest {platform}", editorInputDependencies);
			AddNode(group, $"Editor Project Tests {platform}", new[] { $"Editor BootTest {platform}"});
			targetArguments.Add($"-Target=Editor Project Tests {platform}");

			AddNode(group, $"Client BootTest {platform}", clientInputDependencies);
			AddNode(group, $"Client Project Tests {platform}", new[] { $"Client BootTest {platform}"});
			targetArguments.Add($"-Target=Client Project Tests {platform}");
		}

		async Task<IJob> CreateNightlyBuildTestJobAsync(StreamConfig config, int commitId)
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Update Version Files", null);
			AddNode(group, "Compile Editor", new[] { "Update Version Files" });
			AddNode(group, "Compile Client", new[] { "Update Version Files" });
			AddNode(group, "Cook Client", new[] { "Compile Editor" });

			string[] editorInputDependencies = new[] { "Compile Editor" };
			string[] clientInputDependencies = new[] { "Compile Client", "Cook Client" };
			AddPlatformTestSteps(group, "Platform1", options.Arguments, editorInputDependencies, clientInputDependencies);
			AddPlatformTestSteps(group, "Platform2", options.Arguments, editorInputDependencies, clientInputDependencies);
			AddPlatformTestSteps(group, "Platform3", options.Arguments, editorInputDependencies, clientInputDependencies);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups);

			ITemplate template = new TemplateStub(ContentHash.MD5("graphHash"), _buildAndTestTemplateId.ToString());
			return await JobService.CreateJobAsync(null, config, _buildAndTestTemplateId, template.Hash, graph, "Test job", CommitIdWithOrder.FromPerforceChange(commitId), CommitIdWithOrder.FromPerforceChange(commitId), options);
		}

		async Task<IJob> CreateNightlyBuildJobAsync(StreamConfig config, int commitId)
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Spawn Platform1 Tests");
			options.Arguments.Add("-Target=Spawn Platform2 Tests");
			options.Arguments.Add("-Target=Spawn Platform3 Tests");

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Spawn Platform1 Tests", null);
			AddNode(group, "Spawn Platform2 Tests", null);
			AddNode(group, "Spawn Platform3 Tests", null);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups);

			ITemplate template = new TemplateStub(ContentHash.MD5("graphHash"), _buildTemplateId.ToString());
			return await JobService.CreateJobAsync(null, config, _buildTemplateId, template.Hash, graph, "Test Build job", CommitIdWithOrder.FromPerforceChange(commitId), CommitIdWithOrder.FromPerforceChange(commitId), options);
		}

		async Task<IJob> SpawnNightlyBuildTestJobAsync(StreamConfig config, JobId parentJobId, JobStepId parentJobStepId, string platform, int commitId)
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.ParentJobId = parentJobId;
			options.ParentJobStepId = parentJobStepId;

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddPlatformTestSteps(group, platform, options.Arguments);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups);

			ITemplate template = new TemplateStub(ContentHash.MD5("graphHash"), _spawnedTestTemplateId.ToString());
			return await JobService.CreateJobAsync(null, config, _spawnedTestTemplateId, template.Hash, graph, $"Test job - {platform}", CommitIdWithOrder.FromPerforceChange(commitId), CommitIdWithOrder.FromPerforceChange(commitId), options);
		}

		async Task<IJob> AbortJobAsync(IJob job)
		{
			job = Deref(await JobService.UpdateJobAsync(job, abortedByUserId: UserId.Anonymous));
			Assert.AreEqual(JobState.Complete, job.GetState());
			return job;
		}

		Task<IJob> RunBatchAsync(StreamConfig config, IJob job, int batchIdx)
		{
			return UpdateBatchAsync(config, job, batchIdx, JobStepBatchState.Running);
		}

		async Task<IJob> UpdateBatchAsync(StreamConfig config, IJob job, int batchIdx, JobStepBatchState state)
		{
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[batchIdx].State);

			LogId logId = LogIdUtils.GenerateNewId();
			job = Deref(await job.TryAssignLeaseAsync(batchIdx, new PoolId("foo"), new AgentId("agent"), new SessionId(BinaryIdUtils.CreateNew()), new LeaseId(BinaryIdUtils.CreateNew()), logId));
			if (job.Batches[batchIdx].State != state)
			{
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[batchIdx].Id, config, logId, state));
				Assert.AreEqual(state, job.Batches[batchIdx].State);
			}
			return job;
		}

		async Task<IJob> AbortStepAsync(StreamConfig config, IJob job, int batchIdx, int stepIdx)
		{
			LogId logId = LogIdUtils.GenerateNewId();
			await LogCollection.AddAsync(job.Id, job.Batches[batchIdx].LeaseId, job.Batches[batchIdx].SessionId, LogType.Json, logId);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, config, newAbortRequested: true, newAbortByUserId: UserId.Anonymous, newLogId: logId));
			Assert.AreEqual(JobStepState.Aborted, job.Batches[batchIdx].Steps[stepIdx].State);
			return job;
		}

		async Task<IJob> AddSpawnedJobToParentStepAsync(StreamConfig config, IJob job, int batchIdx, int stepIdx, JobId spawnedJobId)
		{
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, config, newSpawnedJob: spawnedJobId));
			Assert.IsNotNull(job.Batches[batchIdx].Steps[stepIdx].SpawnedJobs);
			Assert.IsTrue(job.Batches[batchIdx].Steps[stepIdx].SpawnedJobs!.Contains(spawnedJobId));
			return job;
		}

		async Task<IJob> RunStepAsync(StreamConfig config, IJob job, int batchIdx, int stepIdx, JobStepOutcome outcome)
		{
			Assert.AreEqual(JobStepState.Ready, job.Batches[batchIdx].Steps[stepIdx].State);

			LogId logId = LogIdUtils.GenerateNewId();
			await LogCollection.AddAsync(job.Id, job.Batches[batchIdx].LeaseId, job.Batches[batchIdx].SessionId, LogType.Json, logId);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, config, JobStepState.Running, JobStepOutcome.Success, newLogId: logId));
			Assert.AreEqual(JobStepState.Running, job.Batches[batchIdx].Steps[stepIdx].State);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, config, JobStepState.Completed, outcome, newLogId: logId));
			Assert.AreEqual(JobStepState.Completed, job.Batches[batchIdx].Steps[stepIdx].State);
			Assert.AreEqual(outcome, job.Batches[batchIdx].Steps[stepIdx].Outcome);
			return job;
		}

		static void ValidateJobStepInformationAsync(IJobStepNotificationStateRef jobStepNotificationState, JobId jobId, TemplateId templateId, string recipient, string group, string platform, string badge, JobId? parentJobId = null, TemplateId? parentJobTemplateId = null)
		{
			Assert.AreEqual(jobId, jobStepNotificationState.JobId);
			Assert.AreEqual(templateId, jobStepNotificationState.TemplateId);
			Assert.AreEqual(recipient, jobStepNotificationState.Recipient);
			Assert.AreEqual(group, jobStepNotificationState.Group);
			Assert.AreEqual(platform, jobStepNotificationState.TargetPlatform);
			Assert.AreEqual(badge, jobStepNotificationState.Badge);
			Assert.AreEqual(parentJobId, jobStepNotificationState.ParentJobId);
			Assert.AreEqual(parentJobTemplateId, jobStepNotificationState.ParentJobTemplateId);
		}

		static async Task ValidateJobStepInformationAsync(JobNotificationCollection jobNotificationCollection, JobId jobId, TemplateId templateId, string recipient, JobStepId jobStepId, string group, string platform, string badge, JobId? parentJobId = null, TemplateId? parentJobTemplateId = null)
		{
			IJobStepNotificationStateQueryBuilder queryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			queryBuilder.AddJobFilter(jobId);
			queryBuilder.AddJobStepFilter(jobStepId);
			queryBuilder.AddTemplateFilter(templateId);
			queryBuilder.AddRecipientFilter(recipient);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(queryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.AreEqual(1, jobStepNotificationStates.Count);

			ValidateJobStepInformationAsync(jobStepNotificationStates[0], jobId, templateId, recipient, group, platform, badge, parentJobId, parentJobTemplateId);
		}

		static async Task WaitForNotificationAsync(FakeExperimentalNotificationSink sink, IJob job, IJobStep? jobStep, int timeoutMs = 30000)
		{
			DateTime endTime = DateTime.Now + TimeSpan.FromMilliseconds(timeoutMs);

			while (endTime >= DateTime.Now)
			{
				if (sink.ProcessedJobNotification is not null && sink.ProcessedJobNotification.Job == job && sink.ProcessedJobNotification.Step == jobStep)
				{
					return;
				}
				await Task.Delay(100);
			}
			Assert.Fail("Expected notification was not triggered within the time allotted.");
		}

		[TestCleanup]
		public async Task TestCleanupAsync()
		{
			if (_testJobNotifications.Count == 0)
			{
				return;
			}

			// Cleanup all database records
			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			foreach (JobNotificationState jobNotificationState in _testJobNotifications)
			{
				await jobNotificationCollection.DeleteJobStepNotificationStatesAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Channel);
				await jobNotificationCollection.DeleteJobNotificationStatesAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Channel);
			}

			_testJobNotifications.Clear();
		}

		[TestMethod]
		public async Task SetupBuildJobStepFailedTestAsync()
		{
			// Scenario: Job matched with the configuration, but due to the Setup Build failing, nothing should be recorded as a Slack message would've been sent and the flow would be completed
			// Expected: No records should exist in the JobNotificationCollection

			ExperimentalConfig experimentalConfig = CreateExperimentalNotificationConfig();
			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob job = await CreateNightlyBuildTestJobAsync(_mainStreamConfig, 10);
			_testJobNotifications.Add(new JobNotificationState(job.Id, job.TemplateId, ChannelId));

			job = await RunBatchAsync(_mainStreamConfig, job, 0);
			job = await RunStepAsync(_mainStreamConfig, job, 0, 0, JobStepOutcome.Failure); // Setup Build

			await WaitForNotificationAsync(fakeSink, job, job.Batches[0].Steps[0]);
			
			// There should be nothing in the collection as a failed job has nothing to store
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(job.Id);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNull(jobNotificationStates);

			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNull(jobStepNotificationStates);
		}

		[TestMethod]
		public async Task JobNotificationTestAsync()
		{
			// Scenario: Job matched with the configuration and recorded a job with its steps
			// Expected: 1 job and 4 job step records with the job step records sorted in order by group, and platform

			NotificationFormatConfig editorNotificationFormatConfig = CreateNotificationFormatConfig(EditorBootTestGroup, "^Editor BootTest\\s");
			NotificationFormatConfig clientNotificationFormatConfig = CreateNotificationFormatConfig(ClientBootTestGroup, "^Client BootTest\\s", "Platform1");
			NotificationFormatConfig editorProjectNotificationFormatConfig = CreateNotificationFormatConfig(EditorProjectTestGroup, "^Editor Project Tests\\s");
			NotificationFormatConfig clientProjectNotificationFormatConfig = CreateNotificationFormatConfig(ClientProjectTestGroup, "^Client Project Tests\\s", "Platform1");

			JobNotificationConfig jobNotificationConfig = new JobNotificationConfig();
			jobNotificationConfig.Enabled = true;
			jobNotificationConfig.Template = _buildAndTestTemplateId;
			jobNotificationConfig.NamePattern = "Platform1|Platform2";
			jobNotificationConfig.NotificationFormats.Add(editorNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(editorProjectNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientProjectNotificationFormatConfig);
			jobNotificationConfig.Channels.Add(ChannelId);

			NotificationStreamConfig notificationStreamConfig = new NotificationStreamConfig();
			notificationStreamConfig.Streams.Add(_mainStreamId);
			notificationStreamConfig.JobNotifications.Add(jobNotificationConfig);

			NotificationConfig notificationConfig = new NotificationConfig();
			notificationConfig.Id = new NotificationConfigId("test-notification");
			notificationConfig.NotificationStreams.Add(notificationStreamConfig);

			ExperimentalConfig experimentalConfig = new ExperimentalConfig();
			experimentalConfig.Notifications.Add(notificationConfig);

			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob job = await CreateNightlyBuildTestJobAsync(_mainStreamConfig, 20);
			_testJobNotifications.Add(new JobNotificationState(job.Id, job.TemplateId, ChannelId));

			job = await RunBatchAsync(_mainStreamConfig, job, 0);
			job = await RunStepAsync(_mainStreamConfig, job, 0, 0, JobStepOutcome.Success); // Setup Build

			await WaitForNotificationAsync(fakeSink, job, job.Batches[0].Steps[0]);
			
			// We should have a single job notification record in the database after the Setup Build job step has completed
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(job.Id);
			jobNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
			jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNotNull(jobNotificationStates);
			Assert.AreEqual(1, jobNotificationStates.Count);

			// There should be 6 job step notification records sorted in order:
			// 1 record for the 'Client Boot Test' group for Platform1
			// 1 record for the 'Client Project Test' group for Platform1
			// 2 records for the 'Editor Boot Test' group (Platform1 and Platform2)
			// 2 records for the 'Editor Project Test' group (Platform1 and Platform2)
			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
			jobStepNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
			jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.AreEqual(6, jobStepNotificationStates.Count);

			// Job Step notification for 'Client Boot Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[0], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Client Project Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[1], job.Id, job.TemplateId, ChannelId, ClientProjectTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Editor Boot Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[2], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Editor Project Test Platform2'
			ValidateJobStepInformationAsync(jobStepNotificationStates[3], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform2", PendingBadge);

			// Job Step notification for 'Editor Boot Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[4], job.Id, job.TemplateId, ChannelId, EditorProjectTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Editor Project Test Platform2'
			ValidateJobStepInformationAsync(jobStepNotificationStates[5], job.Id, job.TemplateId, ChannelId, EditorProjectTestGroup, "Platform2", PendingBadge);

			job = await RunBatchAsync(_mainStreamConfig, job, 1);
			job = await RunStepAsync(_mainStreamConfig, job, 1, 0, JobStepOutcome.Success); // Update Version Files
			job = await RunStepAsync(_mainStreamConfig, job, 1, 1, JobStepOutcome.Success); // Compile Editor
			job = await RunStepAsync(_mainStreamConfig, job, 1, 2, JobStepOutcome.Success); // Compile Client
			job = await RunStepAsync(_mainStreamConfig, job, 1, 3, JobStepOutcome.Success); // Cook Client
			job = await RunStepAsync(_mainStreamConfig, job, 1, 4, JobStepOutcome.Success); // Editor BootTest Platform1

			await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[4]);

			// Validate that the 'Editor BootTest Platform1' step is recorded as passing
			await ValidateJobStepInformationAsync(jobNotificationCollection, job.Id, job.TemplateId,ChannelId, job.Batches[1].Steps[4].Id, EditorBootTestGroup, "Platform1", PassingBadge);

			job = await RunStepAsync(_mainStreamConfig, job, 1, 5, JobStepOutcome.Warnings); // Editor Project Tests Platform1

			await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[5]);

			// Validate that the 'Editor Project Test Platform1' step is recorded as having warnings
			await ValidateJobStepInformationAsync(jobNotificationCollection, job.Id, job.TemplateId, ChannelId, job.Batches[1].Steps[5].Id, EditorProjectTestGroup, "Platform1", WarningBadge);

			job = await RunStepAsync(_mainStreamConfig, job, 1, 6, JobStepOutcome.Failure); // Client BootTest Platform1

			await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[6]);

			// Validate that the 'Client BootTest Platform1' step is recorded as having failures and 'Client Project Tests Platform1' is recorded as being skipped due to having a dependency on the BootTest
			await ValidateJobStepInformationAsync(jobNotificationCollection, job.Id, job.TemplateId, ChannelId, job.Batches[1].Steps[6].Id, ClientBootTestGroup, "Platform1", FailureBadge);
			await ValidateJobStepInformationAsync(jobNotificationCollection, job.Id, job.TemplateId, ChannelId, job.Batches[1].Steps[7].Id, ClientProjectTestGroup, "Platform1", SkippedBadge);

			job = await RunStepAsync(_mainStreamConfig, job, 1, 8, JobStepOutcome.Failure); // Editor BootTest Platform2

			await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[8]);

			// The 'Editor BootTest Platform2' step is recorded as having failures which will mark the 'Editor Project Tests Platform2' as being skipped due to being a dependency
			// At this point, all notifications would be sent out and the notifications should be removed from the collections
			jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
			jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNull(jobStepNotificationStates);

			jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(job.Id);
			jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNull(jobNotificationStates);
		}

		[TestMethod]
		public async Task AbortedJobTestAsync()
		{
			ExperimentalConfig experimentalConfig = CreateExperimentalNotificationConfig();
			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob job = await CreateNightlyBuildTestJobAsync(_mainStreamConfig, 30);
			_testJobNotifications.Add(new JobNotificationState(job.Id, job.TemplateId, ChannelId));

			job = await RunBatchAsync(_mainStreamConfig, job, 0);
			job = await RunStepAsync(_mainStreamConfig, job, 0, 0, JobStepOutcome.Warnings); // Setup Build

			await WaitForNotificationAsync(fakeSink, job, job.Batches[0].Steps[0]);
			
			// We should have a single job notification record in the database after the Setup Build job step has completed, even with warnings
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(job.Id);
			jobNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
			jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNotNull(jobNotificationStates);
			Assert.AreEqual(1, jobNotificationStates.Count);

			// There should be 4 job step notification records: 2 records for the Client Boot Test group (Platform1 and Platform3) and 2 records for the Editor Boot Test group (Platform1 and Platform3) sorted in order
			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
			jobStepNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
			jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.AreEqual(4, jobStepNotificationStates.Count);

			// Job Step notification for 'Client Boot Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[0], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Client Boot Test Platform3'
			ValidateJobStepInformationAsync(jobStepNotificationStates[1], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform3", PendingBadge);

			// Job Step notification for 'Editor Boot Test Platform1'
			ValidateJobStepInformationAsync(jobStepNotificationStates[2], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Editor Boot Test Platform3'
			ValidateJobStepInformationAsync(jobStepNotificationStates[3], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform3", PendingBadge);

			// #1
			// Scenario: Cancellation of a step will report the step as failing
			// Expected: Canceled step will be labeled as failing, but should not have any side-effects on the overall job
			{
				// Simulate the cancellation of the 'Editor BootTest Platform1' step
				job = await AbortStepAsync(_mainStreamConfig, job, 1, 4);

				await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[4]);

				// Aside from the 'Editor BootTest Platform1' now being labeled as a failing step, there should still be 4 steps recorded
				jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
				jobStepNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
				jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
				Assert.IsNotNull(jobStepNotificationStates);
				Assert.AreEqual(4, jobStepNotificationStates.Count);

				// Job Step notification for the 'Client Boot Test Platform1' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[0], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform1", PendingBadge);

				// Job Step notification for the 'Client Boot Test Platform3' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[1], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform3", PendingBadge);

				// Job Step notification for the 'Editor Boot Test Platform1' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[2], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform1", FailureBadge);

				// Job Step notification for the 'Editor Boot Test Platform3' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[3], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform3", PendingBadge);
			}

			// #2
			// Scenario: Cancellation of a dependency step will report the steps as being skipped
			// Expected: Canceled step will be labeled as failing and the steps which had a dependency on the canceled step will be skipped
			{
				// Simulate the cancellation of the 'Compile Client' step
				job = await AbortStepAsync(_mainStreamConfig, job, 1, 2);

				await WaitForNotificationAsync(fakeSink, job, job.Batches[1].Steps[2]);

				// Aside from the Client BootTest step for Platform1 and Platform3 now being labeled as skipped steps, there should still be 4 steps recorded
				jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
				jobStepNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
				jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
				Assert.IsNotNull(jobStepNotificationStates);
				Assert.AreEqual(4, jobStepNotificationStates.Count);

				// Job Step notification for the 'Client Boot Test Platform1' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[0], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform1", SkippedBadge);

				// Job Step notification for the 'Client Boot Test Platform3' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[1], job.Id, job.TemplateId, ChannelId, ClientBootTestGroup, "Platform3", SkippedBadge);

				// Job Step notification for the 'Editor Boot Test Platform1' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[2], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform1", FailureBadge);

				// Job Step notification for the 'Editor Boot Test Platform3' group
				ValidateJobStepInformationAsync(jobStepNotificationStates[3], job.Id, job.TemplateId, ChannelId, EditorBootTestGroup, "Platform3", PendingBadge);
			}

			// #3
			// Scenario: Cancellation of a job will cleanup all notifications
			// Expected: All notification records will be cleaned up
			{
				// Simulate the cancellation of a job
				job = await AbortJobAsync(job);

				await WaitForNotificationAsync(fakeSink, job, null);

				// The job should be removed from the collection
				jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				jobNotificationQueryBuilder.AddJobFilter(job.Id);
				jobNotificationQueryBuilder.AddTemplateFilter(job.TemplateId);
				jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
				Assert.IsNull(jobNotificationStates);

				// The job steps should be removed from the collection
				jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
				jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
				Assert.IsNull(jobStepNotificationStates);
			}
		}

		[TestMethod]
		public async Task ConsolidatedNotificationTestAsync()
		{
			int commitId = 40;
			NotificationFormatConfig spawnNotificationFormatConfig = CreateNotificationFormatConfig(SpawnJobTestGroup, "^Spawn\\s");
			NotificationFormatConfig editorNotificationFormatConfig = CreateNotificationFormatConfig(EditorBootTestGroup, "^Editor BootTest\\s");
			NotificationFormatConfig clientNotificationFormatConfig = CreateNotificationFormatConfig(ClientBootTestGroup, "^Client BootTest\\s");
			NotificationFormatConfig editorProjectNotificationFormatConfig = CreateNotificationFormatConfig(EditorProjectTestGroup, "^Editor Project Tests\\s");
			NotificationFormatConfig clientProjectNotificationFormatConfig = CreateNotificationFormatConfig(ClientProjectTestGroup, "^Client Project Tests\\s");

			JobNotificationConfig jobNotificationConfig = new JobNotificationConfig();
			jobNotificationConfig.Enabled = true;
			jobNotificationConfig.Template = _buildTemplateId;
			jobNotificationConfig.NamePattern = "Platform1|Platform2|Platform3";
			jobNotificationConfig.NotificationFormats.Add(spawnNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(editorNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(editorProjectNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientProjectNotificationFormatConfig);
			jobNotificationConfig.Channels.Add(ChannelId);

			NotificationStreamConfig notificationStreamConfig = new NotificationStreamConfig();
			notificationStreamConfig.Streams.Add(_mainStreamId);
			notificationStreamConfig.JobNotifications.Add(jobNotificationConfig);

			NotificationConfig notificationConfig = new NotificationConfig();
			notificationConfig.Id = new NotificationConfigId("test-notification");
			notificationConfig.NotificationStreams.Add(notificationStreamConfig);

			ExperimentalConfig experimentalConfig = new ExperimentalConfig();
			experimentalConfig.Notifications.Add(notificationConfig);

			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob parentJob = await CreateNightlyBuildJobAsync(_mainStreamConfig, commitId);
			_testJobNotifications.Add(new JobNotificationState(parentJob.Id, parentJob.TemplateId, ChannelId));

			parentJob = await RunBatchAsync(_mainStreamConfig, parentJob, 0);
			parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 0, 0, JobStepOutcome.Success); // Setup Build

			await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[0].Steps[0]);

			// We should have a single job notification record in the database after the Setup Build job step has completed, even with warnings
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
			jobNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
			jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNotNull(jobNotificationStates);
			Assert.AreEqual(1, jobNotificationStates.Count);

			// There should be 3 job step notification records for the Spawn Test Job group for each platform (Platform1, Platform2, and Platform3)
			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(parentJob.Id);
			jobStepNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
			jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.AreEqual(3, jobStepNotificationStates.Count);

			// Job Step notification for 'Spawn Platform1 Test'
			ValidateJobStepInformationAsync(jobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Spawn Platform2 Test'
			ValidateJobStepInformationAsync(jobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PendingBadge);

			// Job Step notification for 'Spawn Platform3 Test'
			ValidateJobStepInformationAsync(jobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PendingBadge);

			// #1
			// Scenario: Spawned job has a failing Setup Build step
			// Expected: The spawned job will not append any notification information
			{
				parentJob = await RunBatchAsync(_mainStreamConfig, parentJob, 1);

				// The spawned job will be created before the job step completes
				IJob spawnedJob = await SpawnNightlyBuildTestJobAsync(_mainStreamConfig, parentJob.Id, parentJob.Batches[1].Steps[0].Id, "Platform1", commitId);
				parentJob = await AddSpawnedJobToParentStepAsync(_mainStreamConfig, parentJob, 1, 0, spawnedJob.Id);
				_testJobNotifications.Add(new JobNotificationState(spawnedJob.Id, spawnedJob.TemplateId, ChannelId));

				parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 1, 0, JobStepOutcome.Success); // Spawn Platform1 Tests

				await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[1].Steps[0]);

				await ValidateJobStepInformationAsync(jobNotificationCollection, parentJob.Id, parentJob.TemplateId, ChannelId, parentJob.Batches[1].Steps[0].Id, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Execute the Setup Build of the newly spawned job
				spawnedJob = await RunBatchAsync(_mainStreamConfig, spawnedJob, 0);
				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 0, 0, JobStepOutcome.Failure); // Setup Build

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[0].Steps[0]);

				// There should be nothing in the job collection for the spawned job as the job has both failed and should've been recognized as a spawned job
				IJobNotificationStateQueryBuilder spawnedJobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				spawnedJobNotificationQueryBuilder.AddJobFilter(spawnedJob.Id);
				IReadOnlyList<IJobNotificationStateRef>? spawnedJobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(spawnedJobNotificationQueryBuilder);
				Assert.IsNull(spawnedJobNotificationStates);

				IJobStepNotificationStateQueryBuilder spawnedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				spawnedJobStepNotificationQueryBuilder.AddJobFilter(spawnedJob.Id);
				IReadOnlyList<IJobStepNotificationStateRef>? spawnedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(spawnedJobStepNotificationQueryBuilder);
				Assert.IsNull(spawnedJobStepNotificationStates);

				// The parent job should only have the notifications regarding the spawn test steps as the failing spawned job did not append its steps to the parent job
				jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
				jobNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
				jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
				Assert.IsNotNull(jobNotificationStates);
				Assert.AreEqual(1, jobNotificationStates.Count);

				// There should be 3 job step notification records for the Spawn Test Job group for each platform (Platform1, Platform2, and Platform3)
				jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				jobStepNotificationQueryBuilder.AddJobFilter(parentJob.Id);
				jobStepNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
				jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
				Assert.IsNotNull(jobStepNotificationStates);
				Assert.AreEqual(3, jobStepNotificationStates.Count);

				// Job Step notification for the 'Spawn Platform1 Test' will now being labeled as passing since the step spawned the new job
				// Even though the actual job had issues which will be reported in the Slack thread
				ValidateJobStepInformationAsync(jobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Job Step notification for the Spawn Platform2 Test group
				ValidateJobStepInformationAsync(jobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PendingBadge);

				// Job Step notification for the Spawn Platform3 Test group
				ValidateJobStepInformationAsync(jobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PendingBadge);
			}

			// #2
			// Scenario: Spawned job has a successfully completed Setup Build, but later has a step and job manually aborted
			// Expected: Spawned job will append its information to the parent job. The later canceled step will be labeled as a failure and the canceled job will then have the information regarding the spawned job removed
			{
				// The spawned job will be created before the job step completes
				IJob spawnedJob = await SpawnNightlyBuildTestJobAsync(_mainStreamConfig, parentJob.Id, parentJob.Batches[1].Steps[1].Id, "Platform2", commitId);
				parentJob = await AddSpawnedJobToParentStepAsync(_mainStreamConfig, parentJob, 1, 0, spawnedJob.Id);
				_testJobNotifications.Add(new JobNotificationState(spawnedJob.Id, spawnedJob.TemplateId, ChannelId));

				parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 1, 1, JobStepOutcome.Success); // Spawn Platform2 Tests

				await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[1].Steps[1]);

				await ValidateJobStepInformationAsync(jobNotificationCollection, parentJob.Id, parentJob.TemplateId, ChannelId, parentJob.Batches[1].Steps[1].Id, SpawnJobTestGroup, "Platform2", PassingBadge);

				// Execute the Setup Build of the newly spawned job
				spawnedJob = await RunBatchAsync(_mainStreamConfig, spawnedJob, 0);
				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 0, 0, JobStepOutcome.Success); // Setup Build

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[0].Steps[0]);

				// There should not be a job notification state for the spawned job as it is linked to the parent job
				IJobNotificationStateQueryBuilder spawnedJobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				spawnedJobNotificationQueryBuilder.AddJobFilter(spawnedJob.Id);
				IReadOnlyList<IJobNotificationStateRef>? spawnedJobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(spawnedJobNotificationQueryBuilder);
				Assert.IsNull(spawnedJobNotificationStates);

				// The spawned job will append its notification records to the parent job and there will be a total of have 7 job step notification records with the parent job step notifications appearing at the beginning of the list:
				// Spawn Platform1 Test, Spawn Platform2 Test, Spawn Platform3 Test, Client BootTest Platform2, Client Project Tests Platform2, Editor BootTest Platform2, and Editor Project Tests Platform2
				IJobStepNotificationStateQueryBuilder consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				consolidatedJobStepNotificationQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
				consolidatedJobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				IReadOnlyList<IJobStepNotificationStateRef>? consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
				Assert.IsNotNull(consolidatedJobStepNotificationStates);
				Assert.AreEqual(7, consolidatedJobStepNotificationStates.Count);

				// Job Step notification for 'Spawn Platform1 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Job Step notification for 'Spawn Platform2 Test' group'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PassingBadge);

				// Job Step notification for 'Spawn Platform3 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PendingBadge);

				// Job Step notification for 'Client BootTest Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[3], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientBootTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Client Project Tests Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[4], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientProjectTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor BootTest Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[5], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorBootTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor Project Tests Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[6], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorProjectTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Simulate the cancellation of the 'Editor BootTest Platform1' step
				spawnedJob = await AbortStepAsync(_mainStreamConfig, spawnedJob, 1, 0);

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[1].Steps[0]);

				// Aside from 'Editor BootTest Platform2' now being labeled as a failing step and its follow-up step 'Editor Project Tests Platform2' being labeled as skipped, there should still be 7 steps recorded
				consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				consolidatedJobStepNotificationQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
				consolidatedJobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
				Assert.IsNotNull(consolidatedJobStepNotificationStates);
				Assert.AreEqual(7, consolidatedJobStepNotificationStates.Count);

				// Job Step notification for 'Spawn Platform1 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Job Step notification for 'Spawn Platform2 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PassingBadge);

				// Job Step notification for 'Spawn Platform3 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PendingBadge);

				// Job Step notification for 'Client BootTest Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[3], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientBootTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Client Project Tests Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[4], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientProjectTestGroup, "Platform2", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor BootTest Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[5], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorBootTestGroup, "Platform2", FailureBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor Project Tests Platform2'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[6], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorProjectTestGroup, "Platform2", SkippedBadge, parentJob.Id, parentJob.TemplateId);

				// Simulate the cancellation of a job
				spawnedJob = await AbortJobAsync(spawnedJob);

				await WaitForNotificationAsync(fakeSink, spawnedJob, null);

				// The parent job notification should still be found after a spawned job has been canceled
				jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
				jobNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
				jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
				Assert.IsNotNull(jobNotificationStates);
				Assert.AreEqual(1, jobNotificationStates.Count);

				// The spawned job will have its job step notifications removed as the Slack summary will provide the link to the canceled job
				// There should be 3 job step notification records for the Spawn Test Job group for each platform (Platform1, Platform2, and Platform3)
				consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				consolidatedJobStepNotificationQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
				consolidatedJobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
				Assert.IsNotNull(consolidatedJobStepNotificationStates);
				Assert.AreEqual(3, consolidatedJobStepNotificationStates.Count);

				// Job Step notification for the 'Spawn Platform2 Test' will now being labeled as passing since the step spawned the new job
				// Even though the actual job had issues and was canceled which will be reported in the Slack thread
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Job Step notification for 'Spawn Platform2 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PassingBadge);

				// Job Step notification for 'Spawn Platform3 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PendingBadge);
			}

			// #3
			// Scenario: Spawned job has a completed Setup Build, albeit with warnings, and is updating with various results
			// Expected: Spawned job will keep updating the job step notification states until the job has run to completion. At which point all jobs have completed and the notification records will be cleaned
			{
				// The spawned job will be created before the job step completes
				IJob spawnedJob = await SpawnNightlyBuildTestJobAsync(_mainStreamConfig, parentJob.Id, parentJob.Batches[1].Steps[2].Id, "Platform3", commitId);
				parentJob = await AddSpawnedJobToParentStepAsync(_mainStreamConfig, parentJob, 1, 0, spawnedJob.Id);
				_testJobNotifications.Add(new JobNotificationState(spawnedJob.Id, spawnedJob.TemplateId, ChannelId));

				parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 1, 2, JobStepOutcome.Success); // Spawn Platform3 Tests

				await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[1].Steps[2]);

				await ValidateJobStepInformationAsync(jobNotificationCollection, parentJob.Id, parentJob.TemplateId, ChannelId, parentJob.Batches[1].Steps[2].Id, SpawnJobTestGroup, "Platform3", PassingBadge);

				// Execute the Setup Build of the newly spawned job
				spawnedJob = await RunBatchAsync(_mainStreamConfig, spawnedJob, 0);
				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 0, 0, JobStepOutcome.Warnings); // Setup Build

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[0].Steps[0]);

				// There should not be a job notification state for the spawned job as it is linked to the parent job
				IJobNotificationStateQueryBuilder spawnedJobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				spawnedJobNotificationQueryBuilder.AddJobFilter(spawnedJob.Id);
				IReadOnlyList<IJobNotificationStateRef>? spawnedJobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(spawnedJobNotificationQueryBuilder);
				Assert.IsNull(spawnedJobNotificationStates);

				// The spawned job will append its notification records to the parent job and there will be a total of have 7 job step notification records with the parent job step notifications appearing at the beginning of the list:
				// Spawn Platform1 Test, Spawn Platform2 Test, Spawn Platform3 Test, Client BootTest Platform3, Client Project Tests Platform3, Editor BootTest Platform3, and Editor Project Tests Platform3
				IJobStepNotificationStateQueryBuilder consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				consolidatedJobStepNotificationQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
				consolidatedJobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
				IReadOnlyList<IJobStepNotificationStateRef>? consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
				Assert.IsNotNull(consolidatedJobStepNotificationStates);
				Assert.AreEqual(7, consolidatedJobStepNotificationStates.Count);

				// Job Step notification for 'Spawn Platform1 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PassingBadge);

				// Job Step notification for 'Spawn Platform2 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PassingBadge);

				// Job Step notification for 'Spawn Platform3 Test'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[2], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform3", PassingBadge);

				// Job Step notification for 'Client BootTest Platform3'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[3], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientBootTestGroup, "Platform3", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Client Project Tests Platform3'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[4], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, ClientProjectTestGroup, "Platform3", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor BootTest Platform3'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[5], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorBootTestGroup, "Platform3", PendingBadge, parentJob.Id, parentJob.TemplateId);

				// Job Step notification for 'Editor Project Tests Platform3'
				ValidateJobStepInformationAsync(consolidatedJobStepNotificationStates[6], spawnedJob.Id, spawnedJob.TemplateId, ChannelId, EditorProjectTestGroup, "Platform3", PendingBadge, parentJob.Id, parentJob.TemplateId);

				spawnedJob = await RunBatchAsync(_mainStreamConfig, spawnedJob, 1);
				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 1, 0, JobStepOutcome.Failure); // Editor BootTest Platform3

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[1].Steps[0]);

				// Validate that the 'Editor BootTest Platform3' step is recorded as having failures and 'Editor Project Tests Platform3' is recorded as being skipped due to having a dependency on the BootTest
				await ValidateJobStepInformationAsync(jobNotificationCollection, spawnedJob.Id, spawnedJob.TemplateId, ChannelId, spawnedJob.Batches[1].Steps[0].Id, EditorBootTestGroup, "Platform3", FailureBadge, parentJob.Id, parentJob.TemplateId);
				await ValidateJobStepInformationAsync(jobNotificationCollection, spawnedJob.Id, spawnedJob.TemplateId, ChannelId, spawnedJob.Batches[1].Steps[1].Id, EditorProjectTestGroup, "Platform3", SkippedBadge, parentJob.Id, parentJob.TemplateId);

				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 1, 2, JobStepOutcome.Warnings); // Client BootTest Platform3

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[1].Steps[2]);

				// Validate that the Client BootTest Platform3 step is recorded as having warnings
				await ValidateJobStepInformationAsync(jobNotificationCollection, spawnedJob.Id, spawnedJob.TemplateId, ChannelId, spawnedJob.Batches[1].Steps[2].Id, ClientBootTestGroup, "Platform3", WarningBadge, parentJob.Id, parentJob.TemplateId);

				spawnedJob = await RunStepAsync(_mainStreamConfig, spawnedJob, 1, 3, JobStepOutcome.Success); // Client Project Tests Platform3

				await WaitForNotificationAsync(fakeSink, spawnedJob, spawnedJob.Batches[1].Steps[3]);

				// At this point, the Client Project Tests job step notification would be marked as passing, all notifications would be sent out, and the notifications should be removed from the collections
				consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
				Assert.IsNull(consolidatedJobStepNotificationStates);

				jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
				jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
				jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
				Assert.IsNull(jobNotificationStates);
			}
		}

		[TestMethod]
		public async Task ParentJobCancellationTestAsync()
		{
			// Scenario: Job matched with the configuration but is canceled after the Setup Build has been run
			// Expected: Job steps are found in the JobNotificationCollection after the Setup Build node executes, but is then cleaned up after the job was canceled

			int commitId = 40;
			NotificationFormatConfig spawnNotificationFormatConfig = CreateNotificationFormatConfig(SpawnJobTestGroup, "^Spawn\\s");
			NotificationFormatConfig editorNotificationFormatConfig = CreateNotificationFormatConfig(EditorBootTestGroup, "^Editor BootTest\\s");
			NotificationFormatConfig clientNotificationFormatConfig = CreateNotificationFormatConfig(ClientBootTestGroup, "^Client BootTest\\s");

			JobNotificationConfig jobNotificationConfig = new JobNotificationConfig();
			jobNotificationConfig.Enabled = true;
			jobNotificationConfig.Template = _buildTemplateId;
			jobNotificationConfig.NamePattern = "Platform1|Platform2";
			jobNotificationConfig.NotificationFormats.Add(spawnNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(editorNotificationFormatConfig);
			jobNotificationConfig.NotificationFormats.Add(clientNotificationFormatConfig);
			jobNotificationConfig.Channels.Add(ChannelId);

			NotificationStreamConfig notificationStreamConfig = new NotificationStreamConfig();
			notificationStreamConfig.Streams.Add(_mainStreamId);
			notificationStreamConfig.JobNotifications.Add(jobNotificationConfig);

			NotificationConfig notificationConfig = new NotificationConfig();
			notificationConfig.Id = new NotificationConfigId("test-notification");
			notificationConfig.NotificationStreams.Add(notificationStreamConfig);

			ExperimentalConfig experimentalConfig = new ExperimentalConfig();
			experimentalConfig.Notifications.Add(notificationConfig);

			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob parentJob = await CreateNightlyBuildJobAsync(_mainStreamConfig, commitId);
			_testJobNotifications.Add(new JobNotificationState(parentJob.Id, parentJob.TemplateId, ChannelId));

			parentJob = await RunBatchAsync(_mainStreamConfig, parentJob, 0);
			parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 0, 0, JobStepOutcome.Success); // Setup Build

			await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[0].Steps[0]);

			// We should have a single job notification record in the database after the Setup Build job step has completed
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
			jobNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
			jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNotNull(jobNotificationStates);
			Assert.AreEqual(1, jobNotificationStates.Count);

			// There should be 2 job step notification records for the Spawn Test Job group for each platform (Platform1 and Platform2)
			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(parentJob.Id);
			jobStepNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
			jobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.AreEqual(2, jobStepNotificationStates.Count);

			// Job Step notification for 'Spawn Platform1 Test'
			ValidateJobStepInformationAsync(jobStepNotificationStates[0], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform1", PendingBadge);

			// Job Step notification for 'Spawn Platform2 Test'
			ValidateJobStepInformationAsync(jobStepNotificationStates[1], parentJob.Id, parentJob.TemplateId, ChannelId, SpawnJobTestGroup, "Platform2", PendingBadge);

			parentJob = await RunBatchAsync(_mainStreamConfig, parentJob, 1);

			// The spawned job will be created before the job step completes
			IJob spawnedJob1 = await SpawnNightlyBuildTestJobAsync(_mainStreamConfig, parentJob.Id, parentJob.Batches[1].Steps[0].Id, "Platform1", commitId);
			parentJob = await AddSpawnedJobToParentStepAsync(_mainStreamConfig, parentJob, 1, 0, spawnedJob1.Id);
			_testJobNotifications.Add(new JobNotificationState(spawnedJob1.Id, spawnedJob1.TemplateId, ChannelId));

			parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 1, 0, JobStepOutcome.Success); // Spawn Platform1 Tests

			// Spawn the second job
			IJob spawnedJob2 = await SpawnNightlyBuildTestJobAsync(_mainStreamConfig, parentJob.Id, parentJob.Batches[1].Steps[1].Id, "Platform2", commitId);
			parentJob = await AddSpawnedJobToParentStepAsync(_mainStreamConfig, parentJob, 1, 1, spawnedJob2.Id);
			_testJobNotifications.Add(new JobNotificationState(spawnedJob2.Id, spawnedJob2.TemplateId, ChannelId));

			parentJob = await RunStepAsync(_mainStreamConfig, parentJob, 1, 1, JobStepOutcome.Success); // Spawn Platform2 Tests

			await WaitForNotificationAsync(fakeSink, parentJob, parentJob.Batches[1].Steps[1]);

			// Execute the Setup Build steps of all of our spawned jobs
			spawnedJob1 = await RunBatchAsync(_mainStreamConfig, spawnedJob1, 0);
			_ = await RunStepAsync(_mainStreamConfig, spawnedJob1, 0, 0, JobStepOutcome.Success); // Setup Build

			spawnedJob2 = await RunBatchAsync(_mainStreamConfig, spawnedJob2, 0);
			spawnedJob2 = await RunStepAsync(_mainStreamConfig, spawnedJob2, 0, 0, JobStepOutcome.Success); // Setup Build

			await WaitForNotificationAsync(fakeSink, spawnedJob2, spawnedJob2.Batches[0].Steps[0]);

			// The spawned job will append its notification records to the parent job and there will be a total of have 10 job step notification records
			IJobStepNotificationStateQueryBuilder consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
			consolidatedJobStepNotificationQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
			consolidatedJobStepNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			IReadOnlyList<IJobStepNotificationStateRef>? consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
			Assert.IsNotNull(consolidatedJobStepNotificationStates);
			Assert.AreEqual(6, consolidatedJobStepNotificationStates.Count);

			// Simulate the cancellation of the parent job
			parentJob = await AbortJobAsync(parentJob);

			await WaitForNotificationAsync(fakeSink, parentJob, null);

			// The job should be removed from the collection
			jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(parentJob.Id);
			jobNotificationQueryBuilder.AddTemplateFilter(parentJob.TemplateId);
			jobNotificationQueryBuilder.AddRecipientFilter(ChannelId);
			jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNull(jobNotificationStates);

			// The parent and spawned job steps should be removed from the collection
			consolidatedJobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			consolidatedJobStepNotificationQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
			consolidatedJobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(consolidatedJobStepNotificationQueryBuilder);
			Assert.IsNull(consolidatedJobStepNotificationStates);
		}

		[TestMethod]
		public async Task NotificationConfigurationsByStreamTagAsync()
		{
			// Scenario: Job matched with the configuration using associated StreamTags enabled in the StreamConfig
			// Expected: Single record should exist in both the JobNotificationCollection and JobStepNotificationCollection

			int commitId = 50;
			NotificationFormatConfig spawnNotificationFormatConfig = CreateNotificationFormatConfig(SpawnJobTestGroup, "^Spawn\\s");

			JobNotificationConfig jobNotificationConfig = new JobNotificationConfig();
			jobNotificationConfig.Enabled = true;
			jobNotificationConfig.Template = _buildTemplateId;
			jobNotificationConfig.NamePattern = "Platform1";
			jobNotificationConfig.NotificationFormats.Add(spawnNotificationFormatConfig);
			jobNotificationConfig.Channels.Add(ChannelId);

			NotificationStreamConfig notificationStreamConfig = new NotificationStreamConfig();
			notificationStreamConfig.EnabledStreamTags.Add("NotificationEnabledTag");
			notificationStreamConfig.JobNotifications.Add(jobNotificationConfig);

			NotificationConfig notificationConfig = new NotificationConfig();
			notificationConfig.Id = new NotificationConfigId("test-notification");
			notificationConfig.NotificationStreams.Add(notificationStreamConfig);

			ExperimentalConfig experimentalConfig = new ExperimentalConfig();
			experimentalConfig.Notifications.Add(notificationConfig);

			UpdateGlobalConfig(experimentalConfig);

			JobNotificationCollection jobNotificationCollection = ServiceProvider.GetRequiredService<JobNotificationCollection>();

			FakeExperimentalNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeExperimentalNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service.StartAsync(cancellationToken: default);

			IJob job = await CreateNightlyBuildJobAsync(_mainStreamConfig, commitId);
			_testJobNotifications.Add(new JobNotificationState(job.Id, job.TemplateId, ChannelId));

			job = await RunBatchAsync(_mainStreamConfig, job, 0);
			job = await RunStepAsync(_mainStreamConfig, job, 0, 0, JobStepOutcome.Success); // Setup Build

			await WaitForNotificationAsync(fakeSink, job, job.Batches[0].Steps[0]);

			// There should be items in the collections if we successfully picked up the job from the tag
			IJobNotificationStateQueryBuilder jobNotificationQueryBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationQueryBuilder.AddJobFilter(job.Id);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationQueryBuilder);
			Assert.IsNotNull(jobNotificationStates);
			Assert.IsTrue(jobNotificationStates.Count == 1);

			IJobStepNotificationStateQueryBuilder jobStepNotificationQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationQueryBuilder.AddJobFilter(job.Id);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationQueryBuilder);
			Assert.IsNotNull(jobStepNotificationStates);
			Assert.IsTrue(jobStepNotificationStates.Count == 1);
		}
	}
}
