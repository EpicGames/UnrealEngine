// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Plugins;
using HordeServer.Streams;
using System.Diagnostics.CodeAnalysis;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the experimental plugin
	/// </summary>
	public class ExperimentalConfig : IPluginConfig
	{
		/// <summary>
		/// Notifications to be sent on the Horde server
		/// </summary>
		public List<NotificationConfig> Notifications { get; set; } = new List<NotificationConfig>();

		readonly Dictionary<NotificationConfigId, NotificationConfig> _notificationConfigLookup = new Dictionary<NotificationConfigId, NotificationConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			_notificationConfigLookup.Clear();
			foreach (NotificationConfig config in Notifications)
			{
				config.PostLoad();
				_notificationConfigLookup.TryAdd(config.Id, config);
			}
		}

		/// <summary>
		/// Attempt to get config for a given stream and template
		/// </summary>
		/// <param name="streamId">Stream id</param>
		/// <param name="templateId">Template id</param>
		/// <param name="jobNotificationConfigs">Receives the list of config objects on success</param>
		/// <param name="streamTags">List of tags for a stream</param>
		/// <param name="includeDisabledConfigs">Flag on whether or not disabled configs should be included</param>
		/// <returns>True if the job notification configs were found</returns>
		public bool TryGetJobNofications(StreamId streamId, TemplateId templateId, [NotNullWhen(true)] out HashSet<JobNotificationConfig>? jobNotificationConfigs, List<StreamTag>? streamTags = null, bool includeDisabledConfigs = false)
		{
			jobNotificationConfigs = new HashSet<JobNotificationConfig>();

			foreach (NotificationConfig config in Notifications)
			{
				HashSet<JobNotificationConfig>? localJobNotificationConfigs;

				// A notification configuration can be setup with a stream, stream tag, or have configurations that use both.
				// We need to search through all of our lookups to make sure that nothing has been missed
				if (config.TryGetJobNofications(streamId, templateId, out localJobNotificationConfigs, includeDisabledConfigs))
				{
					jobNotificationConfigs.UnionWith(localJobNotificationConfigs);
				}
				
				if (streamTags is not null && config.TryGetJobNofications(streamTags, templateId, out localJobNotificationConfigs, includeDisabledConfigs))
				{
					jobNotificationConfigs.UnionWith(localJobNotificationConfigs);
				}
			}

			return jobNotificationConfigs.Count != 0;
		}
	}
}
