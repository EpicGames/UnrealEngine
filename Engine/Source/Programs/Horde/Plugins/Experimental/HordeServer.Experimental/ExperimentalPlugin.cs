// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Experimental.Notifications;
using HordeServer.Notifications;
using HordeServer.Notifications.Sinks;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using System.Diagnostics.CodeAnalysis;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the experimental plugin
	/// </summary>
	[Plugin("Experimental", EnabledByDefault = false, GlobalConfigType = typeof(ExperimentalConfig), ServerConfigType = typeof(ExperimentalServerConfig))]
	public class ExperimentalPlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ExperimentalPlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			serviceCollection.AddSingleton<IJobNotificationCollection, JobNotificationCollection>();
			serviceCollection.AddSingleton<INotificationSink, ExperimentalSlackNotificationSink>();
		}
	}

	/// <summary>
	/// Helper methods for experimental plugin config
	/// </summary>
	public static class ExperimentalPluginExtensions
	{
		/// <summary>
		/// Configures the experimental plugin
		/// </summary>
		public static void AddExperimentalConfig(this IDictionary<PluginName, IPluginConfig> dictionary, ExperimentalConfig experimentalConfig)
			=> dictionary[new PluginName("Experimental")] = experimentalConfig;

		/// <summary>
		/// Gets configuration for the experimental plugin
		/// </summary>
		public static ExperimentalConfig GetExperimentalConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (ExperimentalConfig)dictionary[new PluginName("Experimental")];

		/// <summary>
		/// Gets configuration for the experimental plugin
		/// </summary>
		public static bool TryGetExperimentalConfig(this IDictionary<PluginName, IPluginConfig> dictionary, [NotNullWhen(true)] out ExperimentalConfig? experimentalConfig)
		{
			IPluginConfig? pluginConfig;
			if (dictionary.TryGetValue(new PluginName("Experimental"), out pluginConfig) && pluginConfig is ExperimentalConfig newExperimentalConfig)
			{
				experimentalConfig = newExperimentalConfig;
				return true;
			}
			else
			{
				experimentalConfig = null;
				return false;
			}
		}
	}
}
