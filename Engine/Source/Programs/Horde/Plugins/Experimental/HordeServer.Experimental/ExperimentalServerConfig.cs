// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Server configuration for the experimental plugin
	/// </summary>
	public class ExperimentalServerConfig : PluginServerConfig
	{
	}
}
