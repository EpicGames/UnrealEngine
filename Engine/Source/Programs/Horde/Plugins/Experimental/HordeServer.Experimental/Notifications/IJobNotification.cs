// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using MongoDB.Bson;

namespace HordeServer.Experimental.Notifications
{
	/// <summary>
	/// A job notification state reference
	/// </summary>
	public interface IJobNotificationStateRef
	{
		/// <summary>
		/// The job notification state identifier
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// The job identifier
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The template identifier
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// The recipient of the notification
		/// </summary>
		string Recipient { get; }

		/// <summary>
		/// The channel of the notification
		/// </summary>
		string Channel { get; }

		/// <summary>
		/// The timestamp of the notification
		/// </summary>
		string Ts { get; }
	}

	/// <summary>
	/// A job step notification state reference
	/// </summary>
	public interface IJobStepNotificationStateRef
	{
		/// <summary>
		/// The job notification state identifier
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// The job identifier
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The job step identifier
		/// </summary>
		JobStepId JobStepId { get; set; }

		/// <summary>
		/// The template identifier
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// The recipient of the notification
		/// </summary>
		string Recipient { get; }

		/// <summary>
		/// The associated group of the notification
		/// </summary>
		string Group { get; }

		/// <summary>
		/// The target platform of the notification
		/// </summary>
		string TargetPlatform { get; }

		/// <summary>
		/// The badge of the notification
		/// </summary>
		string Badge { get; }

		/// <summary>
		/// The channel of the notification
		/// </summary>
		string Channel { get; }

		/// <summary>
		/// The timestamp of the threaded notification
		/// </summary>
		string? ThreadTs { get; }

		/// <summary>
		/// The timestamp of the notification
		/// </summary>
		string Ts { get; }

		/// <summary>
		/// The parent job identifier
		/// </summary>
		JobId? ParentJobId { get; }

		/// <summary>
		/// The parent job template identifier
		/// </summary>
		TemplateId? ParentJobTemplateId { get; }
	}

	/// <summary>
	/// Builder to simplify the query generation for job notification states
	/// </summary>
	public interface IJobNotificationStateQueryBuilder
	{
		/// <summary>
		/// Add the filter for a job
		/// </summary>
		/// <param name="jobId">Identifier of the job to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddJobFilter(JobId jobId);

		/// <summary>
		/// Adds a filter for a list of jobs
		/// </summary>
		/// <param name="jobIds">List of job identifiers to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddJobFilters(List<JobId> jobIds);

		/// <summary>
		/// Add the filter for a template
		/// </summary>
		/// <param name="templateId">Identifier of the template to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddTemplateFilter(TemplateId templateId);

		/// <summary>
		/// Adds a filter for a list of templates
		/// </summary>
		/// <param name="templateIds">List of template identifiers to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddTemplateFilters(List<TemplateId> templateIds);

		/// <summary>
		/// Add the filter for a specific recipient
		/// </summary>
		/// <param name="recipient">Recipient to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddRecipientFilter(string recipient);

		/// <summary>
		/// Add the filter for a specific channel
		/// </summary>
		/// <param name="channel">Channel to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddChannelFilter(string channel);

		/// <summary>
		/// Add the filter for a timestamp
		/// </summary>
		/// <param name="ts">Timestamp to filter by</param>
		/// <returns>A reference to the current <see cref="IJobNotificationStateQueryBuilder"/></returns>
		public IJobNotificationStateQueryBuilder AddTimestampFilter(string ts);
	}

	/// <summary>
	/// Builder to simplify the query generation for job step notification states
	/// </summary>
	public interface IJobStepNotificationStateQueryBuilder
	{
		/// <summary>
		/// Add the filter for a job
		/// </summary>
		/// <param name="jobId">Identifier of the job to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddJobFilter(JobId jobId);

		/// <summary>
		/// Add the filter for a template
		/// </summary>
		/// <param name="templateId">Identifier of the template to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddTemplateFilter(TemplateId templateId);

		/// <summary>
		/// Add the filter for a specific recipient
		/// </summary>
		/// <param name="recipient">Recipient to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddRecipientFilter(string recipient);

		/// <summary>
		/// Add the filter for a job step
		/// </summary>
		/// <param name="jobStepId">Identifier of the job step to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddJobStepFilter(JobStepId jobStepId);

		/// <summary>
		/// Add the filter for a specific group
		/// </summary>
		/// <param name="group">Group to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddGroupFilter(string group);

		/// <summary>
		/// Add the filter for a specific platform
		/// </summary>
		/// <param name="platform">Platform to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddPlatformFilter(string platform);

		/// <summary>
		/// Add the filter for a specific badge
		/// </summary>
		/// <param name="badge">Badge to filter by</param>
		/// <param name="shouldMatch">Whether or not the filter should match the exact badge</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddBadgeFilter(string badge, bool shouldMatch);

		/// <summary>
		/// Add the filter for a specific channel
		/// </summary>
		/// <param name="channel">Channel to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddChannelFilter(string channel);

		/// <summary>
		/// Add the filter for a threaded timestamp
		/// </summary>
		/// <param name="threadTs">Thread timestamp to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddThreadTimestampFilter(string threadTs);

		/// <summary>
		/// Add the filter for a timestamp
		/// </summary>
		/// <param name="ts">Timestamp to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddTimestampFilter(string ts);

		/// <summary>
		/// Add the filter for a parent job
		/// </summary>
		/// <param name="jobId">Identifier of the job to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddParentJobFilter(JobId jobId);

		/// <summary>
		/// Add the filter for a parent template
		/// </summary>
		/// <param name="templateId">Identifier of the template to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddParentJobTemplateFilter(TemplateId templateId);

		/// <summary>
		/// Add the filter for either a job or parent job
		/// </summary>
		/// <param name="jobId">Identifier of the job to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddJobAndParentJobFilter(JobId jobId);

		/// <summary>
		/// Add the filter for either a template or parent template
		/// </summary>
		/// <param name="templateId">Identifier of the template to filter by</param>
		/// <returns>A reference to the current <see cref="IJobStepNotificationStateQueryBuilder"/></returns>
		public IJobStepNotificationStateQueryBuilder AddTemplateAndParentTemplateFilter(TemplateId templateId);
	}

	/// <summary>
	/// Collection of job notification documents
	/// </summary>
	public interface IJobNotificationCollection
	{
		/// <summary>
		/// Fetches a single entity of <see cref="IJobNotificationStateRef"/>
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the stream's template</param>
		/// <param name="recipient">Information regarding a specific recipient</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A <see cref="IJobNotificationStateRef"/> if records were found, otherwise returns null for error checking</returns>
		/// <remarks>Only a single <see cref="IJobNotificationStateRef"/> should exist for the aggregate of the job identifier, template identifier, and recipient should ever exist. Will report an error if another record had been found.</remarks>
		Task<IJobNotificationStateRef?> GetJobNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default);

		/// <summary>
		/// Fetches a read only list of <see cref="IJobNotificationStateRef"/>
		/// </summary>
		/// <param name="builder">The <see cref="IJobNotificationStateQueryBuilder"/> used to generate a query to filter against</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A read only list of <see cref="IJobNotificationStateRef"/> if records were found, otherwise returns null for error checking</returns>
		Task<IReadOnlyList<IJobNotificationStateRef>?> GetJobNotificationStatesAsync(IJobNotificationStateQueryBuilder builder, CancellationToken cancellationToken = default);

		/// <summary>
		/// Add or update the <see cref="IJobNotificationStateRef"/> to the collection
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the stream's template</param>
		/// <param name="recipient">Information regarding a specific recipient</param>
		/// <param name="channel">Information regarding a specific channel</param>
		/// <param name="timestamp">Timestamp for the notification</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A <see cref="IJobNotificationStateRef"/> object</returns>
		Task<IJobNotificationStateRef> AddOrUpdateJobNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, string channel, string timestamp, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a <see cref="IJobNotificationStateRef"/> from the collection
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the stream's template</param>
		/// <param name="recipient">Information regarding a specific recipient</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The number of records which were deleted</returns>
		Task<long> DeleteJobNotificationStatesAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default);

		/// <summary>
		/// Fetches a read only list of <see cref="IJobStepNotificationStateRef"/>
		/// </summary>
		/// <param name="builder">The <see cref="IJobStepNotificationStateQueryBuilder"/> used to generate a query to filter against</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A read only list of <see cref="IJobStepNotificationStateRef"/> if records were found, otherwise returns null for error checking</returns>
		Task<IReadOnlyList<IJobStepNotificationStateRef>?> GetJobStepNotificationStatesAsync(IJobStepNotificationStateQueryBuilder builder, CancellationToken cancellationToken = default);

		/// <summary>
		/// Add or update the <see cref="IJobStepNotificationStateRef"/> to the collection
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the stream's template</param>
		/// <param name="recipient">Information regarding a specific recipient</param>
		/// <param name="jobStepId">Identifier of the job step</param>
		/// <param name="group">Group associated with the notification</param>
		/// <param name="platform">Target platform associated with the notification</param>
		/// <param name="badge">Slack emoji associated with the notification</param>
		/// <param name="channel">Information regarding a specific channel</param>
		/// <param name="ts">Timestamp for the notification</param>
		/// <param name="threadTs">Optional Threaded timestamp for the notification</param>
		/// <param name="parentJobId">Optional identifier of the parent job</param>
		/// <param name="parentJobTemplateId">Optional identifier of the parent stream's template</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A <see cref="IJobStepNotificationStateRef"/> object</returns>
		Task<IJobStepNotificationStateRef> AddOrUpdateJobStepNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, JobStepId jobStepId, string group, string platform, string badge, string channel, string ts, string? threadTs = null, JobId? parentJobId = null, TemplateId? parentJobTemplateId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes <see cref="IJobStepNotificationStateRef"/> from the collection
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the stream's template</param>
		/// <param name="recipient">Information regarding a specific recipient</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The number of records which were deleted</returns>
		Task<long> DeleteJobStepNotificationStatesAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default);
	}
}
