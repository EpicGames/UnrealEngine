// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Slack;
using HordeServer.Server;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Experimental.Notifications
{
	/// <summary>
	/// Wrapper around the job notifications collection in a Mongo DB
	/// </summary>
	public class JobNotificationCollection : IJobNotificationCollection
	{
		/// <summary>
		/// MongoDB document for job notifications
		/// </summary>
		class JobNotificationStateDocument : IJobNotificationStateRef
		{
			[BsonId]
			public ObjectId Id { get; set; }

			[BsonElement("jid")]
			public JobId JobId { get; set; }

			[BsonElement("tid")]
			public TemplateId TemplateId { get; set; }

			[BsonElement("rcp")]
			public string Recipient { get; set; } = String.Empty;

			[BsonElement("ch")]
			public string Channel { get; set; } = String.Empty;

			[BsonElement("ts")]
			public string Ts { get; set; } = String.Empty;

			[BsonIgnore]
			public SlackMessageId MessageId => new SlackMessageId(Channel, null, Ts);

			private JobNotificationStateDocument()
			{
				Id = ObjectId.GenerateNewId();
			}

			public JobNotificationStateDocument(JobId jobId, TemplateId templateId, string recipient, string channel, string ts)
			{
				Id = ObjectId.GenerateNewId();
				JobId = jobId;
				TemplateId = templateId;
				Recipient = recipient;
				Channel = channel;
				Ts = ts;
			}
		}

		/// <summary>
		/// MongoDB document for job step notifications
		/// </summary>
		class JobStepNotificationStateDocument : IJobStepNotificationStateRef
		{
			[BsonId]
			public ObjectId Id { get; set; }

			[BsonElement("jid")]
			public JobId JobId { get; set; }

			[BsonElement("tid")]
			public TemplateId TemplateId { get; set; }

			[BsonElement("rcp")]
			public string Recipient { get; set; } = String.Empty;

			[BsonElement("sid")]
			public JobStepId JobStepId { get; set; }

			[BsonElement("grp")]
			public string Group { get; set; } = String.Empty;

			[BsonElement("plt")]
			public string TargetPlatform { get; set; } = String.Empty;

			[BsonElement("bdg")]
			public string Badge { get; set; } = String.Empty;

			[BsonElement("ch")]
			public string Channel { get; set; } = String.Empty;

			[BsonElement("tts")]
			public string? ThreadTs { get; set; }

			[BsonElement("ts")]
			public string Ts { get; set; } = String.Empty;

			[BsonIgnoreIfNull, BsonElement("pjid")]
			public JobId? ParentJobId { get; set; } = null;

			[BsonIgnoreIfNull, BsonElement("ptid")]
			public TemplateId? ParentJobTemplateId { get; set; } = null;

			[BsonIgnore]
			public SlackMessageId MessageId => new SlackMessageId(Channel, ThreadTs, Ts);

			private JobStepNotificationStateDocument()
			{
				Id = ObjectId.GenerateNewId();
			}

			public JobStepNotificationStateDocument(JobId jobId, TemplateId templateId, string recipient, JobStepId jobStepId, string group, string targetPlatform, string badge, string channel, string ts, string? tts = null, JobId? parentJobId = null, TemplateId? parentTemplateId = null)
			{
				Id = ObjectId.GenerateNewId();
				JobId = jobId;
				TemplateId = templateId;
				Recipient = recipient;
				JobStepId = jobStepId;
				Group = group;
				TargetPlatform = targetPlatform;
				Badge = badge;
				Channel = channel;
				Ts = ts;
				ThreadTs = tts;
				ParentJobId = parentJobId;
				ParentJobTemplateId = parentTemplateId;
			}
		}

		/// <inheritdoc/>
		class JobNotificationStateQueryBuilder : IJobNotificationStateQueryBuilder
		{
			public FilterDefinition<JobNotificationStateDocument> Filter { get; private set; }

			readonly FilterDefinitionBuilder<JobNotificationStateDocument> _filterBuilder;

			public JobNotificationStateQueryBuilder()
			{
				_filterBuilder = Builders<JobNotificationStateDocument>.Filter;
				Filter = _filterBuilder.Empty;
			}

			public IJobNotificationStateQueryBuilder AddJobFilter(JobId jobId)
			{
				Filter &= _filterBuilder.Eq(x => x.JobId, jobId);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddJobFilters(List<JobId> jobIds)
			{
				Filter &= _filterBuilder.In(x => x.JobId, jobIds);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddTemplateFilter(TemplateId templateId)
			{
				Filter &= _filterBuilder.Eq(x => x.TemplateId, templateId);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddTemplateFilters(List<TemplateId> templateId)
			{
				Filter &= _filterBuilder.In(x => x.TemplateId, templateId);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddRecipientFilter(string recipient)
			{
				Filter &= _filterBuilder.Eq(x => x.Recipient, recipient);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddChannelFilter(string channel)
			{
				Filter &= _filterBuilder.Eq(x => x.Channel, channel);
				return this;
			}

			public IJobNotificationStateQueryBuilder AddTimestampFilter(string ts)
			{
				Filter &= _filterBuilder.Eq(x => x.Ts, ts);
				return this;
			}
		}

		/// <inheritdoc/>
		class JobStepNotificationStateQueryBuilder : IJobStepNotificationStateQueryBuilder
		{
			public FilterDefinition<JobStepNotificationStateDocument> Filter { get; private set; }

			readonly FilterDefinitionBuilder<JobStepNotificationStateDocument> _filterBuilder;

			public JobStepNotificationStateQueryBuilder()
			{
				_filterBuilder = Builders<JobStepNotificationStateDocument>.Filter;
				Filter = _filterBuilder.Empty;
			}

			public IJobStepNotificationStateQueryBuilder AddJobFilter(JobId jobId)
			{
				Filter &= _filterBuilder.Eq(x => x.JobId, jobId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddTemplateFilter(TemplateId templateId)
			{
				Filter &= _filterBuilder.Eq(x => x.TemplateId, templateId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddRecipientFilter(string recipient)
			{
				Filter &= _filterBuilder.Eq(x => x.Recipient, recipient);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddJobStepFilter(JobStepId jobStepId)
			{
				Filter &= _filterBuilder.Eq(x => x.JobStepId, jobStepId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddGroupFilter(string group)
			{
				Filter &= _filterBuilder.Eq(x => x.Group, group);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddPlatformFilter(string platform)
			{
				Filter &= _filterBuilder.Eq(x => x.TargetPlatform, platform);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddBadgeFilter(string badge, bool shouldMatch)
			{
				if (shouldMatch)
				{
					Filter &= _filterBuilder.Eq(x => x.Badge, badge);
				}
				else
				{
					Filter &= _filterBuilder.Ne(x => x.Badge, badge);
				}
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddChannelFilter(string channel)
			{
				Filter &= _filterBuilder.Eq(x => x.Channel, channel);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddThreadTimestampFilter(string threadTs)
			{
				Filter &= _filterBuilder.Eq(x => x.ThreadTs, threadTs);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddTimestampFilter(string ts)
			{
				Filter &= _filterBuilder.Eq(x => x.Ts, ts);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddParentJobFilter(JobId jobId)
			{
				Filter &= _filterBuilder.Eq(x => x.ParentJobId, jobId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddParentJobTemplateFilter(TemplateId templateId)
			{
				Filter &= _filterBuilder.Eq(x => x.ParentJobTemplateId, templateId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddJobAndParentJobFilter(JobId jobId)
			{
				Filter &= _filterBuilder.Where(x => x.JobId == jobId || x.ParentJobId == jobId);
				return this;
			}

			public IJobStepNotificationStateQueryBuilder AddTemplateAndParentTemplateFilter(TemplateId templateId)
			{
				Filter &= _filterBuilder.Where(x => x.TemplateId == templateId || x.ParentJobTemplateId == templateId);
				return this;
			}
		}

		readonly IMongoCollection<JobNotificationStateDocument> _jobNotificationStates;
		readonly IMongoCollection<JobStepNotificationStateDocument> _jobStepNotificationStates;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobNotificationCollection(IMongoService mongoService, ILogger<JobNotificationCollection> logger)
		{
			_logger = logger;

			List<MongoIndex<JobNotificationStateDocument>> jobNotificationIndexes = new List<MongoIndex<JobNotificationStateDocument>>();
			jobNotificationIndexes.Add(keys => keys.Ascending(x => x.JobId).Ascending(x => x.TemplateId).Ascending(x => x.Recipient), unique: true);
			_jobNotificationStates = mongoService.GetCollection<JobNotificationStateDocument>("SlackExperimentalJob", jobNotificationIndexes);

			List<MongoIndex<JobStepNotificationStateDocument>> jobNotificationStepIndexes = new List<MongoIndex<JobStepNotificationStateDocument>>();
			jobNotificationStepIndexes.Add(keys => keys.Ascending(x => x.JobId).Ascending(x => x.TemplateId).Ascending(x => x.Recipient).Ascending(x => x.Group).Ascending(x => x.JobStepId), unique: true);
			_jobStepNotificationStates = mongoService.GetCollection<JobStepNotificationStateDocument>("SlackExperimentalJobStep", jobNotificationStepIndexes);
		}

		#region Job Notification

		/// <summary>
		/// Creates a <see cref="JobNotificationStateQueryBuilder"/> for creating a <see cref="JobNotificationStateDocument"/> query filter
		/// </summary>
		/// <returns>A <see cref="IJobNotificationStateQueryBuilder"/> builder</returns>
		public static IJobNotificationStateQueryBuilder CreateJobNotificationStateQueryBuilder()
		{
			return new JobNotificationStateQueryBuilder();
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJobNotificationStateRef>?> GetJobNotificationStatesAsync(IJobNotificationStateQueryBuilder builder, CancellationToken cancellationToken = default)
		{
			JobNotificationStateQueryBuilder? jobNotificationStateQueryBuilder = builder as JobNotificationStateQueryBuilder;
			if (jobNotificationStateQueryBuilder is null)
			{
				return null;
			}

			List<JobNotificationStateDocument> states = await _jobNotificationStates.Find(jobNotificationStateQueryBuilder.Filter).ToListAsync(cancellationToken);
			return (states.Count != 0) ? states : null;
		}

		/// <inheritdoc/>
		public async Task<IJobNotificationStateRef?> GetJobNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default)
		{
			JobNotificationStateQueryBuilder builder = new JobNotificationStateQueryBuilder();
			builder.AddJobFilter(jobId);
			builder.AddTemplateFilter(templateId);
			builder.AddRecipientFilter(recipient);
			IReadOnlyList<IJobNotificationStateRef>? states = await GetJobNotificationStatesAsync(builder, cancellationToken);
			if (states is null || states.Count == 0)
			{
				return null;
			}

			if (states.Count != 1)
			{
				// Log error that multiple records were found and could cause issues
				_logger.LogInformation("Found {Count} records when fetching for {JobId}:{TemplateId}:{Recipient}. Using first record which could cause issues.", states.Count, jobId, templateId, recipient);
			}

			return states[0];
		}

		/// <inheritdoc/>
		public async Task<IJobNotificationStateRef> AddOrUpdateJobNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, string channel, string timestamp, CancellationToken cancellationToken = default)
		{
			JobNotificationStateQueryBuilder builder = new JobNotificationStateQueryBuilder();
			builder.AddJobFilter(jobId);
			builder.AddTemplateFilter(templateId);
			builder.AddRecipientFilter(recipient);

			JobNotificationStateDocument? jobNotificationState = await _jobNotificationStates.Find(builder.Filter).FirstOrDefaultAsync(cancellationToken);
			if (jobNotificationState == null)
			{
				jobNotificationState = new JobNotificationStateDocument(jobId, templateId, recipient, channel, timestamp);
				await _jobNotificationStates.InsertOneAsync(jobNotificationState, null, cancellationToken);
				_logger.LogInformation("Posted message {StateId} (jobId: {JobId}, templateId: {TemplateId}, recipient: {Recipient}, messageId: {MessageId})",
										jobNotificationState.Id, jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, jobNotificationState.MessageId);
			}
			else
			{
				UpdateDefinitionBuilder<JobNotificationStateDocument> updateBuilder = Builders<JobNotificationStateDocument>.Update;
				UpdateDefinition<JobNotificationStateDocument> updates = updateBuilder.Set(x => x.Channel, channel).Set(x => x.Ts, timestamp);
				jobNotificationState = await _jobNotificationStates.FindOneAndUpdateAsync<JobNotificationStateDocument>(x => x.Id == jobNotificationState.Id, updates, new FindOneAndUpdateOptions<JobNotificationStateDocument> { IsUpsert = false, ReturnDocument = ReturnDocument.After }, cancellationToken);
				_logger.LogInformation("Updated message {StateId} (jobId: {JobId}, templateId: {TemplateId}, recipient: {Recipient}, messageId: {MessageId})",
										jobNotificationState.Id, jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, jobNotificationState.MessageId);
			}

			return jobNotificationState;
		}

		/// <inheritdoc/>
		public async Task<long> DeleteJobNotificationStatesAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default)
		{
			JobNotificationStateQueryBuilder builder = new JobNotificationStateQueryBuilder();
			builder.AddJobFilter(jobId);
			builder.AddTemplateFilter(templateId);
			builder.AddRecipientFilter(recipient);

			DeleteResult result = await _jobNotificationStates.DeleteManyAsync(builder.Filter, cancellationToken);

			_logger.LogInformation("Removed {MessageCount} JobNotificationState(s) for jobId: {JobId}, templateId: {TemplateId}, and recipient: {Recipient}", result.DeletedCount, jobId, templateId, recipient);

			return result.DeletedCount;
		}

		#endregion Job Notification

		#region Job Step Notification

		/// <summary>
		/// Creates a <see cref="JobStepNotificationStateQueryBuilder"/> for creating a <see cref="JobStepNotificationStateDocument"/> query filter
		/// </summary>
		/// <returns>A <see cref="IJobStepNotificationStateQueryBuilder"/> builder</returns>
		public static IJobStepNotificationStateQueryBuilder CreateJobStepNotificationStateQueryBuilder()
		{
			return new JobStepNotificationStateQueryBuilder();
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJobStepNotificationStateRef>?> GetJobStepNotificationStatesAsync(IJobStepNotificationStateQueryBuilder builder, CancellationToken cancellationToken = default)
		{
			JobStepNotificationStateQueryBuilder? jobStepNotificationStateQueryBuilder = builder as JobStepNotificationStateQueryBuilder;
			if (jobStepNotificationStateQueryBuilder is null)
			{
				return null;
			}

			// Sort the results for easy processing
			// When retrieving steps for both parent and spawned jobs, sorting ascending by PartentJobId will have the parent job steps on top of the list
			SortDefinition<JobStepNotificationStateDocument> sort = Builders<JobStepNotificationStateDocument>.Sort.Ascending(x => x.ParentJobId).Ascending(x => x.Recipient).Ascending(x => x.Group).Ascending(x => x.TargetPlatform);
			List<JobStepNotificationStateDocument> states = await _jobStepNotificationStates.Find(jobStepNotificationStateQueryBuilder.Filter).Sort(sort).ToListAsync(cancellationToken);
			return (states.Count != 0) ? states : null;
		}

		/// <inheritdoc/>
		public async Task<IJobStepNotificationStateRef> AddOrUpdateJobStepNotificationStateAsync(JobId jobId, TemplateId templateId, string recipient, JobStepId jobStepId, string group, string platform, string badge, string channel, string ts, string? threadTs = null, JobId? parentJobId = null, TemplateId? parentJobTemplateId = null, CancellationToken cancellationToken = default)
		{
			JobStepNotificationStateQueryBuilder filterBuilder = new JobStepNotificationStateQueryBuilder();
			filterBuilder.AddJobFilter(jobId);
			filterBuilder.AddTemplateFilter(templateId);
			filterBuilder.AddRecipientFilter(recipient);
			filterBuilder.AddJobStepFilter(jobStepId);
			filterBuilder.AddGroupFilter(group);
			filterBuilder.AddPlatformFilter(platform);

			JobStepNotificationStateDocument? jobStepNotificationState = await _jobStepNotificationStates.Find(filterBuilder.Filter).FirstOrDefaultAsync(cancellationToken);
			if (jobStepNotificationState == null)
			{
				jobStepNotificationState = new JobStepNotificationStateDocument(jobId, templateId, recipient, jobStepId, group, platform, badge, channel, ts, threadTs, parentJobId, parentJobTemplateId);
				await _jobStepNotificationStates.InsertOneAsync(jobStepNotificationState, null, cancellationToken);
				_logger.LogInformation("Posted message {StateId} (jobId: {JobId}, templateId: {TemplateId}, recipient: {Recipient}, jobStepId: {JobStepId}, group: {Group}, platform: {Platform}, badge: {Badge}, parentJobId: {ParentJobId}, parentJobTemplateId: {ParentJobTemplateId}, messageId: {MessageId})",
										jobStepNotificationState.Id, jobStepNotificationState.JobId, jobStepNotificationState.TemplateId, jobStepNotificationState.Recipient, jobStepNotificationState.JobStepId, jobStepNotificationState.Group, jobStepNotificationState.TargetPlatform, jobStepNotificationState.Badge, jobStepNotificationState.ParentJobId, jobStepNotificationState.ParentJobTemplateId, jobStepNotificationState.MessageId);
			}
			else
			{
				UpdateDefinitionBuilder<JobStepNotificationStateDocument> updateBuilder = Builders<JobStepNotificationStateDocument>.Update;
				UpdateDefinition<JobStepNotificationStateDocument> updates = updateBuilder.Set(x => x.Badge, badge).Set(x => x.Channel, channel).Set(x => x.ThreadTs, threadTs).Set(x => x.Ts, ts);
				jobStepNotificationState = await _jobStepNotificationStates.FindOneAndUpdateAsync<JobStepNotificationStateDocument>(x => x.Id == jobStepNotificationState.Id, updates, new FindOneAndUpdateOptions<JobStepNotificationStateDocument> { IsUpsert = false, ReturnDocument = ReturnDocument.After }, cancellationToken);
				_logger.LogInformation("Updated message {StateId} (jobId: {JobId}, templateId: {TemplateId}, recipient: {Recipient}, jobStepId: {JobStepId}, group: {Group}, platform: {Platform}, badge: {Badge}, parentJobId: {ParentJobId}, parentJobTemplateId: {ParentJobTemplateId}, messageId: {MessageId})",
										jobStepNotificationState.Id, jobStepNotificationState.JobId, jobStepNotificationState.TemplateId, jobStepNotificationState.Recipient, jobStepNotificationState.JobStepId, jobStepNotificationState.Group, jobStepNotificationState.TargetPlatform, jobStepNotificationState.Badge, jobStepNotificationState.ParentJobId, jobStepNotificationState.ParentJobTemplateId, jobStepNotificationState.MessageId);
			}

			return jobStepNotificationState;
		}

		/// <inheritdoc/>
		public async Task<long> DeleteJobStepNotificationStatesAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default)
		{
			JobStepNotificationStateQueryBuilder filterBuilder = new JobStepNotificationStateQueryBuilder();
			filterBuilder.AddRecipientFilter(recipient);
			filterBuilder.AddJobAndParentJobFilter(jobId);
			filterBuilder.AddTemplateAndParentTemplateFilter(templateId);

			DeleteResult result = await _jobStepNotificationStates.DeleteManyAsync(filterBuilder.Filter, cancellationToken);

			_logger.LogInformation("Removed {MessageCount} JobStepNotificationState(s) for jobId: {JobId}, templateId: {TemplateId}, and recipient: {Recipient}", result.DeletedCount, jobId, templateId, recipient);

			return result.DeletedCount;
		}

		#endregion Job Step Notification
	}
}
