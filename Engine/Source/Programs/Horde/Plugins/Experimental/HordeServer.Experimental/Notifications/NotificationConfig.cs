// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Configuration;
using HordeServer.Streams;
using HordeServer.Utilities;
using System.Diagnostics.CodeAnalysis;
using System.Text.RegularExpressions;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the format of the notification group
	/// </summary>
	public class NotificationFormatConfig
	{
		/// <summary>
		/// Name of the group
		/// </summary>
		public string Group { get; set; } = String.Empty;

		/// <summary>
		/// Regex pattern to determine if a job step belongs within the group
		/// </summary>
		public string StepPattern { get; set; } = String.Empty;

		/// <summary>
		/// Optional regex pattern to override the <see cref="JobNotificationConfig.NamePattern"/>
		/// </summary>
		public string? AlternativeNamePattern { get; set; }
	}

	/// <summary>
	/// Configuration for the job notification
	/// </summary>
	public class JobNotificationConfig
	{
		/// <summary>
		/// Whether or not the current configuration is enabled
		/// </summary>
		public bool Enabled { get; set; } = true;

		/// <summary>
		/// Identifier of the template this notification applies to
		/// </summary>
		public TemplateId Template { get; set; }

		/// <summary>
		/// Regex pattern used to extract name of the step this notification will use for display
		/// </summary>
		public string NamePattern { get; set; } = String.Empty;

		/// <summary>
		/// List of formats to be applied for this notification
		/// </summary>
		public List<NotificationFormatConfig> NotificationFormats { get; set; } = new List<NotificationFormatConfig>();

		/// <summary>
		/// List of Slack channel or user identifiers the notification will be sent to
		/// </summary>
		public List<string> Channels { get; set; } = new List<string>();

		/// <summary>
		/// Gets the group and platform information of the corresponding step within the list of notification formats
		/// </summary>
		/// <param name="stepName">Name of the job step to test against</param>
		/// <param name="group">Name of the notification group the job step belongs to or null if not found</param>
		/// <param name="name">Name associated with the job step or null if not found</param>
		/// <returns>True if the job step is to be included in the notification or false otherwise</returns>
		public bool GetStepDetails(string stepName, out string? group, out string? name)
		{
			group = null;
			name = null;

			foreach (NotificationFormatConfig formatConfig in NotificationFormats)
			{
				Match monitoredPattern = Regex.Match(stepName, formatConfig.StepPattern);
				if (monitoredPattern.Success)
				{
					// Fetch our platform from the step name
					string namePattern = !String.IsNullOrEmpty(formatConfig.AlternativeNamePattern) ? formatConfig.AlternativeNamePattern : NamePattern;
					Match match = Regex.Match(stepName, namePattern);
					if (match.Success)
					{
						group = formatConfig.Group;
						name = match.Groups[0].Value;

						return true;
					}
				}
			}

			return false;
		}
	}

	/// <summary>
	/// Config for the notifications based on the streams
	/// </summary>
	public class NotificationStreamConfig
	{
		/// <summary>
		/// List of streams for this notification
		/// </summary>
		public List<StreamId> Streams { get; set; } = new List<StreamId>();

		/// <summary>
		/// List of enabled tags for a stream for this notification
		/// </summary>
		public List<string> EnabledStreamTags { get; set; } = new List<string>();

		/// <summary>
		/// Configuration for sending experimental Job Summary Slack notifications
		/// </summary>
		public List<JobNotificationConfig> JobNotifications { get; set; } = new List<JobNotificationConfig>();

		readonly Dictionary<TemplateId, List<JobNotificationConfig>> _notificationLookup = new Dictionary<TemplateId, List<JobNotificationConfig>>();

		/// <summary>
		/// Called after the store has been deserialized to compute cached values
		/// </summary>
		public void PostLoad()
		{
			_notificationLookup.Clear();
			foreach (JobNotificationConfig notificationConfig in JobNotifications)
			{
				if (!_notificationLookup.TryGetValue(notificationConfig.Template, out List<JobNotificationConfig>? configs))
				{
					configs = new List<JobNotificationConfig>();
					_notificationLookup[notificationConfig.Template] = configs;
				}

				configs.Add(notificationConfig);
			}
		}

		/// <summary>
		/// Attempt to get config for a given template
		/// </summary>
		/// <param name="templateId">Template id</param>
		/// <param name="notificationConfigList">Receives the list of config objects on success</param>
		/// <returns>True if the stream was found</returns>
		public bool TryGetTemplateConfigs(TemplateId templateId, [NotNullWhen(true)] out List<JobNotificationConfig>? notificationConfigList)
			=> _notificationLookup.TryGetValue(templateId, out notificationConfigList);
	}

	/// <summary>
	/// Config for notifications
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/notification")]
	[JsonSchemaCatalog("Horde Notifications", "Horde notification configuration file", new[] { "*.notification.json", "Notifications/*.json" })]
	[ConfigDoc("*.notification.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Notification.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class NotificationConfig
	{
		/// <summary>
		/// Identifier for this store
		/// </summary>
		public NotificationConfigId Id { get; set; }

		/// <summary>
		/// Streams for the notifications to be sent on the Horde server
		/// </summary>
		public List<NotificationStreamConfig> NotificationStreams { get; set; } = new List<NotificationStreamConfig>();

		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		readonly Dictionary<StreamId, HashSet<NotificationStreamConfig>> _streamLookup = new Dictionary<StreamId, HashSet<NotificationStreamConfig>>();
		readonly Dictionary<string, HashSet<NotificationStreamConfig>> _streamTagLookup = new Dictionary<string, HashSet<NotificationStreamConfig>>();

		/// <summary>
		/// Called after the store has been deserialized to compute cached values
		/// </summary>
		public void PostLoad()
		{
			_streamLookup.Clear();
			_streamTagLookup.Clear();
			foreach (NotificationStreamConfig notificationStream in NotificationStreams)
			{
				notificationStream.PostLoad();

				foreach (StreamId streamId in notificationStream.Streams)
				{
					if (!_streamLookup.TryGetValue(streamId, out HashSet<NotificationStreamConfig>? configs))
					{
						configs = new HashSet<NotificationStreamConfig>();
						_streamLookup[streamId] = configs;
					}

					configs.Add(notificationStream);
				}

				foreach (string streamTag in notificationStream.EnabledStreamTags)
				{
					if (!_streamTagLookup.TryGetValue(streamTag, out HashSet<NotificationStreamConfig>? configs))
					{
						configs = new HashSet<NotificationStreamConfig>();
						_streamTagLookup[streamTag] = configs;
					}

					configs.Add(notificationStream);
				}
			}
		}

		/// <summary>
		/// Attempt to get config for a given stream and template
		/// </summary>
		/// <param name="streamId">Stream id</param>
		/// <param name="templateId">Template id</param>
		/// <param name="jobNotificationConfigs">Receives the list of config objects on success</param>
		/// <param name="includeDisabledConfigs">Flag on whether or not disabled configs should be included</param>
		/// <returns>True if the job notification configs were found</returns>
		public bool TryGetJobNofications(StreamId streamId, TemplateId templateId, [NotNullWhen(true)] out HashSet<JobNotificationConfig>? jobNotificationConfigs, bool includeDisabledConfigs = false)
		{
			jobNotificationConfigs = new HashSet<JobNotificationConfig>();

			HashSet<NotificationStreamConfig>? streamConfigList;
			if (!_streamLookup.TryGetValue(streamId, out streamConfigList))
			{
				return false;
			}

			foreach (NotificationStreamConfig streamConfig in streamConfigList)
			{
				List<JobNotificationConfig>? notificationConfigList;
				if (streamConfig.TryGetTemplateConfigs(templateId, out notificationConfigList))
				{
					foreach (JobNotificationConfig notificationConfig in notificationConfigList)
					{
						if (notificationConfig.Enabled || includeDisabledConfigs)
						{
							jobNotificationConfigs.Add(notificationConfig);
						}
					}
				}
			}

			return jobNotificationConfigs.Count != 0;
		}

		/// <summary>
		/// Attempt to get config for a given list of stream tags and template
		/// </summary>
		/// <param name="streamTags">List of tags for a stream</param>
		/// <param name="templateId">Template id</param>
		/// <param name="jobNotificationConfigs">Receives the list of config objects on success</param>
		/// <param name="includeDisabledConfigs">Flag on whether or not disabled configs should be included</param>
		/// <returns>True if the job notification configs were found</returns>
		public bool TryGetJobNofications(List<StreamTag> streamTags, TemplateId templateId, [NotNullWhen(true)] out HashSet<JobNotificationConfig>? jobNotificationConfigs, bool includeDisabledConfigs = false)
		{
			jobNotificationConfigs = new HashSet<JobNotificationConfig>();

			if (_streamTagLookup.Count == 0)
			{
				return false;
			}

			HashSet<NotificationStreamConfig>? streamConfigList;
			foreach (StreamTag streamTag in streamTags)
			{
				if (!streamTag.Enabled || !_streamTagLookup.TryGetValue(streamTag.Name, out streamConfigList))
				{
					continue;
				}

				foreach (NotificationStreamConfig streamConfig in streamConfigList)
				{
					List<JobNotificationConfig>? notificationConfigList;
					if (streamConfig.TryGetTemplateConfigs(templateId, out notificationConfigList))
					{
						foreach (JobNotificationConfig notificationConfig in notificationConfigList)
						{
							if (notificationConfig.Enabled || includeDisabledConfigs)
							{
								jobNotificationConfigs.Add(notificationConfig);
							}
						}
					}
				}
			}

			return jobNotificationConfigs.Count != 0;
		}
	}
}
