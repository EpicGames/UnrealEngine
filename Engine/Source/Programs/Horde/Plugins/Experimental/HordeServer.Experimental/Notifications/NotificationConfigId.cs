// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Serialization;
using System.ComponentModel;

namespace HordeServer
{
	/// <summary>
	/// Identifier for a notification configuration
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<NotificationConfigId, NotificationStoreIdConverter>))]
	[StringIdConverter(typeof(NotificationStoreIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<NotificationConfigId, NotificationStoreIdConverter>))]
	public readonly record struct NotificationConfigId(StringId Id)
	{
		/// <summary>
		/// Default notification configuration identifier
		/// </summary>
		public static NotificationConfigId Default { get; } = new NotificationConfigId("default");

		/// <summary>
		/// Constructor
		/// </summary>
		public NotificationConfigId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	public class NotificationStoreIdConverter : StringIdConverter<NotificationConfigId>
	{
		/// <inheritdoc/>
		public override NotificationConfigId FromStringId(StringId id) => new NotificationConfigId(id);

		/// <inheritdoc/>
		public override StringId ToStringId(NotificationConfigId value) => value.Id;
	}
}
