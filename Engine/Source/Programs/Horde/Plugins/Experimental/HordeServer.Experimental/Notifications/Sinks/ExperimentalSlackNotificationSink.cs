// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Experimental.Notifications;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Notifications.Sinks
{
	/// <summary>
	/// Maintains a connection to Slack, in order to receive socket-mode notifications of user interactions
	/// </summary>
	public sealed class ExperimentalSlackNotificationSink : INotificationSink, IDisposable
	{
		readonly SlackNotificationProcessor _slackNotificationProcessor;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buildServerConfig">Options from the build plugin's server config</param>
		/// <param name="buildConfig">Monitored options from the build plugin's global config</param>
		/// <param name="experimentalConfig">Monitored options from the experimental plugin's global config</param>
		/// <param name="jobNotificationCollection">Collection of job and job step notifications</param>
		/// <param name="jobCollection">Collection of jobs</param>
		/// <param name="userCollection">Collection of users</param>
		/// <param name="serverInfo">Information of the current server instance</param>
		/// <param name="logger">Logger for output</param>
		public ExperimentalSlackNotificationSink(IOptions<BuildServerConfig> buildServerConfig,
			IOptionsMonitor<BuildConfig> buildConfig,
			IOptionsMonitor<ExperimentalConfig> experimentalConfig,
			IJobNotificationCollection jobNotificationCollection,
			IJobCollection jobCollection,
			IUserCollection userCollection,
			IServerInfo serverInfo,
			ILogger<ExperimentalSlackNotificationSink> logger)
		{
			_slackNotificationProcessor = new SlackNotificationProcessor(buildServerConfig, buildConfig, experimentalConfig, jobNotificationCollection, jobCollection, userCollection, serverInfo, logger);
		}

		/// <inheritdoc/>
		public Task NotifyConfigUpdateAsync(ConfigUpdateInfo info, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task NotifyConfigUpdateFailureAsync(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null, CancellationToken cancellationToken = default)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task NotifyDeviceServiceAsync(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task NotifyIssueUpdatedAsync(IIssue issue, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public async Task NotifyJobCompleteAsync(IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			await _slackNotificationProcessor.ProcessJobCompleteAsync(job, cancellationToken);
		}

		/// <inheritdoc/>
		public Task NotifyJobCompleteAsync(IUser user, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task NotifyJobScheduledAsync(List<JobScheduledNotification> notifications, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public async Task NotifyJobStepAbortedAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			// Currently we only want to notify from the plugin's configuration which would not have this set
			if (usersToNotify is not null)
			{
				return;
			}

			await _slackNotificationProcessor.ProcessJobStepAbortedAsync(job, step, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task NotifyJobStepCompleteAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			// Currently we only want to notify from the plugin's configuration which would not have this set
			if (usersToNotify is not null)
			{
				return;
			}

			await _slackNotificationProcessor.ProcessJobStepCompleteAsync(job, step, jobStepEventData, cancellationToken);
		}

		/// <inheritdoc/>
		public Task NotifyLabelCompleteAsync(IUser user, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> stepData, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		/// <remarks>No-op: this sink only handles job completion/step notifications via the processor. Session conflict reports are sent by the primary SlackNotificationSink.</remarks>
		public Task SendSessionConflictReportAsync(IReadOnlyList<(AgentId Id, int Count)> conflicts, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken = default)
			=> Task.FromResult<string?>(null);

		/// <inheritdoc/>
		public Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken = default)
			=> Task.FromResult<string?>(null);

		/// <inheritdoc/>
		public void Dispose()
			=> _slackNotificationProcessor.Dispose();
	}
}
