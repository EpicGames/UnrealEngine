// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Slack;
using EpicGames.Slack.Blocks;
using EpicGames.Slack.Elements;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Experimental.Notifications
{
	/// <summary>
	/// Processes notifications to be sent through the Slack API
	/// </summary>
	public sealed class SlackNotificationProcessor : IDisposable
	{
		/// <summary>
		/// Structure for generic use of log reporting
		/// </summary>
		/// <remarks>Cancellation of jobs and job steps may not always include log information. In these cases we want to still provide a message to the user for the reason.</remarks>
		internal struct LogData
		{
			public LogEventSeverity Severity { get; }
			public string Message { get; }
			public LogId? LogId { get; } = null;
			public LogData(LogEventSeverity severity, string message, LogId? logId = null)
			{
				Severity = severity;
				Message = message;
				LogId = logId;
			}
		}

		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IOptionsMonitor<ExperimentalConfig> _experimentalConfig;
		readonly IJobNotificationCollection _jobNotificationCollection;
		readonly IJobCollection _jobCollection;
		readonly IUserCollection _userCollection;
		readonly IServerInfo _serverInfo;
		readonly ILogger _logger;
		readonly SlackClient? _slackClient;

		/// <summary>
		/// Slack emoji used to emphasize state
		/// </summary>
		/// <remarks>All emoji were verified against what is available in Slack by default</remarks>
		const string PendingBadge = ":large_blue_square:";
		const string WarningBadge = ":large_yellow_square:";
		const string FailureBadge = ":large_red_square:";
		const string SkippedBadge = ":black_square:";
		const string PassingBadge = ":large_green_square:";

		/// <summary>
		/// Slack emoji used to emphasize an aborted job
		/// </summary>
		const string AlertBadge = ":rotating_light:";

		/// <summary>
		/// Limit of logs to be provided in a Slack notification
		/// </summary>
		const int MaxEventMessages = 5;

		/// <summary>
		/// Length limit to avoid character limit on each individual block container from Slack
		/// </summary>
		const int MaxBlockCharacterLength = 2048;

		/// <summary>
		/// Block limit per Slack Context before an exception is thrown
		/// </summary>
		/// <remarks>See https://api.slack.com/reference/block-kit/blocks#context for detailed information</remarks>
		const int MaxSlackBlocks = 10;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buildServerConfig">Options from the build plugin's server config</param>
		/// <param name="buildConfig">Monitored options from the build plugin's global config</param>
		/// <param name="experimentalConfig">Monitored options from the experimental plugin's global config</param>
		/// <param name="jobNotificationCollection">Collection of job and job step notifications</param>
		/// <param name="jobCollection">Collection of jobs</param>
		/// <param name="userCollection">Collection of users</param>
		/// <param name="serverInfo">Information of the current server instance</param>
		/// <param name="logger">Logger for output</param>
		public SlackNotificationProcessor(IOptions<BuildServerConfig> buildServerConfig,
			IOptionsMonitor<BuildConfig> buildConfig,
			IOptionsMonitor<ExperimentalConfig> experimentalConfig,
			IJobNotificationCollection jobNotificationCollection,
			IJobCollection jobCollection,
			IUserCollection userCollection,
			IServerInfo serverInfo,
			ILogger logger)
		{
			_buildConfig = buildConfig;
			_experimentalConfig = experimentalConfig;
			_jobNotificationCollection = jobNotificationCollection;
			_jobCollection = jobCollection;
			_userCollection = userCollection;
			_serverInfo = serverInfo;
			_logger = logger;

			if (!String.IsNullOrEmpty(buildServerConfig.Value.SlackToken))
			{
				_slackClient = SlackClient.Create(buildServerConfig.Value.SlackToken, _logger);
			}
		}

		/// <inheritdoc/>
		public void Dispose() => _slackClient?.Dispose();

		/// <summary>
		/// Processes a Job complete notification
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <remarks>Currently this should only be called if a job is canceled. Completed jobs should still receive a job step to be processed.</remarks>
		public async Task ProcessJobCompleteAsync(IJob job, CancellationToken cancellationToken = default)
		{
			// Only handle the job complete notification if the job was canceled
			// Otherwise we will leave it to the steps to handle when all notifications have been sent out to avoid conditions where the job complete notification gets received before the job step complete notification
			if (job.AbortedByUserId is null)
			{
				return;
			}

			bool wasSpawnJobProcessed = await TryProcessSpawnJobCompleteNotificationAsync(job, cancellationToken);
			if (wasSpawnJobProcessed)
			{
				_logger.LogInformation("Job complete notification for '{JobId}' processed for parent job '{ParentJobId}'", job.Id, job.ParentJobId);
			}

			IJobNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			builder.AddJobFilter(job.Id);
			builder.AddTemplateFilter(job.TemplateId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await _jobNotificationCollection.GetJobNotificationStatesAsync(builder, cancellationToken);
			if (jobNotificationStates is null)
			{
				_logger.LogInformation("No job notifications found for job id '{JobId}'.", job.Id);
				return;
			}

			string abortedByUserName = "Unknown User";
			IUser? user = await _userCollection.GetUserAsync(job.AbortedByUserId.Value, cancellationToken);
			if (user is not null)
			{
				abortedByUserName = user.Name;
			}

			string branch = GetBranchFromStream(_buildConfig.CurrentValue, job.StreamId);
			foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
			{
				// Update our steps to reflect the canceled job
				_ = await UpdatePendingJobStepNotificationsAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, cancellationToken);

				await SendSlackSummaryMessageAsync(job, branch, jobNotificationState.Recipient, jobNotificationState, null, abortedByUserName, cancellationToken);

				// No further notifications will be sent out and can be cleaned up
				_ = await _jobNotificationCollection.DeleteJobStepNotificationStatesAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, cancellationToken);
				_ = await _jobNotificationCollection.DeleteJobNotificationStatesAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, cancellationToken);
			}
		}

		/// <summary>
		/// Processes a notification for a job step that has been canceled
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <remarks>A job step that has been canceled will be handled the same way as if the job step has failures.</remarks>
		public async Task ProcessJobStepAbortedAsync(IJob job, IJobStep step, CancellationToken cancellationToken = default)
		{
			List<LogData> logEvents = new List<LogData>();
			string cancellationDetails = "Step was canceled.";
			if (step.AbortedByUserId is not null)
			{
				IUser? user = await _userCollection.GetUserAsync(step.AbortedByUserId.Value, cancellationToken);
				if (user is not null)
				{
					cancellationDetails = $"Step was canceled by {user.Name}.";
				}
			}

			if (!String.IsNullOrEmpty(step.CancellationReason))
			{
				cancellationDetails += $" Reason for cancellation: '{step.CancellationReason}'.";
			}
			logEvents.Add(new LogData(LogEventSeverity.Error, cancellationDetails));
			await ProcessJobStepCompleteNotificationAsync(job, step, logEvents, cancellationToken);
		}

		/// <summary>
		/// Process a job step that has been completed
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task ProcessJobStepCompleteAsync(IJob job, IJobStep step, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken = default)
		{
			List<LogData> logEvents = ExtractLogInformation(step, jobStepEventData);
			await ProcessJobStepCompleteNotificationAsync(job, step, logEvents, cancellationToken);
		}

		#region Job Processing

		/// <summary>
		/// Search the job for steps to be included in the notifications based on the experimental plugin's global config
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="config">Configuration to fetch notification information</param>
		/// <param name="parentJob">Optional information regarding a parent job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task ProcessJobAsync(IJob job, JobNotificationConfig config, IJob? parentJob = null, CancellationToken cancellationToken = default)
		{
			// Populate our job information needed for the notification
			try
			{
				foreach (IJobStepBatch batch in job.Batches)
				{
					foreach (IJobStep step in batch.Steps)
					{
						if (config.GetStepDetails(step.Name, out string? group, out string? platform) && !String.IsNullOrEmpty(group) && !String.IsNullOrEmpty(platform))
						{
							string badge = GetBadgeForStep(step);
							foreach (string recipient in config.Channels)
							{
								await _jobNotificationCollection.AddOrUpdateJobStepNotificationStateAsync(job.Id, job.TemplateId, recipient, step.Id, group, platform, badge, String.Empty, String.Empty, null, parentJob?.Id, parentJob?.TemplateId, cancellationToken);
							}
						}
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Failed to extract step details for '{Job}' in stream '{StreamId}' and for template '{TemplateId}' ({Error})", job.Id, job.StreamId, job.TemplateId, ex.Message);
			}
		}

		/// <summary>
		/// Try to process the job complete notification as a spawned job
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True is the job was found to be a valid spawned job, otherwise, will return false.</returns>
		async Task<bool> TryProcessSpawnJobCompleteNotificationAsync(IJob job, CancellationToken cancellationToken = default)
		{
			// Fetch the parent information if available
			if (job.ParentJobId is null || job.ParentJobStepId is null)
			{
				return false;
			}

			IJob? parentJob = await _jobCollection.GetAsync(job.ParentJobId!.Value, cancellationToken);
			if (parentJob is null)
			{
				_logger.LogInformation("Unable to process job complete notification as parent job was not found for job '{JobId}' and parent job '{ParentJob}'", job.Id, job.ParentJobId);
				return false;
			}

			IJobNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			builder.AddJobFilter(parentJob.Id);
			builder.AddTemplateFilter(parentJob.TemplateId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await _jobNotificationCollection.GetJobNotificationStatesAsync(builder, cancellationToken);
			if (jobNotificationStates is null)
			{
				_logger.LogInformation("No active notifications found for parent job '{ParentJob}' with parent job template '{ParentJobTemplateId}' in spawned job '{JobId}' with spawned job template '{TemplateId}'", parentJob.Id, parentJob.TemplateId, job.Id, job.TemplateId);
				return false;
			}

			string branch = GetBranchFromStream(_buildConfig.CurrentValue, parentJob.StreamId);
			foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
			{
				// No further notifications will be sent out and can be cleaned up
				_ = await _jobNotificationCollection.DeleteJobStepNotificationStatesAsync(job.Id, job.TemplateId, jobNotificationState.Recipient, cancellationToken);

				// Send out an updated summary to include the current job has been canceled
				await SendSlackSummaryMessageAsync(parentJob, branch, jobNotificationState.Recipient, jobNotificationState, null, null, cancellationToken);
			}

			return true;
		}

		/// <summary>
		/// Try to process the job setup notification as a spawned job
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True is the job was found to be a valid spawned job, otherwise, will return false.</returns>
		async Task<bool> TryProcessSpawnJobSetupNotificationAsync(IJob job, IJobStep step, List<LogData> jobStepEventData, CancellationToken cancellationToken = default)
		{
			// Fetch the parent information if available
			if (job.ParentJobId is null || job.ParentJobStepId is null)
			{
				return false;
			}

			// Fetch all of our notifications for the parent job and parent job step to see if we need to consolidate jobs
			IJobStepNotificationStateQueryBuilder jobStepNotificationBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			jobStepNotificationBuilder.AddJobFilter(job.ParentJobId!.Value);
			jobStepNotificationBuilder.AddJobStepFilter(job.ParentJobStepId!.Value);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(jobStepNotificationBuilder, cancellationToken);
			if (jobStepNotificationStates is null || jobStepNotificationStates.Count == 0)
			{
				_logger.LogInformation("No active notifications found for parent step '{ParentJobStep}' in parent job '{ParentJob}' for spawned job step '{JobStepId}' in spawned job '{JobId}'", job.ParentJobStepId, job.ParentJobId, step.Id, job.Id);
				return false;
			}

			IJob? parentJob = await _jobCollection.GetAsync(job.ParentJobId!.Value, cancellationToken);
			if (parentJob is null)
			{
				_logger.LogInformation("Unable to process spawned setup as parent job was not found for job '{JobId}' and parent job '{ParentJob}'", job.Id, job.ParentJobId);
				return false;
			}

			IJobNotificationStateQueryBuilder jobNotificationBuilder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			jobNotificationBuilder.AddJobFilter(parentJob.Id);
			jobNotificationBuilder.AddTemplateFilter(parentJob.TemplateId);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await _jobNotificationCollection.GetJobNotificationStatesAsync(jobNotificationBuilder, cancellationToken);
			if (jobNotificationStates is null)
			{
				_logger.LogInformation("No job notifications found for parent job id '{JobId}'.", job.ParentJobId);
				return false;
			}

			// From this point forward the current job/step will be handled as a spawned job

			StreamConfig? streamConfig = GetStreamConfig(_buildConfig.CurrentValue, parentJob.StreamId);
			string branch = GetBranchFromStream(streamConfig);

			// Check if the Setup step failed to notify that the job can't be executed
			bool wasSetupSuccess = (step.Outcome == JobStepOutcome.Success || step.Outcome == JobStepOutcome.Warnings);
			if (!wasSetupSuccess)
			{
				// Send the failed setup job to the thread and update the summary
				foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
				{
					await SendSetupFailedSlackMessageToThreadAsync(job, step, jobStepEventData, jobNotificationState.Channel, jobNotificationState.Ts, cancellationToken);
					await SendSlackSummaryMessageAsync(job, branch, jobNotificationState.Recipient, jobNotificationState, parentJob, null, cancellationToken);

					// Check to see if all notifications for this job were sent then delete all documents related to this job as no other notifications will be sent
					await TryNotificationCleanupAsync(parentJob, jobNotificationState.Recipient, cancellationToken);
				}
				return true;
			}

			// Fetch our notification configurations for the parent job to append our current job's steps
			ExperimentalConfig currentExperimentalConfig = _experimentalConfig.CurrentValue;

			string streamInformation = $"'{parentJob.StreamId}'";
			if (streamConfig is not null && streamConfig.StreamTags.Count > 0)
			{
				streamInformation += $" or with stream tags {String.Join(',', streamConfig.StreamTags.Where(x => x.Enabled).Select(x => x.Name))}";
			}

			HashSet<JobNotificationConfig>? jobNotificationConfigs;
			if (!currentExperimentalConfig.TryGetJobNofications(parentJob.StreamId, parentJob.TemplateId, out jobNotificationConfigs, streamConfig?.StreamTags ?? null))
			{
				_logger.LogInformation("Parent configurations not found for parent step '{ParentJobStep}' in parent job '{ParentJob}' for '{JobId}' in stream {StreamInformation} and for template '{TemplateId}'", job.ParentJobStepId, job.ParentJobId, job.Id, streamInformation, parentJob.TemplateId);
				return true;
			}
			_logger.LogInformation("{NumConfigurations} parent configurations found for parent step '{ParentJobStep}' in parent job '{ParentJob}' for '{JobId}' in stream {StreamInformation} and for template '{TemplateId}'", jobNotificationConfigs.Count, job.ParentJobStepId, job.ParentJobId, job.Id, streamInformation, parentJob.TemplateId);

			foreach (JobNotificationConfig config in jobNotificationConfigs)
			{
				await ProcessJobAsync(job, config, parentJob, cancellationToken);
			}

			// Send the updated summary after the steps has been updated
			foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
			{
				await SendSlackSummaryMessageAsync(job, branch, jobNotificationState.Recipient, jobNotificationState, parentJob, null, cancellationToken);

				// Check to see if all notifications for this job were sent then delete all documents related to this job as no other notifications will be sent
				await TryNotificationCleanupAsync(parentJob, jobNotificationState.Recipient, cancellationToken);
			}

			return true;
		}

		/// <summary>
		/// Process the job setup notification
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task ProcessJobSetupNotificationAsync(IJob job, IJobStep step, List<LogData> jobStepEventData, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig = GetStreamConfig(_buildConfig.CurrentValue, job.StreamId);

			// Fetch our notification configurations for the parent job to append our current job's steps
			ExperimentalConfig currentExperimentalConfig = _experimentalConfig.CurrentValue;

			string streamInformation = $"'{job.StreamId}'";
			if (streamConfig is not null && streamConfig.StreamTags.Count > 0)
			{
				streamInformation += $" or with stream tags {String.Join(',', streamConfig.StreamTags.Where(x => x.Enabled).Select(x => x.Name))}";
			}

			HashSet<JobNotificationConfig>? jobNotificationConfigs;
			if (!currentExperimentalConfig.TryGetJobNofications(job.StreamId, job.TemplateId, out jobNotificationConfigs, streamConfig?.StreamTags ?? null))
			{
				_logger.LogInformation("No configurations found for '{JobId}' in stream '{StreamInformation}' and for template '{TemplateId}'", job.Id, streamInformation, job.TemplateId);
				return;
			}

			_logger.LogInformation("{NumConfigurations} configurations found for '{JobId}' in stream '{StreamInformation}' and for template '{TemplateId}'", jobNotificationConfigs.Count, job.Id, streamInformation, job.TemplateId);

			string branch = GetBranchFromStream(_buildConfig.CurrentValue, job.StreamId);

			// Check if the Setup step failed to notify that the job can't be executed
			bool wasSetupSuccess = (step.Outcome == JobStepOutcome.Success || step.Outcome == JobStepOutcome.Warnings);
			if (!wasSetupSuccess)
			{
				await SendSetupFailedSlackMessageAsync(job, step, branch, jobStepEventData, jobNotificationConfigs, cancellationToken);
				return;
			}

			foreach (JobNotificationConfig config in jobNotificationConfigs)
			{
				await ProcessJobAsync(job, config, null, cancellationToken);
				foreach (string recipient in config.Channels)
				{
					await SendSlackSummaryMessageAsync(job, branch, recipient, null, null, null, cancellationToken);
				}
			}
		}

		/// <summary>
		/// Process the job step completed notification
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task ProcessJobStepCompleteNotificationAsync(IJob job, IJobStep step, List<LogData> jobStepEventData, CancellationToken cancellationToken = default)
		{
			// Perform a check to see if the completed step was the Setup step for the job, as this will use the configurations to populate the notification information
			if (step.Name == IJob.SetupNodeName)
			{
				// Check to see if we're a spawned job which requires the job to be merged into a parent job
				bool wasSpawnJobProcessed = await TryProcessSpawnJobSetupNotificationAsync(job, step, jobStepEventData, cancellationToken);
				if (wasSpawnJobProcessed)
				{
					_logger.LogInformation("Job '{JobId}' processed for parent job '{ParentJobId}'", job.Id, job.ParentJobId);
				}
				else
				{
					await ProcessJobSetupNotificationAsync(job, step, jobStepEventData, cancellationToken);
				}
				return;
			}

			// From here we already have a summary notification sent and we need to focus on updating our notification

			// Send a message for our current step
			bool needsSummaryUpdate = await UpdateAndSendSlackNotificationsForStepAsync(job, step, jobStepEventData, null, cancellationToken);

			// Fetch all dependent steps if there was an error and send notifications with this step as the cause for the dependent steps being skipped
			if (step.Outcome != JobStepOutcome.Success && step.Outcome != JobStepOutcome.Warnings)
			{
				HashSet<JobStepId>? dependentSteps = await FindDependentJobStepIdsAsync(job, step.Id, cancellationToken);
				if (dependentSteps is not null)
				{
					foreach (JobStepId stepId in dependentSteps)
					{
						// Send a message for our dependent step
						if (job.TryGetStep(stepId, out IJobStep? dependentStep))
						{
							needsSummaryUpdate |= await UpdateAndSendSlackNotificationsForStepAsync(job, dependentStep, jobStepEventData, step, cancellationToken);
						}
					}
				}
			}

			// Get all of the job information such as identifier and template identifier
			List<JobId> jobIds = new List<JobId>()
			{
				job.Id
			};

			List<TemplateId> templateIds = new List<TemplateId>()
			{
				job.TemplateId
			};

			// Check if the job has a parent to include in the list when filtering for job notifications
			IJob? parentJob = null;
			if (job.ParentJobId is not null)
			{
				parentJob = await _jobCollection.GetAsync(job.ParentJobId!.Value, cancellationToken);
				if (parentJob is not null)
				{
					jobIds.Add(parentJob.Id);
					templateIds.Add(parentJob.TemplateId);
				}
			}

			IJobNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobNotificationStateQueryBuilder();
			builder.AddJobFilters(jobIds);
			builder.AddTemplateFilters(templateIds);
			IReadOnlyList<IJobNotificationStateRef>? jobNotificationStates = await _jobNotificationCollection.GetJobNotificationStatesAsync(builder, cancellationToken);
			if (jobNotificationStates is not null && job.GetState() == JobState.Complete)
			{
				foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
				{
					needsSummaryUpdate |= await UpdatePendingJobStepNotificationsAsync(jobNotificationState.JobId, jobNotificationState.TemplateId, jobNotificationState.Recipient, cancellationToken);
				}
			}

			// Check to see if any of the steps were updated which requires the Slack summary to be updated
			if (!needsSummaryUpdate)
			{
				return;
			}

			// Send the updated summary after the steps has been updated
			if (jobNotificationStates is null)
			{
				_logger.LogInformation("No job notifications found for job id '{JobId}' even though job step '{StepId}' was updated.", job.Id, step.Id);
				return;
			}

			string branch = GetBranchFromStream(_buildConfig.CurrentValue, job.StreamId);
			foreach (IJobNotificationStateRef jobNotificationState in jobNotificationStates)
			{
				// Check to see if the current job is the main job or if it was spawned from the parent
				bool isSpawnedJob = parentJob is not null && jobNotificationState.JobId == parentJob.Id;
				IJob notificationJob = isSpawnedJob ? parentJob! : job;

				await SendSlackSummaryMessageAsync(job, branch, jobNotificationState.Recipient, jobNotificationState, isSpawnedJob ? parentJob : null, null, cancellationToken);

				// Check to see if all notifications for this job were sent then delete all documents related to this job as no other notifications will be sent
				await TryNotificationCleanupAsync(notificationJob, jobNotificationState.Recipient, cancellationToken);
			}
		}

		/// <summary>
		/// Finds all dependent steps of the <see cref="IJobStepNotificationStateRef"/>
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="underlyingStepId">Identifier of the possible job step dependency</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>HashSet of <see cref="JobStepId"/> if dependent steps have been found, otherwise returns null</returns>
		async Task<HashSet<JobStepId>?> FindDependentJobStepIdsAsync(IJob job, JobStepId underlyingStepId, CancellationToken cancellationToken = default)
		{
			// Fetch all of our job steps sending notifications
			IJobStepNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			builder.AddJobFilter(job.Id);
			builder.AddTemplateFilter(job.TemplateId);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStateDocuments = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(builder, cancellationToken);
			if (jobStepNotificationStateDocuments is null)
			{
				return null;
			}

			HashSet<JobStepId> dependencyJobStepNotifications = new HashSet<JobStepId>();
			HashSet<JobStepId> cachedSteps = new HashSet<JobStepId>();
			foreach (IJobStepNotificationStateRef jobStepNotificationState in jobStepNotificationStateDocuments)
			{
				if (cachedSteps.Contains(jobStepNotificationState.JobStepId))
				{
					continue;
				}
				cachedSteps.Add(jobStepNotificationState.JobStepId);

				List<JobStepId> dependencySteps = new List<JobStepId>()
				{
					jobStepNotificationState.JobStepId
				};

				HashSet<JobStepId> visitedDependencySteps = new HashSet<JobStepId>()
				{
					jobStepNotificationState.JobStepId
				};

				bool isMatchingDependency = false;
				while (dependencySteps.Count > 0 && !isMatchingDependency)
				{
					JobStepId jobStepId = dependencySteps[0];
					dependencySteps.RemoveAt(0);

					IJobStep? currentStep;
					if (!job.TryGetStep(jobStepId, out currentStep))
					{
						continue;
					}

					// Iterate through all of the step's dependencies
					// Add the step Id of the notification state if the dependencies match
					foreach (JobStepId dependencyStepId in currentStep.InputDependencies)
					{
						if (underlyingStepId == dependencyStepId)
						{
							isMatchingDependency = true;
							dependencyJobStepNotifications.Add(jobStepNotificationState.JobStepId);
							break;
						}
						else if (!visitedDependencySteps.Contains(dependencyStepId))
						{
							dependencySteps.Add(dependencyStepId);
							visitedDependencySteps.Add(dependencyStepId);
						}
					}

					// Check to see if we already found a potential match before continuing searching through dependencies
					if (isMatchingDependency)
					{
						continue;
					}

					foreach (JobStepId dependencyStepId in currentStep.OrderDependencies)
					{
						if (underlyingStepId == dependencyStepId)
						{
							isMatchingDependency = true;
							dependencyJobStepNotifications.Add(jobStepNotificationState.JobStepId);
							break;
						}
						else if (!visitedDependencySteps.Contains(dependencyStepId))
						{
							dependencySteps.Add(dependencyStepId);
							visitedDependencySteps.Add(dependencyStepId);
						}
					}
				}
			}

			return dependencyJobStepNotifications;
		}

		/// <summary>
		/// Syncs up the main job information with our current recorded steps
		/// </summary>
		/// <param name="jobId">Identifier of the job</param>
		/// <param name="templateId">Identifier of the template</param>
		/// <param name="recipient">Identifier of the channel or user in Slack or user's email address</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the information has been found and updated, otherwise false</returns>
		async Task<bool> UpdatePendingJobStepNotificationsAsync(JobId jobId, TemplateId templateId, string recipient, CancellationToken cancellationToken = default)
		{
			// Fetch all of our job steps that are still pending to determine if we still have notifications to send
			// Because we need to fetch not only the main job step documents as well as any spawned job step documents
			// We need to include the parent job and template identifiers to get all of our records
			IJobStepNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			builder.AddJobAndParentJobFilter(jobId);
			builder.AddTemplateAndParentTemplateFilter(templateId);
			builder.AddRecipientFilter(recipient);
			builder.AddBadgeFilter(PendingBadge, true);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(builder, cancellationToken);
			if (jobStepNotificationStates is null)
			{
				return false;
			}

			bool wasUpdated = false;
			foreach (IJobStepNotificationStateRef jobStepNotificationState in jobStepNotificationStates)
			{
				IJob? job = await _jobCollection.GetAsync(jobStepNotificationState.JobId, cancellationToken);
				if (job is null)
				{
					continue;
				}

				IJobStep? step = null;
				if (!job.TryGetStep(jobStepNotificationState.JobStepId, out step))
				{
					continue;
				}

				string badge = GetBadgeForStep(step);
				if (badge != jobStepNotificationState.Badge)
				{
					_ = await _jobNotificationCollection.AddOrUpdateJobStepNotificationStateAsync(jobStepNotificationState.JobId, jobStepNotificationState.TemplateId, jobStepNotificationState.Recipient, jobStepNotificationState.JobStepId,
																									jobStepNotificationState.Group, jobStepNotificationState.TargetPlatform, badge, jobStepNotificationState.Channel, jobStepNotificationState.Ts,
																									jobStepNotificationState.ThreadTs, jobStepNotificationState.ParentJobId, jobStepNotificationState.ParentJobTemplateId, cancellationToken);
					wasUpdated = true;
				}
			}

			return wasUpdated;
		}

		/// <summary>
		/// Clean up notification states once all notifications have been sent
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="recipient">Identifier of the channel or user in Slack or user's email address</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task TryNotificationCleanupAsync(IJob job, string recipient, CancellationToken cancellationToken = default)
		{
			// Fetch all of our job steps that are still pending to determine if we still have notifications to send
			// Because we need to fetch not only the main job step documents as well as any spawned job step documents
			// We need to include the parent job and template identifiers to get all of our records
			IJobStepNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			builder.AddJobAndParentJobFilter(job.Id);
			builder.AddTemplateAndParentTemplateFilter(job.TemplateId);
			builder.AddRecipientFilter(recipient);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(builder, cancellationToken);
			if (jobStepNotificationStates is null)
			{
				// Nothing to clean up
				return;
			}

			// Because we could have multiple jobs providing notification information, we need to check and make sure all jobs have been accounted for
			bool allNotificationsSent = true;
			foreach (IJobStepNotificationStateRef jobStepNotificationState in jobStepNotificationStates)
			{
				if (jobStepNotificationState.Badge == PendingBadge)
				{
					allNotificationsSent = false;
					break;
				}

				// Check if the current step spawned another job that may still need to run it's Setup node
				if (job.TryGetStep(jobStepNotificationState.JobStepId, out IJobStep? step) && step.SpawnedJobs is not null)
				{
					foreach (JobId spawnedJobId in step.SpawnedJobs)
					{
						IJob? spawnedJob = await _jobCollection.GetAsync(spawnedJobId, cancellationToken);
						IJobStep? setupStep = GetSetupStepFromJob(spawnedJob);
						if (setupStep is not null && setupStep.State == JobStepState.Ready)
						{
							allNotificationsSent = false;
							break;
						}
					}
				}
			}

			if (allNotificationsSent)
			{
				_ = await _jobNotificationCollection.DeleteJobStepNotificationStatesAsync(job.Id, job.TemplateId, recipient, cancellationToken);
				_ = await _jobNotificationCollection.DeleteJobNotificationStatesAsync(job.Id, job.TemplateId, recipient, cancellationToken);
			}
		}

		/// <summary>
		/// Fetch the latest stream config
		/// </summary>
		/// <param name="buildConfig">Build plugin's global config</param>
		/// <param name="streamId">Identifier of the stream</param>
		/// <returns>Latest <see cref="StreamConfig"/> or null if one is not found</returns>
		static StreamConfig? GetStreamConfig(BuildConfig buildConfig, StreamId streamId)
		{
			buildConfig.TryGetStream(streamId, out StreamConfig? streamConfig);
			return streamConfig;
		}

		/// <summary>
		/// Fetch the branch name from the stream
		/// </summary>
		/// <param name="streamConfig">Config for the current stream</param>
		/// <returns>Name of the corresponding branch or 'Unknown Branch' if no stream was found</returns>
		static string GetBranchFromStream(StreamConfig? streamConfig)
		{
			return streamConfig?.Name ?? "Unknown Branch";
		}

		/// <summary>
		/// Fetch the branch name from the stream
		/// </summary>
		/// <param name="buildConfig">Build plugin's global config</param>
		/// <param name="streamId">Identifier of the stream</param>
		/// <returns>Name of the corresponding branch or 'Unknown Branch' if no stream was found</returns>
		static string GetBranchFromStream(BuildConfig buildConfig, StreamId streamId)
		{
			StreamConfig? streamConfig = GetStreamConfig(buildConfig, streamId);

			return GetBranchFromStream(streamConfig);
		}

		/// <summary>
		/// Determines the badge based on the job step's state and outcome
		/// </summary>
		/// <param name="step">Information about the step</param>
		/// <returns>Emoji representation of the job step's current state and outcome</returns>
		static string GetBadgeForStep(IJobStep step)
		{
			// Check the state
			switch (step.State)
			{
				case JobStepState.Waiting:
				case JobStepState.Ready:
				case JobStepState.Running:
					{
						// Step is either queued or currently being executed
						return PendingBadge;
					}
				case JobStepState.Skipped:
					{
						return SkippedBadge;
					}
				case JobStepState.Aborted:
					{
						// Treat aborted as failure conditions
						return FailureBadge;
					}
				case JobStepState.Completed:
					{
						// Continue to check the outcome of the job
						break;
					}
				default:
					{
						return String.Empty;
					}
			}

			// Check the outcome
			switch (step.Outcome)
			{
				case JobStepOutcome.Failure:
					{
						return FailureBadge;
					}
				case JobStepOutcome.Warnings:
					{
						return WarningBadge;
					}
				case JobStepOutcome.Success:
					{
						return PassingBadge;
					}
				default:
					{
						return String.Empty;
					}
			}
		}

		/// <summary>
		/// Fetch the setup job step
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <returns>A <see cref="IJobStep"/> corressponding to the Setup</returns>
		static IJobStep? GetSetupStepFromJob(IJob? job)
		{
			if (job is null)
			{
				return null;
			}

			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep step in batch.Steps)
				{
					if (step.Name == IJob.SetupNodeName)
					{
						return step;
					}
				}
			}

			return null;
		}

		/// <summary>
		/// Parses list of <see cref="ILogEventData"/> into the generic <see cref="LogData"/> structure
		/// </summary>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <returns>List of parsed logged events to be consumed by the Slack notifications</returns>
		static List<LogData> ExtractLogInformation(IJobStep step, List<ILogEventData> jobStepEventData)
		{
			List<LogData> logData = new List<LogData>();
			foreach (ILogEventData eventData in jobStepEventData)
			{
				logData.Add(new LogData(eventData.Severity, eventData.Message, step.LogId));
			}
			return logData;
		}

		#endregion Job Processing

		#region Slack

		/// <summary>
		/// Create the job's summary message
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="branch">Name of the branch from the stream</param>
		/// <param name="recipient">Identifier of the channel or user in Slack or user's email address</param>
		/// <param name="parentJob">Optional information regarding a parent job</param>
		/// <param name="abortedByUserName">Optional username provided when cancelling a job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Generated <see cref="SlackMessage"/> if successful, otherwise returns null</returns>
		async Task<SlackMessage?> GenerateSlackSummaryMessageAsync(IJob job, string branch, string recipient, IJob? parentJob = null, string? abortedByUserName = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStateDocuments;

			// Fetch all of our notification information of any parent/spawned jobs
			if (parentJob is not null)
			{
				IJobStepNotificationStateQueryBuilder parentQueryBuilder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
				parentQueryBuilder.AddJobAndParentJobFilter(parentJob.Id);
				parentQueryBuilder.AddTemplateAndParentTemplateFilter(parentJob.TemplateId);
				parentQueryBuilder.AddRecipientFilter(recipient);
				jobStepNotificationStateDocuments = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(parentQueryBuilder, cancellationToken);
				if (jobStepNotificationStateDocuments is not null)
				{
					return await CreateSlackSummaryNotificationAsync(parentJob, branch, jobStepNotificationStateDocuments, abortedByUserName, cancellationToken);
				}
			}

			// If we still have no notifications then fetch against our main job
			IJobStepNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			builder.AddJobAndParentJobFilter(job.Id);
			builder.AddTemplateAndParentTemplateFilter(job.TemplateId);
			builder.AddRecipientFilter(recipient);
			jobStepNotificationStateDocuments = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(builder, cancellationToken);
			if (jobStepNotificationStateDocuments is not null)
			{
				return await CreateSlackSummaryNotificationAsync(job, branch, jobStepNotificationStateDocuments, abortedByUserName, cancellationToken);
			}

			return null;
		}

		/// <summary>
		/// Send a notification on a job setup failure
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="branch">Name of the branch from the stream</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="jobNotificationConfigs">Collection of configurations tied to this job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task SendSetupFailedSlackMessageAsync(IJob job, IJobStep step, string branch, List<LogData> jobStepEventData, HashSet<JobNotificationConfig> jobNotificationConfigs, CancellationToken cancellationToken = default)
		{
			if (_slackClient is null)
			{
				_logger.LogInformation("Cannot send setup failed message. No Slack client set.");
				return;
			}

			SlackMessage message = new SlackMessage();

			string jobUrlLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}").ToString();
			string jobStepLink = new Uri($"{jobUrlLink}?step={step.Id}").ToString();

			// Create our notification text
			string notificationText = GenerateSlackMessageTitle(job, branch);
			message.Text = $"{notificationText} Details...";

			// Create a Block for the Slack Title
			string messageTitle = GenerateSlackMessageTitle(job, branch, jobUrlLink);
			ContextBlock jobContext = new ContextBlock();
			jobContext.Elements.Add(new TextObject(messageTitle));
			message.Blocks.Add(jobContext);

			ContextBlock infoBlock = new ContextBlock();
			infoBlock.Elements.Add(new TextObject($"<{jobStepLink}|*{step.Name}*> encountered errors"));
			message.Blocks.Add(infoBlock);

			AddLogDataContext(message, jobStepEventData, step.LogId);

			foreach (JobNotificationConfig config in jobNotificationConfigs)
			{
				foreach (string recipient in config.Channels)
				{
					// This is our first time sending the summary grid, add both the job and the job steps to the collection
					string channel = recipient;
					SlackUser? userInfo = await _slackClient.FindUserByEmailAsync(recipient, cancellationToken);
					if (userInfo is not null && userInfo.Id is not null)
					{
						channel = userInfo.Id;
					}
					
					try
					{
						_ = await _slackClient.PostMessageAsync(channel, message, cancellationToken);
					}
					catch (Exception ex)
					{
						_logger.LogInformation(ex, "Failed to send Slack setup failure message for job '{JobId}' and the template '{Template}' ({Error})", job.Id, job.TemplateId, ex.Message);
					}
				}
			}

			_logger.LogInformation("Finished sending failing setup notification for job '{JobId}'", job.Id);
		}

		/// <summary>
		/// Send a threaded message on a job setup failure
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="channel">Identifier of the Slack channel for the threaded message</param>
		/// <param name="timestamp">Timestamp of the initial notification for the threaded message</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task SendSetupFailedSlackMessageToThreadAsync(IJob job, IJobStep step, List<LogData> jobStepEventData, string channel, string timestamp, CancellationToken cancellationToken = default)
		{
			if (_slackClient is null)
			{
				_logger.LogInformation("Cannot send setup failed thread message. No Slack client set.");
				return;
			}

			string badge = GetBadgeForStep(step);
			string headerOverride = $"{job.Name} - {step.Name}";
			SlackMessage? threadMessage = GenerateSlackThreadMessage(step, badge, jobStepEventData, headerOverride);
			if (threadMessage is null)
			{
				_logger.LogInformation("Failed to generate thread message for the failed job step '{JobStepId}' with Job Id '{JobId}'", step.Id, job.Id);
				return;
			}

			try
			{
				SlackMessageId jobMessageId = new SlackMessageId(channel, null, timestamp);
				_ = await _slackClient.PostMessageToThreadAsync(jobMessageId, threadMessage, cancellationToken);
			
				_logger.LogInformation("Finished sending threaded notification for the failed job step '{JobStepId}' with Job Id '{JobId}'", step.Id, job.Id);
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Failed to send Slack threaded message for the failed job step '{JobStepId}' with Job Id '{JobId}' ({Error})", step.Id, job.Id, ex.Message);
			}

			_logger.LogInformation("Finished sending failing setup thread notification for job '{JobId}'", job.Id);
		}

		/// <summary>
		/// Sends a Slack notification for the specified job
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="branch">Name of the branch from the stream</param>
		/// <param name="recipient">Identifier of the channel or user in Slack or user's email address</param>
		/// <param name="jobNotificationState">Optional information regarding a previously sent job notification</param>
		/// <param name="parentJob">Optional information regarding a parent job</param>
		/// <param name="abortedByUserName">Optional username provided when cancelling a job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task SendSlackSummaryMessageAsync(IJob job, string branch, string recipient, IJobNotificationStateRef? jobNotificationState = null, IJob? parentJob = null, string? abortedByUserName = null, CancellationToken cancellationToken = default)
		{
			SlackMessage? message = await GenerateSlackSummaryMessageAsync(job, branch, recipient, parentJob, abortedByUserName, cancellationToken);
			if (message is null)
			{
				_logger.LogInformation("Unable to generate Job summary notification for job '{JobId}' and the template '{TemplateId}'", job.Id, job.TemplateId);
				return;
			}

			if (jobNotificationState is not null)
			{
				if (_slackClient is null)
				{
					_logger.LogInformation("Cannot update summary message. No Slack client set.");
					return;
				}

				try
				{
					SlackMessageId messageId = new SlackMessageId(jobNotificationState.Channel, null, jobNotificationState.Ts);
					_ = _slackClient.UpdateMessageAsync(messageId, message, cancellationToken);
				}
				catch (Exception ex)
				{
					_logger.LogInformation(ex, "Failed to update Slack summary message for job '{JobId}' and the template '{Template}' ({Error})", jobNotificationState.JobId, jobNotificationState.TemplateId, ex.Message);
				}
				return;
			}
			
			if (_slackClient is null)
			{
				_logger.LogInformation("Cannot send summary message. No Slack client set.");
				_ = await _jobNotificationCollection.AddOrUpdateJobNotificationStateAsync(job.Id, job.TemplateId, recipient, recipient, String.Empty, cancellationToken);
				return;
			}
			
			// This is our first time sending the summary grid, add both the job and the job steps to the collection
			string channel = recipient;
			SlackUser? userInfo = await _slackClient.FindUserByEmailAsync(recipient, cancellationToken);
			if (userInfo is not null && userInfo.Id is not null)
			{
				channel = userInfo.Id;
			}
				
			try
			{
				SlackMessageId messageId = await _slackClient.PostMessageAsync(channel, message, cancellationToken);
				_ = await _jobNotificationCollection.AddOrUpdateJobNotificationStateAsync(job.Id, job.TemplateId, recipient, messageId.Channel, messageId.Ts, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Failed to send Slack summary message for job '{JobId}' and the template '{Template}' ({Error})", job.Id, job.TemplateId, ex.Message);
			}
		}

		/// <summary>
		/// Sends a Slack message to a new or existing thread
		/// </summary>
		/// <param name="jobMessageId"><see cref="SlackMessageId"/> of the summary message</param>
		/// <param name="jobStepNotificationState">Information about the job step's notification state</param>
		/// <param name="threadMessage"><see cref="SlackMessage"/> to be sent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The <see cref="SlackMessageId"/> of either the newly sent notification or from the <see cref="IJobStepNotificationStateRef"/></returns>
		async Task<SlackMessageId?> SendSlackThreadMessageAsync(SlackMessageId jobMessageId, IJobStepNotificationStateRef jobStepNotificationState, SlackMessage threadMessage, CancellationToken cancellationToken = default)
		{
			// Check to see if a message has already been sent out to avoid sending duplicate notifications
			if (!String.IsNullOrEmpty(jobStepNotificationState.ThreadTs))
			{
				return new SlackMessageId(jobStepNotificationState.Channel, jobStepNotificationState.ThreadTs, jobStepNotificationState.Ts);
			}

			if (_slackClient is null)
			{
				_logger.LogInformation("Cannot send thread message. No Slack client set.");
				return null;
			}

			try
			{
				SlackMessageId threadMessageId = await _slackClient.PostMessageToThreadAsync(jobMessageId, threadMessage, cancellationToken);
			
				_logger.LogInformation("Finished sending notification for job step '{JobStepId}' with Job Id '{JobId}' for template '{Template}'", jobStepNotificationState.JobStepId, jobStepNotificationState.JobId, jobStepNotificationState.TemplateId);
				return threadMessageId;
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Failed to send Slack message for job step '{JobStepId}' with Job Id '{JobId}' for template '{Template}' ({Error})", jobStepNotificationState.JobStepId, jobStepNotificationState.JobId, jobStepNotificationState.TemplateId, ex.Message);
			}

			return null;
		}

		/// <summary>
		/// Sends a Slack notification for the specified job step
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="step">Information about the step</param>
		/// <param name="jobStepEventData">List of the step's logged events</param>
		/// <param name="underlyingStep">Optional information about the step's dependent step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if a job step notification needed to be updated, otherwise returns false</returns>
		async Task<bool> UpdateAndSendSlackNotificationsForStepAsync(IJob job, IJobStep step, List<LogData> jobStepEventData, IJobStep? underlyingStep = null, CancellationToken cancellationToken = default)
		{
			string badge = GetBadgeForStep(step);

			// Update our job step if it's being monitored and send an updated summary and post to the thread if needed
			IJobStepNotificationStateQueryBuilder builder = JobNotificationCollection.CreateJobStepNotificationStateQueryBuilder();
			builder.AddJobFilter(job.Id);
			builder.AddJobStepFilter(step.Id);
			builder.AddBadgeFilter(badge, false);
			IReadOnlyList<IJobStepNotificationStateRef>? jobStepNotificationStates = await _jobNotificationCollection.GetJobStepNotificationStatesAsync(builder, cancellationToken);
			if (jobStepNotificationStates is null || jobStepNotificationStates.Count == 0)
			{
				_logger.LogInformation("No notification to be sent for step '{JobStepId}' in job '{JobId}'", step.Id, job.Id);
				return false;
			}

			foreach (IJobStepNotificationStateRef jobStepNotificationState in jobStepNotificationStates)
			{
				string channel = jobStepNotificationState.Channel;
				string timestamp = jobStepNotificationState.Ts;
				string? threadTimestamp = null;

				SlackMessage? threadMessage = null;

				// Notify on any problems of the Job Step
				if (step.Outcome != JobStepOutcome.Success)
				{
					string? headerOverride = null;
					if (jobStepNotificationState.ParentJobId is not null && jobStepNotificationState.ParentJobTemplateId is not null)
					{
						// Our current step has a parent job notification associated with it so we need to provide the header override to differentiate the job from the parent
						headerOverride = $"{job.Name} - {step.Name}";
					}
					threadMessage = GenerateSlackThreadMessage(step, badge, jobStepEventData, headerOverride, underlyingStep);
				}

				if (threadMessage is not null)
				{
					// Fetch our initial job notification to get the Message Id for threads
					JobId notificationJobId = jobStepNotificationState.ParentJobId ?? jobStepNotificationState.JobId;
					TemplateId notificationTemplateId = jobStepNotificationState.ParentJobTemplateId ?? jobStepNotificationState.TemplateId;
					IJobNotificationStateRef? jobNotificationState = await _jobNotificationCollection.GetJobNotificationStateAsync(notificationJobId, notificationTemplateId, jobStepNotificationState.Recipient, cancellationToken);
					if (jobNotificationState is null)
					{
						_logger.LogInformation("No message id found for job '{JobId}' with template '{TemplateId}' for thread to be posted.", job.Id, jobStepNotificationState.TemplateId);
						continue;
					}
					SlackMessageId jobMessageId = new SlackMessageId(jobNotificationState.Channel, null, jobNotificationState.Ts);
					SlackMessageId? threadMessageId = await SendSlackThreadMessageAsync(jobMessageId, jobStepNotificationState, threadMessage, cancellationToken);
					if (threadMessageId is not null)
					{
						channel = threadMessageId.Channel;
						threadTimestamp = threadMessageId.ThreadTs;
						timestamp = threadMessageId.Ts;
					}
				}
				_ = await _jobNotificationCollection.AddOrUpdateJobStepNotificationStateAsync(jobStepNotificationState.JobId, jobStepNotificationState.TemplateId, jobStepNotificationState.Recipient, jobStepNotificationState.JobStepId, jobStepNotificationState.Group, jobStepNotificationState.TargetPlatform, badge, channel, timestamp, threadTimestamp, jobStepNotificationState.ParentJobId, jobStepNotificationState.ParentJobTemplateId, cancellationToken);
			}

			return true;
		}

		/// <summary>
		/// Create the job's summary message
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="branch">Name of the branch from the stream</param>
		/// <param name="jobStepNotificationStateDocuments">List of <see cref="IJobStepNotificationStateRef"/></param>
		/// <param name="abortedByUserName">Optional username provided when cancelling a job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Generated <see cref="SlackMessage"/></returns>
		async Task<SlackMessage> CreateSlackSummaryNotificationAsync(IJob job, string branch, IReadOnlyList<IJobStepNotificationStateRef> jobStepNotificationStateDocuments, string? abortedByUserName = null, CancellationToken cancellationToken = default)
		{
			DividerBlock divider = new DividerBlock();
			SlackMessage message = new SlackMessage();

			// Create our notification text
			string notificationText = GenerateSlackMessageTitle(job, branch);
			message.Text = $"{notificationText} Details...";

			// Create a Block for the Slack Title
			string hordeJobURL = new Uri(_serverInfo.DashboardUrl, $"job/{job.Id}").ToString();
			string messageTitle = GenerateSlackMessageTitle(job, branch, hordeJobURL);
			ContextBlock jobContext = new ContextBlock();
			jobContext.Elements.Add(new TextObject(messageTitle));
			message.Blocks.Add(jobContext);

			// If the job was manually aborted, then provide a message that catches the recipients eyes that the full information may not be available
			if (!String.IsNullOrEmpty(abortedByUserName))
			{
				AddCancelledJobSlackContext(message, $"*{AlertBadge} Job was canceled by {abortedByUserName} and may be missing information {AlertBadge}*", job.CancellationReason);
				message.Blocks.Add(divider);
			}

			int itemsInGroup = 0;
			string currentGroup = String.Empty;
			ContextBlock? header = null;
			ContextBlock? stepContext = null;
			List<IJob> pendingSpawnedJobs = new List<IJob>();
			List<IJob> canceledSpawnedJobs = new List<IJob>();
			List<IJob> errorSpawnedJobs = new List<IJob>();
			foreach (IJobStepNotificationStateRef state in jobStepNotificationStateDocuments)
			{
				// Nothing to report on if the step is no longer in the job
				IJob? notificationJob = await _jobCollection.GetAsync(state.JobId, cancellationToken);
				if (notificationJob is null || !notificationJob.TryGetStep(state.JobStepId, out IJobStep? step))
				{
					continue;
				}

				if (currentGroup != state.Group || header is null)
				{
					// Check to see if we have an existing step context that needs to be added
					if (stepContext is not null)
					{
						message.Blocks.Add(stepContext);
						message.Blocks.Add(divider);
						stepContext = null;
					}

					currentGroup = state.Group;
					header = new ContextBlock();
					header.Elements.Add(new TextObject($"*{currentGroup}*"));
					message.Blocks.Add(header);
				}

				string hordeJobStepURL = new Uri(_serverInfo.DashboardUrl, $"job/{state.JobId}?step={state.JobStepId}").ToString();

				// Slack can only allow for 10 elements per block (https://api.slack.com/reference/block-kit/blocks#context) and we want to track when to create new ContextBlocks
				bool bNeedsNewContext = (itemsInGroup % MaxSlackBlocks == 0 || stepContext is null);
				if (bNeedsNewContext)
				{
					if (stepContext is not null)
					{
						message.Blocks.Add(stepContext);
					}
					stepContext = new ContextBlock();
				}
				stepContext?.Elements.Add(new TextObject($"<{hordeJobStepURL}|{state.Badge} {state.TargetPlatform}>"));

				// The StepContext should always be valid here, but to disable the warning, we provide a default so that we only have 1 block per context.
				itemsInGroup = stepContext?.Elements.Count ?? MaxSlackBlocks;

				// Check to see if our step spawned any jobs
				if (step.SpawnedJobs is null)
				{
					continue;
				}

				foreach (JobId spawnedJobId in step.SpawnedJobs)
				{
					// Fetch the job and see if the jobs are queued to run, were canceled, or encountered issues during the setup build
					IJob? spawnedJob = await _jobCollection.GetAsync(spawnedJobId, cancellationToken);
					if (spawnedJob is null)
					{
						continue;
					}

					if (spawnedJob.AbortedByUserId is not null)
					{
						canceledSpawnedJobs.Add(spawnedJob);
						continue;
					}

					IJobStep? setupStep = GetSetupStepFromJob(spawnedJob);
					if (setupStep is null || setupStep.Outcome == JobStepOutcome.Failure)
					{
						errorSpawnedJobs.Add(spawnedJob);
					}
					else if (setupStep.State == JobStepState.Aborted)
					{
						canceledSpawnedJobs.Add(spawnedJob);
					}
					else if (setupStep.State != JobStepState.Completed)
					{
						pendingSpawnedJobs.Add(spawnedJob);
					}
				}
			}

			// Check to see if we have an existing step context that needs to be added
			if (stepContext is not null)
			{
				message.Blocks.Add(stepContext);
				message.Blocks.Add(divider);
			}

			// Generate our summary to include information regarding the spawned jobs only if the current job wasn't canceled
			// Reason being is that the parent job will no longer have information to handle updates
			if (job.AbortedByUserId is null)
			{
				TryAddSpawnedJobSlackContext(message, $"*Queued jobs:*", pendingSpawnedJobs);
				TryAddSpawnedJobSlackContext(message, $"*Canceled jobs:*", canceledSpawnedJobs);
				TryAddSpawnedJobSlackContext(message, $"*Failed jobs:*", errorSpawnedJobs);
			}

			return message;
		}

		/// <summary>
		/// Adds a context to the Slack message with log details
		/// </summary>
		/// <param name="slackBlockContainer">Slack block container</param>
		/// <param name="logData">List of our log information</param>
		/// <param name="logId">Optional identifier of the log</param>
		void AddLogDataContext(ISlackBlockContainer slackBlockContainer, List<LogData> logData, LogId? logId = null)
		{
			// Don't clutter the message with all of the events and do a preliminary check to see if the user needs to be made aware that events have been truncated
			if (logData.Count > MaxEventMessages)
			{
				ContextBlock limitContext = new ContextBlock();
				limitContext.Elements.Add(new TextObject($":warning: *Step has multiple events reported. Will only show first 5.* :warning:"));
				slackBlockContainer.Blocks.Add(limitContext);
			}

			foreach (LogData log in logData.Take(MaxEventMessages))
			{
				SectionBlock logSection = new SectionBlock(new TextObject(SanitizeQuoteText(log.Message)));
				slackBlockContainer.Blocks.Add(logSection);
			}

			if (logId is not null)
			{
				string logUrl = new Uri(_serverInfo.DashboardUrl, $"log/{logId}").ToString();

				ContextBlock detailsContext = new ContextBlock();
				detailsContext.Elements.Add(new TextObject($"<{logUrl}|View Job Step Log>"));
				slackBlockContainer.Blocks.Add(detailsContext);
			}
		}

		/// <summary>
		/// Create the job step informational threaded message
		/// </summary>
		/// <param name="step">Information about the step</param>
		/// <param name="badge">Slack emoji associated with the notification</param>
		/// <param name="logData">List of our log information</param>
		/// <param name="headerOverride">Optional string header to be used within the message</param>
		/// <param name="underlyingStep">Optional information about the step's dependent step</param>
		/// <returns>Generated <see cref="SlackMessage"/> if successful, otherwise returns null</returns>
		SlackMessage? GenerateSlackThreadMessage(IJobStep step, string badge, List<LogData> logData, string? headerOverride = null, IJobStep? underlyingStep = null)
		{
			if (logData.Count == 0)
			{
				return null;
			}

			List<LogData> data = logData.Where(x => x.Severity == LogEventSeverity.Error).ToList();
			if (!data.Any())
			{
				data = logData.Where(x => x.Severity == LogEventSeverity.Warning).ToList();
			}

			string jobStepLink = new Uri($"{_serverInfo.DashboardUrl}job/{step.Job.Id}?step={step.Id}").ToString();
			string alertInfo = $"{step.Name}";
			if (!String.IsNullOrEmpty(headerOverride))
			{
				alertInfo = headerOverride;
			}
			string alertHeader = $"<{jobStepLink}|*{alertInfo}*>";

			SlackMessage alertMessage = new SlackMessage();

			ContextBlock alertStepBlock = new ContextBlock();
			alertStepBlock.Elements.Add(new PlainTextObject(badge, true));
			alertStepBlock.Elements.Add(new TextObject(alertHeader));
			alertStepBlock.Elements.Add(new PlainTextObject(badge, true));
			alertMessage.Blocks.Add(alertStepBlock);

			if (underlyingStep is null)
			{
				AddLogDataContext(alertMessage, data, step.LogId);
				return alertMessage;
			}

			// If an underlying step was provided then the error occurred further up the job and impacted the current step
			ContextBlock dependencyContext = new ContextBlock();
			dependencyContext.Elements.Add(new TextObject("*Step was marked as skipped due to errors in the following dependency*"));
			alertMessage.Blocks.Add(dependencyContext);

			// Slack Attachments are marked as being legacy and is recommended to be avoided
			// The reason why Attachments are used is because this information is secondary to describe why a test had skipped
			// We also wanted to differentiate the dependencies from the actual monitored node by indenting the dependency information
			SlackAttachment attachment = new SlackAttachment();
			attachment.Color = "#666666";

			string dependencyJobStepLink = new Uri($"{_serverInfo.DashboardUrl}job/{step.Job.Id}?step={underlyingStep.Id}").ToString();

			attachment.AddContext(new TextObject($"<{dependencyJobStepLink}|*{underlyingStep.Name}*>"));
			AddLogDataContext(attachment, data, underlyingStep.LogId);
			alertMessage.Attachments.Add(attachment);

			return alertMessage;
		}

		/// <summary>
		/// Attempts to add a Slack context to the <see cref="SlackMessage"/>
		/// </summary>
		/// <param name="message"><see cref="SlackMessage"/> to add the context to</param>
		/// <param name="header">String header of the context</param>
		/// <param name="spawnedJobs">List of spawned jobs for this context</param>
		void TryAddSpawnedJobSlackContext(SlackMessage message, string header, List<IJob> spawnedJobs)
		{
			if (!spawnedJobs.Any())
			{
				return;
			}

			ContextBlock? headerContext = new ContextBlock();
			headerContext.Elements.Add(new TextObject(header));
			message.Blocks.Add(headerContext);

			// Create a Block for the Slack Title
			int itemsInGroup = 0;
			ContextBlock? spawnJobContext = null;
			foreach (IJob job in spawnedJobs)
			{
				string hordeJobURL = new Uri(_serverInfo.DashboardUrl, $"job/{job.Id}").ToString();

				// Slack can only allow for 10 elements per block (https://api.slack.com/reference/block-kit/blocks#context) and we want to track when to create new ContextBlocks
				bool bNeedsNewContext = (itemsInGroup % MaxSlackBlocks == 0 || spawnJobContext is null);
				if (bNeedsNewContext)
				{
					if (spawnJobContext is not null)
					{
						message.Blocks.Add(spawnJobContext);
					}
					spawnJobContext = new ContextBlock();
				}
				spawnJobContext?.Elements.Add(new TextObject($"<{hordeJobURL}|{job.Name}>"));

				// The StepContext should always be valid here, but to disable the warning, we provide a default so that we only have 1 block per context.
				itemsInGroup = spawnJobContext?.Elements.Count ?? MaxSlackBlocks;
			}

			if (spawnJobContext is null)
			{
				// Log an error and return
				return;
			}

			message.Blocks.Add(spawnJobContext);
		}

		/// <summary>
		/// Generates the title for the summary messages
		/// </summary>
		/// <param name="job">Information about the job</param>
		/// <param name="branch">Name of the branch from the stream</param>
		/// <param name="hordeJobURL">Optional URL of the job in Horde</param>
		/// <returns>Title message using Slack's markdown format</returns>
		static string GenerateSlackMessageTitle(IJob job, string branch, string? hordeJobURL = null)
		{
			string changelistDetails = $"CL {job.CommitId}";
			if (job.PreflightCommitId is not null)
			{
				changelistDetails = $"Preflight CL {job.PreflightCommitId} against {changelistDetails}";
			}

			if (!String.IsNullOrEmpty(hordeJobURL))
			{
				return $"<{hordeJobURL}|*{branch} - {changelistDetails} - {job.Name}*>";
			}

			return $"{branch} - {changelistDetails} - {job.Name}";
		}

		/// <summary>
		/// Sanitizes the text for a Slack quote message
		/// </summary>
		/// <param name="text">Text to sanitize</param>
		/// <param name="maxLength">Maximum allowed length for the message</param>
		static string SanitizeQuoteText(string text, int maxLength = MaxBlockCharacterLength)
		{
			if (text.Length > maxLength)
			{
				text = text.Substring(0, maxLength - 4) + "...";
			}
			return $"```{text}```";
		}

		/// <summary>
		/// Adds a context to the Slack message with cancellation details
		/// </summary>
		/// <param name="message"><see cref="SlackMessage"/> to add the context to</param>
		/// <param name="text">Information to provide in the context</param>
		/// <param name="cancellationReason">Optional message explaining the cancellation</param>
		static void AddCancelledJobSlackContext(SlackMessage message, string text, string? cancellationReason = null)
		{
			ContextBlock context = new ContextBlock();
			context.Elements.Add(new TextObject(text));
			message.Blocks.Add(context);

			if (!String.IsNullOrEmpty(cancellationReason))
			{
				ContextBlock cancellationHeader = new ContextBlock();
				cancellationHeader.Elements.Add(new TextObject($"*Cancellation Reason*"));
				message.Blocks.Add(cancellationHeader);

				ContextBlock cancellationContext = new ContextBlock();
				cancellationContext.Elements.Add(new TextObject(cancellationReason));
				message.Blocks.Add(cancellationContext);
			}
		}

		#endregion Slack
	}
}
