// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data.Common;
using System.Data.Odbc;
using HordeServer.Analytics;
using HordeServer.Dialects;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.DataSources
{
	/// <summary>
	/// ODBC-backed implementation of <see cref="IAnalyticsDataSource"/>. Reads the connection string from <see cref="OdbcTelemetryConfig.ODBCConnectionString"/>, folds in the OAuth bearer-token fragment (Databricks pass-through pattern) when an <see cref="IAuthenticationProvider{TConfig}"/> is registered, and caches the assembled connection string for <see cref="OdbcTelemetryConfig.ODBCConnectionStringRefreshTimeMins"/> minutes (default 5).
	///
	/// Returns each connection as a <see cref="DbConnection"/> typed by the framework's <see cref="OdbcConnection"/>; consumers receive the abstract type and never need <see cref="System.Data.Odbc"/> themselves.
	/// </summary>
	public sealed class OdbcTelemetryDataSource : IAnalyticsDataSource, IAsyncDisposable, IDisposable
	{
		/// <summary>
		/// Default connection-string refresh time (minutes) when <see cref="OdbcTelemetryConfig.ODBCConnectionStringRefreshTimeMins"/> is unset.
		/// </summary>
		public const int DefaultRefreshTimeMins = 5;

		private readonly IOptionsMonitor<OdbcTelemetryConfig> _config;
		private readonly IAuthenticationProvider<OdbcTelemetryConfig>? _authProvider;
		private readonly ILogger<OdbcTelemetryDataSource> _logger;
		private readonly ISqlDialect _dialect;

		private readonly SemaphoreSlim _connectionStringSemaphore = new(1, 1);
		private string? _cachedConnectionString;
		private DateTime _cacheExpiry = DateTime.MinValue;
		private int _currentRefreshMins;
		private bool _disposed;

		/// <inheritdoc/>
		public ISqlDialect Dialect => _dialect;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="config">Plugin configuration carrying the ODBC connection string and refresh time.</param>
		/// <param name="dialect">SQL dialect to use for placeholder syntax, identifier quoting, and table-name formatting. The plugin registers <see cref="OdbcDialect"/> as the default; deployments that point this plugin at backends with non-standard identifier rules (e.g. ClickHouse via the Simba ODBC driver) override this DI registration with their own implementation.</param>
		/// <param name="logger">Logger.</param>
		/// <param name="authProvider">Optional OAuth provider that contributes a bearer-token connection-string fragment when configured. Resolved as nullable so deployments without OAuth still work.</param>
		public OdbcTelemetryDataSource(
			IOptionsMonitor<OdbcTelemetryConfig> config,
			ISqlDialect dialect,
			ILogger<OdbcTelemetryDataSource> logger,
			IAuthenticationProvider<OdbcTelemetryConfig>? authProvider = null)
		{
			_config = config;
			_dialect = dialect;
			_logger = logger;
			_authProvider = authProvider;
			_currentRefreshMins = config.CurrentValue.ODBCConnectionStringRefreshTimeMins ?? DefaultRefreshTimeMins;
		}

		/// <inheritdoc/>
		public async Task<DbConnection> OpenAsync(CancellationToken cancellationToken = default)
		{
			string? connectionString = await GetConnectionStringAsync(cancellationToken: cancellationToken);
			if (String.IsNullOrEmpty(connectionString))
			{
				throw new InvalidOperationException($"OdbcTelemetry is enabled but {nameof(OdbcTelemetryConfig.ODBCConnectionString)} is not configured.");
			}

			OdbcConnection connection = new OdbcConnection(connectionString);
			try
			{
				await connection.OpenAsync(cancellationToken);
				return connection;
			}
			catch
			{
				connection.Dispose();
				throw;
			}
		}

		private async Task<string?> GetConnectionStringAsync(bool forceRefresh = false, CancellationToken cancellationToken = default)
		{
			forceRefresh |= CheckRefreshConfigChange();

			if (_cachedConnectionString != null && DateTime.UtcNow < _cacheExpiry && !forceRefresh)
			{
				return _cachedConnectionString;
			}

			string baseConnectionString = _config.CurrentValue.ODBCConnectionString;
			if (String.IsNullOrEmpty(baseConnectionString))
			{
				_logger.LogError("Invalid configuration: missing or empty {PropertyName}. OdbcTelemetryDataSource cannot open a connection.", nameof(OdbcTelemetryConfig.ODBCConnectionString));
				return null;
			}

			await _connectionStringSemaphore.WaitAsync(cancellationToken);
			try
			{
				string? built = await BuildConnectionStringAsync(baseConnectionString);
				if (!String.IsNullOrEmpty(built))
				{
					_cachedConnectionString = built;
					_cacheExpiry = DateTime.UtcNow.AddMinutes(_currentRefreshMins);
				}
				return built;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to build ODBC connection string: {Message}", ex.Message);
				return null;
			}
			finally
			{
				_connectionStringSemaphore.Release();
			}
		}

		private bool CheckRefreshConfigChange()
		{
			int resolvedRefresh = _config.CurrentValue.ODBCConnectionStringRefreshTimeMins ?? DefaultRefreshTimeMins;
			if (_currentRefreshMins != resolvedRefresh)
			{
				_currentRefreshMins = Math.Max(resolvedRefresh, 0);
				return true;
			}
			return false;
		}

		private async Task<string?> BuildConnectionStringAsync(string baseConnectionString)
		{
			OdbcConnectionStringBuilder odbcBuilder = new OdbcConnectionStringBuilder(baseConnectionString);

			if (_authProvider == null)
			{
				_logger.LogDebug("No OAuth provider registered; ODBC connection string passes through unmodified.");
				return odbcBuilder.ConnectionString;
			}

			KeyValuePair<string, string>? authFragment = await _authProvider.GetConnectionStringAuthenticationFragmentAsync();
			if (!authFragment.HasValue)
			{
				return odbcBuilder.ConnectionString;
			}

			odbcBuilder[authFragment.Value.Key] = authFragment.Value.Value;
			return odbcBuilder.ConnectionString;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_disposed)
			{
				return;
			}

			_connectionStringSemaphore.Dispose();
			_disposed = true;

			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
		{
			Dispose();
			return ValueTask.CompletedTask;
		}
	}
}
