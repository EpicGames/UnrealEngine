// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;

namespace HordeServer.Dialects
{
	/// <summary>
	/// Hybrid SQL dialect for the "ClickHouse via the clickhouse-odbc driver" deployment shape: ODBC-style positional <c>?</c> placeholders (because the driver binds positionally and ClickHouse never sees the placeholder text), but ClickHouse-style identifier handling.
	/// </summary>
	public sealed class ClickHouseOdbcDialect : ISqlDialect
	{
		/// <summary>
		/// Standard name of the dialect.
		/// </summary>
		public const string DialectIdentifier = "ClickHouseOdbc";

		/// <inheritdoc/>
		public string FormatParameterPlaceholder(string canonicalName, DbType dbType) => "?";

		/// <inheritdoc/>
		public void AddParameter(DbCommand command, string canonicalName, object? value, DbType dbType)
		{
			DbParameter parameter = command.CreateParameter();
			parameter.ParameterName = canonicalName;
			parameter.DbType = dbType;
			parameter.Value = value ?? DBNull.Value;
			command.Parameters.Add(parameter);
		}

		/// <inheritdoc/>
		public string QuoteIdentifier(string identifier) => $"`{identifier}`";

		/// <inheritdoc/>
		public string FormatTableName(string? schemaName, string tableName)
		{
			// Logical table names from the schema document may contain dots (e.g. "horde.state_job_summary"), we will normalize to '_' to provide a better direct-query experience.
			string physicalTableName = tableName.Replace('.', '_');

			return String.IsNullOrEmpty(schemaName) ? physicalTableName : $"{schemaName}.{physicalTableName}";
		}
	}
}
