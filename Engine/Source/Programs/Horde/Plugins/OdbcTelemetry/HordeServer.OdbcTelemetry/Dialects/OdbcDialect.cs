// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;

namespace HordeServer.Dialects
{
	/// <summary>
	/// SQL dialect for ODBC-fronted backends. Positional <c>?</c> placeholders, no identifier quoting (Simba/standard ODBC behavior).
	/// </summary>
	public sealed class OdbcDialect : ISqlDialect
	{

		/// <summary>
		/// Standard name of the dialect.
		/// </summary>
		public const string DialectIdentifier = "StandardOdbc";

		/// <inheritdoc/>
		public string FormatParameterPlaceholder(string canonicalName, DbType dbType) => "?";

		/// <inheritdoc/>
		public void AddParameter(DbCommand command, string canonicalName, object? value, DbType dbType)
		{
			DbParameter parameter = command.CreateParameter();
			parameter.ParameterName = canonicalName;
			parameter.DbType = dbType;
			parameter.Value = value ?? DBNull.Value;
			command.Parameters.Add(parameter);
		}

		/// <inheritdoc/>
		public string QuoteIdentifier(string identifier) => identifier;

		/// <inheritdoc/>
		public string FormatTableName(string? schemaName, string tableName)
		{
			return String.IsNullOrEmpty(schemaName) ? tableName : $"{schemaName}.{tableName}";
		}
	}
}
