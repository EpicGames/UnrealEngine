<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="System.Data.Odbc" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\..\Plugins\Analytics\HordeServer.Analytics\HordeServer.Analytics.csproj" />
    <ProjectReference Include="..\..\StructuredAnalytics\HordeServer.StructuredAnalytics\HordeServer.StructuredAnalytics.csproj" />
  </ItemGroup>

</Project>
