// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the OdbcTelemetry plugin. Provides an ODBC-backed <see cref="IAnalyticsDataSource"/> for any consumer plugin (StructuredAnalytics, PerformanceTrends, ...). Today's Databricks-via-Simba shape lives here; Snowflake-via-Simba and other ODBC-accessible backends fit the same plugin.
	///
	/// Implements <see cref="IOAuthClientCredentialsConfig"/> so the plugin can fold an OAuth bearer token into the ODBC connection string at open time (Databricks pass-through auth pattern).
	/// </summary>
	public class OdbcTelemetryConfig : IPluginConfig, IOAuthClientCredentialsConfig
	{
		/// <summary>
		/// Connection string for the ODBC connector.
		/// </summary>
		/// <remarks>Not ';' terminated.</remarks>
		public string ODBCConnectionString { get; set; } = String.Empty;

		/// <summary>
		/// Connection-string refresh time, in minutes. Controls how often the auth-fragment-folded connection string is rebuilt.
		/// </summary>
		public int? ODBCConnectionStringRefreshTimeMins { get; set; }

		/// <summary>
		/// HTTPS endpoint for the OAuth2 client-credentials token exchange.
		/// When empty, no token is fetched and the connection string is used as-is.
		/// </summary>
		public string OAuthTokenEndpoint { get; set; } = String.Empty;

		/// <summary>
		/// OAuth2 scope sent in the token request. Defaults to <c>all-apis</c> (Databricks). Set to empty to omit the scope form field.
		/// </summary>
		public string OAuthScope { get; set; } = "all-apis";

		/// <summary>
		/// Connection-string key the access token is folded into. Defaults to <c>Auth_AccessToken</c> (Simba Spark ODBC driver convention).
		/// </summary>
		public string OAuthAccessTokenFragmentKey { get; set; } = "Auth_AccessToken";

		/// <summary>
		/// Secret-store ID for the OAuth client_id / client_secret pair.
		/// </summary>
		public string OAuthCredentialsSecretId { get; set; } = String.Empty;

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
		}
	}
}
