// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.DataSources;
using HordeServer.Dialects;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.DependencyInjection.Extensions;
using Microsoft.Extensions.Options;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the OdbcTelemetry plugin. Registers an ODBC-backed <see cref="IAnalyticsDataSource"/> + <see cref="ISqlDialect"/> + <see cref="IAuthenticationProvider{TConfig}"/> for any consumer plugin (StructuredAnalytics, PerformanceTrends, ...) that takes <see cref="IAnalyticsDataSource"/> in via DI.
	/// </summary>
	[Plugin("OdbcTelemetry", EnabledByDefault = false, GlobalConfigType = typeof(OdbcTelemetryConfig), ServerConfigType = typeof(OdbcTelemetryServerConfig))]
	public class OdbcTelemetryPlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public OdbcTelemetryPlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <summary>
		/// Key under which this plugin's <see cref="IAnalyticsDataSource"/> is registered. Consumer plugins reference this in their <c>BackendName</c> config field.
		/// </summary>
		public const string BackendKey = "Odbc";

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			serviceCollection.AddHttpClient();

			serviceCollection.AddSingleton<IAuthenticationProvider<OdbcTelemetryConfig>, OAuthClientCredentialsTokenProvider<OdbcTelemetryConfig>>();

			// Dialect is driven by the plugin's server config (boot-time selection). The factory defers resolution
			// until the first request so IOptions is fully populated by then. Operators that need a dialect not
			// listed here register their own ISqlDialect after this plugin loads — AddSingleton overrides TryAdd.
			serviceCollection.TryAddSingleton<ISqlDialect>(sp => CreateDialect(sp.GetRequiredService<IOptions<OdbcTelemetryServerConfig>>().Value));

			serviceCollection.AddKeyedSingleton<IAnalyticsDataSource, OdbcTelemetryDataSource>(BackendKey);
		}

		/// <summary>
		/// Maps the configured dialect string to its concrete <see cref="ISqlDialect"/> implementation. Throws on unrecognised values so a typo in <c>OdbcTelemetryServerConfig.Dialect</c> fails loudly at first request rather than silently falling back to a wrong default.
		/// </summary>
		internal static ISqlDialect CreateDialect(OdbcTelemetryServerConfig serverConfig)
		{
			string kind = serverConfig.Dialect ?? "StandardOdbc";
			return kind switch
			{
				OdbcDialect.DialectIdentifier => new OdbcDialect(),
				ClickHouseOdbcDialect.DialectIdentifier => new ClickHouseOdbcDialect(),
				_ => throw new InvalidOperationException($"Unrecognised {nameof(OdbcTelemetryServerConfig.Dialect)} value: \"{kind}\". Expected one of: \"{OdbcDialect.DialectIdentifier}\", \"{ClickHouseOdbcDialect.DialectIdentifier}\". Register a custom ISqlDialect in your composition root to override this plugin's default.")
			};
		}
	}

	/// <summary>
	/// Helper methods for OdbcTelemetry plugin config
	/// </summary>
	public static class OdbcTelemetryExtensions
	{
		/// <summary>
		/// Configures the OdbcTelemetry plugin
		/// </summary>
		public static void AddOdbcTelemetryConfig(this IDictionary<PluginName, IPluginConfig> dictionary, OdbcTelemetryConfig config)
			=> dictionary[new PluginName("OdbcTelemetry")] = config;

		/// <summary>
		/// Gets configuration for the OdbcTelemetry plugin
		/// </summary>
		public static OdbcTelemetryConfig GetOdbcTelemetryConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (OdbcTelemetryConfig)dictionary[new PluginName("OdbcTelemetry")];
	}
}
