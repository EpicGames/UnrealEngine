// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Server configuration for the OdbcTelemetry plugin. Read at boot time during the plugin's <c>ConfigureServices</c> phase, so any value here drives DI shape rather than per-request behavior.
	/// </summary>
	public class OdbcTelemetryServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Selects which <c>ISqlDialect</c> implementation to register for the OdbcTelemetry data source.
		/// Operators register a custom <c>ISqlDialect</c> in their composition root if neither built-in fits.
		/// </summary>
		public string Dialect { get; set; } = "StandardOdbc";
	}
}
