<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer.Tests</RootNamespace>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Testing" Version="10.0.3" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.PerformanceTrends\EpicGames.Analytics.PerformanceTrends.csproj" />
    <ProjectReference Include="..\..\Build\HordeServer.Build\HordeServer.Build.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\StructuredAnalytics\HordeServer.StructuredAnalytics\HordeServer.StructuredAnalytics.csproj" />
    <ProjectReference Include="..\HordeServer.PerformanceTrends\HordeServer.PerformanceTrends.csproj" />
  </ItemGroup>
  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>
</Project>