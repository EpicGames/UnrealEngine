// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.PerformanceTrends;

namespace HordeServer.Epic.EpicSandbox.Tests.PerformanceTrends
{
	[TestClass]
	public class KeyStatsPerformanceSummaryHandlerTests
	{
		[TestMethod]
		public void PerformanceSummaryType_ReturnsKeyStats()
		{
			// Arrange
			KeyStatsPerformanceSummaryHandler handler = new KeyStatsPerformanceSummaryHandler();

			// Act
			string summaryType = handler.PerformanceSummaryType;

			// Assert
			Assert.AreEqual("KeyStats", summaryType);
		}
	}
}
