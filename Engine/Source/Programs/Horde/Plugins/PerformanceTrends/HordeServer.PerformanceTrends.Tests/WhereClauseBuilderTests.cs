// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using HordeServer.Analytics;
using HordeServer.PerformanceTrends;

namespace HordeServer.Epic.EpicSandbox.Tests.PerformanceTrends
{
	[TestClass]
	public class WhereClauseBuilderTests
	{
		#region -- Empty Builder Tests --

		[TestMethod]
		public void Build_WithNoConditions_ReturnsEmptyClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder.Build();

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		#endregion -- Empty Builder Tests --

		#region -- And Tests --

		[TestMethod]
		public void And_SingleCondition_ReturnsWhereClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
			Assert.AreEqual("value1", result.Parameters[0].Value);
			Assert.AreEqual(DbType.AnsiString, result.Parameters[0].DbType);
		}

		[TestMethod]
		public void And_MultipleConditions_ReturnsAndJoinedClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("column1", "=", "value1", DbType.AnsiString)
					.And("column2", ">=", 100, DbType.Int32)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND column2 >= ?", result.Clause);
			Assert.AreEqual(2, result.Parameters.Count);
			Assert.AreEqual("value1", result.Parameters[0].Value);
			Assert.AreEqual(100, result.Parameters[1].Value);
		}

		[TestMethod]
		public void And_PreservesParameterOrder()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("col1", "=", "first", DbType.AnsiString)
					.And("col2", "=", "second", DbType.AnsiString)
					.And("col3", "=", "third", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("first", result.Parameters[0].Value);
			Assert.AreEqual("second", result.Parameters[1].Value);
			Assert.AreEqual("third", result.Parameters[2].Value);
		}

		#endregion -- And Tests --

		#region -- Or Tests --

		[TestMethod]
		public void Or_AfterAnd_ReturnsOrJoinedClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("column1", "=", "value1", DbType.AnsiString)
					.Or("column2", "=", "value2", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? OR column2 = ?", result.Clause);
			Assert.AreEqual(2, result.Parameters.Count);
		}

		[TestMethod]
		public void Or_AsFirstCondition_ReturnsWhereClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.Or("column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
		}

		#endregion -- Or Tests --

		#region -- AndIsNotNull Tests --

		[TestMethod]
		public void AndIsNotNull_SingleCondition_ReturnsIsNotNullClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIsNotNull("column1")
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 IS NOT NULL", result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		[TestMethod]
		public void AndIsNotNull_AfterAnd_ReturnsAndJoinedClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("column1", "=", "value1", DbType.AnsiString)
					.AndIsNotNull("column2")
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND column2 IS NOT NULL", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
		}

		#endregion -- AndIsNotNull Tests --

		#region -- AndIsNull Tests --

		[TestMethod]
		public void AndIsNull_SingleCondition_ReturnsIsNullClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIsNull("column1")
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 IS NULL", result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		#endregion -- AndIsNull Tests --

		#region -- AndIf Tests --

		[TestMethod]
		public void AndIf_WhenConditionTrue_AddsClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIf(true, "column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
		}

		[TestMethod]
		public void AndIf_WhenConditionFalse_DoesNotAddClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIf(false, "column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		[TestMethod]
		public void AndIf_MixedConditions_OnlyAddsTrue()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIf(true, "column1", "=", "value1", DbType.AnsiString)
					.AndIf(false, "column2", "=", "value2", DbType.AnsiString)
					.AndIf(true, "column3", "=", "value3", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND column3 = ?", result.Clause);
			Assert.AreEqual(2, result.Parameters.Count);
			Assert.AreEqual("value1", result.Parameters[0].Value);
			Assert.AreEqual("value3", result.Parameters[1].Value);
		}

		#endregion -- AndIf Tests --

		#region -- AndIfNotEmpty Tests --

		[TestMethod]
		public void AndIfNotEmpty_WithValue_AddsClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIfNotEmpty("testValue", "column1")
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
			Assert.AreEqual("testValue", result.Parameters[0].Value);
		}

		[TestMethod]
		public void AndIfNotEmpty_WithNull_DoesNotAddClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIfNotEmpty(null, "column1")
					.Build();

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		[TestMethod]
		public void AndIfNotEmpty_WithEmptyString_DoesNotAddClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.AndIfNotEmpty(String.Empty, "column1")
					.Build();

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		#endregion -- AndIfNotEmpty Tests --

		#region -- AndIfNotNull Tests --

		[TestMethod]
		public void AndIfNotNull_WithValue_AddsClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();
			int? value = 100;

			// Act
			WhereClauseResult result = builder
					.AndIfNotNull(value, "column1", ">=", DbType.Int32)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 >= ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
			Assert.AreEqual(100, result.Parameters[0].Value);
		}

		[TestMethod]
		public void AndIfNotNull_WithNull_DoesNotAddClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();
			int? value = null;

			// Act
			WhereClauseResult result = builder
					.AndIfNotNull(value, "column1", ">=", DbType.Int32)
					.Build();

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		[TestMethod]
		public void AndIfNotNull_WithDateTime_AddsClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();
			DateTime? value = new DateTime(2025, 1, 1, 0, 0, 0, DateTimeKind.Utc);

			// Act
			WhereClauseResult result = builder
					.AndIfNotNull(value, "timestamp", ">=", DbType.DateTime)
					.Build();

			// Assert
			Assert.AreEqual("WHERE timestamp >= ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
			Assert.AreEqual(value, result.Parameters[0].Value);
			Assert.AreEqual(DbType.DateTime, result.Parameters[0].DbType);
		}

		#endregion -- AndIfNotNull Tests --

		#region -- BeginGroup/EndGroup Tests --

		[TestMethod]
		public void BeginGroup_EndGroup_CreatesGroupedClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.And("column1", "=", "value1", DbType.AnsiString)
					.BeginGroupAnd()
					.And("column2", "=", "value2", DbType.AnsiString)
					.Or("column3", "=", "value3", DbType.AnsiString)
					.EndGroup()
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND (column2 = ? OR column3 = ?)", result.Clause);
			Assert.AreEqual(3, result.Parameters.Count);
		}

		[TestMethod]
		public void NestedGroups_CreatesNestedGroupedClause()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act
			WhereClauseResult result = builder
					.BeginGroupAnd()
					.And("col1", "=", "a", DbType.AnsiString)
					.Or("col2", "=", "b", DbType.AnsiString)
					.EndGroup()
					.And("col3", "=", "c", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE (col1 = ? OR col2 = ?) AND col3 = ?", result.Clause);
			Assert.AreEqual(3, result.Parameters.Count);
		}

		#endregion -- BeginGroup/EndGroup Tests --

		#region -- From Tests --

		[TestMethod]
		public void From_ExistingResult_PreservesClauseAndParameters()
		{
			// Arrange
			WhereClauseResult existing = new WhereClauseBuilder()
					.And("column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Act
			WhereClauseResult result = WhereClauseBuilder.From(existing)
					.And("column2", "=", "value2", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND column2 = ?", result.Clause);
			Assert.AreEqual(2, result.Parameters.Count);
			Assert.AreEqual("value1", result.Parameters[0].Value);
			Assert.AreEqual("value2", result.Parameters[1].Value);
		}

		[TestMethod]
		public void From_EmptyResult_ReturnsBuilderThatWorks()
		{
			// Arrange
			WhereClauseResult existing = new WhereClauseResult(String.Empty, new List<ParameterEntry>());

			// Act
			WhereClauseResult result = WhereClauseBuilder.From(existing)
					.And("column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ?", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
		}

		[TestMethod]
		public void From_CanAddIsNotNull()
		{
			// Arrange
			WhereClauseResult existing = new WhereClauseBuilder()
					.And("column1", "=", "value1", DbType.AnsiString)
					.Build();

			// Act
			WhereClauseResult result = WhereClauseBuilder.From(existing)
					.AndIsNotNull("column2")
					.Build();

			// Assert
			Assert.AreEqual("WHERE column1 = ? AND column2 IS NOT NULL", result.Clause);
			Assert.AreEqual(1, result.Parameters.Count);
		}

		#endregion -- From Tests --

		#region -- Fluent Chaining Tests --

		[TestMethod]
		public void FluentChaining_ReturnsBuilder()
		{
			// Arrange
			WhereClauseBuilder builder = new WhereClauseBuilder();

			// Act & Assert - verify fluent API returns builder
			Assert.IsInstanceOfType(builder.And("col", "=", "val", DbType.AnsiString), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.Or("col", "=", "val", DbType.AnsiString), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.AndIf(true, "col", "=", "val", DbType.AnsiString), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.AndIfNotEmpty("val", "col"), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.AndIfNotNull<int>(1, "col", "=", DbType.Int32), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.AndIsNotNull("col"), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.AndIsNull("col"), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.BeginGroupAnd(), typeof(WhereClauseBuilder));
			Assert.IsInstanceOfType(builder.EndGroup(), typeof(WhereClauseBuilder));
		}

		#endregion -- Fluent Chaining Tests --

		#region -- Integration with PerformanceTrendFilter Tests --

		[TestMethod]
		public void Builder_MatchesPerformanceTrendFilter_Output()
		{
			// Arrange
			PerformanceTrendFilter filter = new PerformanceTrendFilter
			{
				TestProject = "TestProject1",
				Platforms = ["Windows"],
				MinChangelist = 10000,
				MaxChangelist = 20000
			};

			// Act
			WhereClauseResult filterResult = filter.BuildWhereClause();

			// Assert - verify filter still produces expected output
			Assert.IsTrue(filterResult.Clause.StartsWith("WHERE ", StringComparison.Ordinal));
			Assert.IsTrue(filterResult.Clause.Contains(" AND ", StringComparison.Ordinal));
			Assert.AreEqual(4, filterResult.Parameters.Count);
		}

		#endregion -- Integration with PerformanceTrendFilter Tests --
	}
}
