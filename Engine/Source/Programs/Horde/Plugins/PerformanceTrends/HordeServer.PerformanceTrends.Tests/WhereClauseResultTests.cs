// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Odbc;
using HordeServer.Analytics;
using HordeServer.PerformanceTrends;

namespace HordeServer.Epic.EpicSandbox.Tests.PerformanceTrends
{
	[TestClass]
	public class WhereClauseResultTests
	{
		#region -- Constructor Tests --

		[TestMethod]
		public void WhereClauseResult_Constructor_SetsClause()
		{
			// Arrange
			string clause = "WHERE column = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>();

			// Act
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert
			Assert.AreEqual(clause, result.Clause);
		}

		[TestMethod]
		public void WhereClauseResult_Constructor_SetsParameters()
		{
			// Arrange
			string clause = "WHERE column = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "test", DbType.AnsiString)
			};

			// Act
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert
			Assert.AreEqual(1, result.Parameters.Count);
			Assert.AreEqual("test", result.Parameters[0].Value);
		}

		[TestMethod]
		public void WhereClauseResult_Constructor_EmptyClause()
		{
			// Arrange
			string clause = String.Empty;
			List<ParameterEntry> parameters = new List<ParameterEntry>();

			// Act
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert
			Assert.AreEqual(String.Empty, result.Clause);
			Assert.AreEqual(0, result.Parameters.Count);
		}

		[TestMethod]
		public void WhereClauseResult_Constructor_MultipleParameters()
		{
			// Arrange
			string clause = "WHERE col1 = ? AND col2 = ? AND col3 >= ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "value1", DbType.AnsiString),
				new ParameterEntry("p1", "value2", DbType.AnsiString),
				new ParameterEntry("p2", 100, DbType.Int32)
			};

			// Act
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert
			Assert.AreEqual(3, result.Parameters.Count);
			Assert.AreEqual("value1", result.Parameters[0].Value);
			Assert.AreEqual("value2", result.Parameters[1].Value);
			Assert.AreEqual(100, result.Parameters[2].Value);
		}

		#endregion -- Constructor Tests --

		#region -- ApplyTo Tests --

		[TestMethod]
		public void ApplyTo_AddsParametersToCommand()
		{
			// Arrange
			string clause = "WHERE column = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "test", DbType.AnsiString)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(1, cmd.Parameters.Count);
		}

		[TestMethod]
		public void ApplyTo_AddsMultipleParametersToCommand()
		{
			// Arrange
			string clause = "WHERE col1 = ? AND col2 = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "value1", DbType.AnsiString),
				new ParameterEntry("p1", 42, DbType.Int32)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(2, cmd.Parameters.Count);
		}

		[TestMethod]
		public void ApplyTo_WithNoParameters_DoesNotAddAnything()
		{
			// Arrange
			string clause = String.Empty;
			List<ParameterEntry> parameters = new List<ParameterEntry>();
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(0, cmd.Parameters.Count);
		}

		[TestMethod]
		public void ApplyTo_PreservesParameterOrder()
		{
			// Arrange
			string clause = "WHERE col1 = ? AND col2 = ? AND col3 = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "first", DbType.AnsiString),
				new ParameterEntry("p1", "second", DbType.AnsiString),
				new ParameterEntry("p2", "third", DbType.AnsiString)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual("first", cmd.Parameters[0].Value);
			Assert.AreEqual("second", cmd.Parameters[1].Value);
			Assert.AreEqual("third", cmd.Parameters[2].Value);
		}

		[TestMethod]
		public void ApplyTo_PreservesParameterTypes()
		{
			// Arrange
			string clause = "WHERE col1 = ? AND col2 >= ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "string", DbType.AnsiString),
				new ParameterEntry("p1", 100, DbType.Int32)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(DbType.AnsiString, cmd.Parameters[0].DbType);
			Assert.AreEqual(DbType.Int32, cmd.Parameters[1].DbType);
		}

		[TestMethod]
		public void ApplyTo_AppendsToExistingParameters()
		{
			// Arrange
			string clause = "WHERE col1 = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "new", DbType.AnsiString)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();
			cmd.Parameters.Add(new OdbcParameter { OdbcType = OdbcType.VarChar, Value = "existing" });

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(2, cmd.Parameters.Count);
			Assert.AreEqual("existing", cmd.Parameters[0].Value);
			Assert.AreEqual("new", cmd.Parameters[1].Value);
		}

		[TestMethod]
		public void ApplyTo_NullValueBecomesDBNull()
		{
			// Arrange
			string clause = "WHERE col1 = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", null, DbType.AnsiString)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			using OdbcCommand cmd = new OdbcCommand();

			// Act
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(DBNull.Value, cmd.Parameters[0].Value);
		}

		#endregion -- ApplyTo Tests --

		#region -- Readonly Struct Tests --

		[TestMethod]
		public void WhereClauseResult_IsReadonly_CannotModifyClause()
		{
			// Arrange
			string clause = "WHERE column = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>();
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert - Clause property has no setter, this is a compile-time check
			Assert.AreEqual(clause, result.Clause);
		}

		[TestMethod]
		public void WhereClauseResult_ParametersIsReadOnlyList()
		{
			// Arrange
			string clause = "WHERE column = ?";
			List<ParameterEntry> parameters = new List<ParameterEntry>
			{
				new ParameterEntry("p0", "test", DbType.AnsiString)
			};
			WhereClauseResult result = new WhereClauseResult(clause, parameters);

			// Assert
			Assert.IsInstanceOfType(result.Parameters, typeof(IReadOnlyList<ParameterEntry>));
		}

		#endregion -- Readonly Struct Tests --

		#region -- Integration with PerformanceTrendFilter Tests --

		[TestMethod]
		public void WhereClauseResult_FromFilter_CanBeAppliedToCommand()
		{
			// Arrange
			PerformanceTrendFilter filter = new PerformanceTrendFilter
			{
				TestProject = "TestProject1",
				Platforms = ["Windows"],
				MinChangelist = 10000
			};

			// Act
			WhereClauseResult result = filter.BuildWhereClause();

			using OdbcCommand cmd = new OdbcCommand();
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(3, cmd.Parameters.Count);
			Assert.IsTrue(result.Clause.StartsWith("WHERE ", StringComparison.Ordinal));
		}

		[TestMethod]
		public void WhereClauseResult_FromEmptyFilter_AppliesNoParameters()
		{
			// Arrange
			PerformanceTrendFilter filter = new PerformanceTrendFilter();

			// Act
			WhereClauseResult result = filter.BuildWhereClause();

			using OdbcCommand cmd = new OdbcCommand();
			result.ApplyTo(cmd);

			// Assert
			Assert.AreEqual(0, cmd.Parameters.Count);
			Assert.AreEqual(String.Empty, result.Clause);
		}

		#endregion -- Integration with PerformanceTrendFilter Tests --
	}
}
