<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="Polly" Version="8.2.1" />
    <PackageReference Include="System.Data.Odbc" Version="10.0.6" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.PerformanceTrends\EpicGames.Analytics.PerformanceTrends.csproj" />
    <ProjectReference Include="..\..\Build\HordeServer.Build\HordeServer.Build.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\StructuredAnalytics\HordeServer.StructuredAnalytics\HordeServer.StructuredAnalytics.csproj" />
  </ItemGroup>
  <PropertyGroup>
    <EmitCompilerGeneratedFiles>true</EmitCompilerGeneratedFiles>
    <CompilerGeneratedFilesOutputPath>$(BaseIntermediateOutputPath)g</CompilerGeneratedFilesOutputPath>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.Generators\EpicGames.Analytics.Generators.csproj" OutputItemType="Analyzer" ReferenceOutputAssembly="false" />
  </ItemGroup>
</Project>