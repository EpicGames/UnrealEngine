// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using EpicGames.Analytics.Generated;
using EpicGames.Analytics.PerformanceTrends;
using HordeServer.Analytics;
using HordeServer.Dialects;
using Microsoft.Extensions.Logging;

namespace HordeServer.PerformanceTrends
{

	/// <summary>
	/// Interface that describes performance trend summary handler that is used for dispatch based off of the summary type.
	///
	/// It is closely related to a <see cref="IPerformanceTrendsService"/>, in that it defines and performs the data retrieval as made specific by the summary type.
	/// </summary>
	/// <remarks>Non-generic interface for DI and polymorphic dispatch. Backend-agnostic: takes <see cref="DbConnection"/>, not <c>OdbcConnection</c>, so the same handler runs against any registered <see cref="IAnalyticsDataSource"/>.</remarks>
	public interface IPerformanceSummaryTrendHandler
	{
		/// <summary>
		/// The type the handler is designed to be dispatched for.
		/// </summary>
		string PerformanceSummaryType { get; }

		/// <summary>
		/// Gets a collection of <see cref="PerformanceTrendTelemetry"/> that corresponds to the underlying metric type.
		/// </summary>
		/// <param name="dbConnection">The open db connection.</param>
		/// <param name="filter">The filter to use in constraining the collection of telemetry.</param>
		/// <param name="logger">The logger.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>The collection of telemetry.</returns>
		Task<IEnumerable<PerformanceTrendTelemetry>> ProcessGeneralMetricRequest(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the distinct test projects for a specific Summary Trend.
		/// </summary>
		/// <param name="dbConnection">The open db connection.</param>
		/// <param name="filter">The filter to use in constraining the collection of telemetry.</param>
		/// <param name="logger">The logger.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of narrow performance trend telemetry that describes the test projects that meet the filter criteria.</returns>
		Task<IEnumerable<PerformanceTrendTelemetry>> ProcessDistinctTestProjectRequestAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the distinct platforms for a specific filter criteria.
		/// </summary>
		/// <param name="dbConnection">The open db connection.</param>
		/// <param name="filter">The filter to consider when obtaining platforms.</param>
		/// <param name="logger">The logger to use throughout.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of narrow performance trend telemetry that describes the platforms that meet the filter criteria.</returns>
		Task<IEnumerable<PerformanceTrendTelemetry>> GetDistinctPlatformsAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the distinct commits for a specific filter criteria.
		/// </summary>
		/// <param name="dbConnection">The open db connection.</param>
		/// <param name="filter">The filter to consider when obtaining commits.</param>
		/// <param name="logger">The logger to use throughout.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of narrow performance trend telemetry that describes the commits that meet the filter criteria.</returns>
		Task<IEnumerable<PerformanceTrendTelemetry>> GetDistinctCommitsAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Abstract class that defines the basic interface for a summary trend handler, as well as provides default implementations for common metadata filters on summary tables.
	/// </summary>
	public abstract class AbstractPerformanceSummaryTrendHandler<TTelemetry> : IPerformanceSummaryTrendHandler where TTelemetry : PerformanceTrendTelemetry
	{
		/// <summary>
		/// Constructor.
		/// </summary>
		protected AbstractPerformanceSummaryTrendHandler()
		{
		}

		/// <inheritdoc/>
		public abstract string PerformanceSummaryType { get; }

		#region -- IPerformanceSummaryTrendHandler Api --

		#region -- Abstract API --

		/// <inheritdoc/>
		public abstract Task<IEnumerable<PerformanceTrendTelemetry>> ProcessGeneralMetricRequest(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken);

		#endregion -- Abstract API --

		#region -- Virtual API - Common Performance Trend Telemetry Metadata Operations --

		/// <inheritdoc/>
		public virtual async Task<IEnumerable<PerformanceTrendTelemetry>> ProcessDistinctTestProjectRequestAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken)
		{
			IReadOnlyList<PerformanceTrendTelemetry> results = [];

			using (DbCommand cmd = dbConnection.CreateCommand())
			{
				// Create base filter and clause.
				WhereClauseResult whereClauseResult = filter.BuildWhereClause();

				// Instantiate a new builder to merge in non null for test identity.
				WhereClauseBuilder newBuilder = WhereClauseBuilder.From(whereClauseResult);
				newBuilder.AndIsNotNull(EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestIdentity);

				// Produce new merged result.
				WhereClauseResult mergedResult = newBuilder.Build();
				string whereSql = mergedResult.Clause;

				mergedResult.ApplyTo(cmd);

				string sql = $@"
								SELECT
									DISTINCT {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestName},
									{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.SummaryName},
									{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestIdentity},
									{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.GauntletSubTest},
									COALESCE({EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StreamId}, {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch}) AS {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.ComputedStream}
								FROM {GetTableName(filter)}
								{whereSql}";

				// We can suppress these as the issue here is:
				// - GetTableName - which is not user provided, and protected as a compile time, generated strings.
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities

				cmd.CommandType = CommandType.Text;

				using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
				{
					results = ParseTypeFromReader(reader);

					await reader.CloseAsync();
				}
			}

			return results;
		}

		/// <inheritdoc/>
		public virtual async Task<IEnumerable<PerformanceTrendTelemetry>> GetDistinctPlatformsAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<PerformanceTrendTelemetry> platforms;

			using (DbCommand cmd = dbConnection.CreateCommand())
			{
				WhereClauseResult whereClauseResult = filter.BuildWhereClause();
				string whereSql = whereClauseResult.Clause;
				whereClauseResult.ApplyTo(cmd);

				string sql = $@"
								SELECT
									DISTINCT
										{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Platform},
										{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.GauntletSubTest},
										COALESCE({EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StreamId}, {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch}) AS {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.ComputedStream}
								FROM {GetTableName(filter)}
								{whereSql}
								ORDER BY {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Platform} DESC";

				// We can suppress these as the issue here is:
				// - GetTableName - which is not user provided, and protected as a compile time, generated strings.
				// - whereSql - which is not user provided, and represents compliant parameter bindings.
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandType = CommandType.Text;

				using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
				{
					platforms = ParseTypeFromReader(reader);

					await reader.CloseAsync();
				}
			}

			return platforms;
		}

		/// <inheritdoc/>
		public virtual async Task<IEnumerable<PerformanceTrendTelemetry>> GetDistinctCommitsAsync(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<PerformanceTrendTelemetry> commits;

			using (DbCommand cmd = dbConnection.CreateCommand())
			{
				WhereClauseResult whereClauseResult = filter.BuildWhereClause();
				string whereSql = whereClauseResult.Clause;
				whereClauseResult.ApplyTo(cmd);

				string sql = $@"
								SELECT
									DISTINCT
										{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitIdOrdered},
										{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.GauntletSubTest},
										COALESCE({EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StreamId}, {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch}) AS {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.ComputedStream}
								FROM {GetTableName(filter)}
								{whereSql}
								ORDER BY {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitIdOrdered} DESC";

				// We can suppress these as the issue here is:
				// - GetTableName - which is not user provided, and protected as a compile time, generated strings.
				// - whereSql - which is not user provided, and represents compliant parameter bindings.
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandType = CommandType.Text;

				using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
				{
					commits = ParseTypeFromReader(reader);

					await reader.CloseAsync();
				}
			}

			return commits;
		}

		/// <summary>
		/// The schema/database the underlying table lives in (e.g. <c>ingest</c>), or null to use the backend's default database.
		/// </summary>
		protected abstract string? SchemaName { get; }

		/// <summary>
		/// The unqualified table name (may itself contain dots, e.g. <c>horde.state_job_summary</c>).
		/// </summary>
#pragma warning disable CA1721 // Property names should not match get methods
		protected abstract string TableName { get; }
#pragma warning restore CA1721 // Property names should not match get methods

		/// <summary>
		/// Returns the fully qualified, backend-formatted identifier suitable for inclusion in a SQL <c>FROM</c> clause. Routes through the dialect carried on the per-call filter (set by the calling service from the active backend's <see cref="IAnalyticsDataSource.Dialect"/>) so handlers always format identifiers per the consumer's selected backend.
		/// </summary>
		protected string GetTableName(PerformanceTrendFilter filter)
		{
			ISqlDialect dialect = filter.Dialect ?? PositionalQuestionMarkDialect.Instance;
			return dialect.FormatTableName(SchemaName, TableName);
		}

		/// <summary>
		/// Parses the concrete type from the reader, and returns the abstract telemetry type.
		/// </summary>
		/// <param name="reader">The reader to read from.</param>
		/// <returns>A list of performance trend telemetry items.</returns>
		protected abstract IReadOnlyList<PerformanceTrendTelemetry> ParseTypeFromReader(DbDataReader reader);

		#endregion -- Virtual API - Common Performance Trend Telemetry Metadata Operations --

		#endregion -- IPerformanceSummaryTrendHandler Api --
	}
}
