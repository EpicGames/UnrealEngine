// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Analytics.PerformanceTrends;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Interface for the Performance Trends service used for handling requests for performance trend data.
	/// </summary>
	/// <remarks>This is the primary extension point for implementing a performance trend service.</remarks>
	public interface IPerformanceTrendsService
	{
		/// <summary>
		/// Gets the performance summary records of a requested type.
		/// </summary>
		/// <param name="filter">Filter to used in querying performance trend data.</param>
		/// <param name="type">The type of handler</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>The list of resulting records.</returns>
		Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryRecordsAsync(PerformanceTrendFilter filter, string type, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the distinct test projects across all known summary type handlers.
		/// </summary>
		/// <param name="excludeOrphanedSummaryTypes">Whether to exclude results that have invalid summary types (and as a result would be orphaned).</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>The distinct test projects that are serviceable by the underlying service.</returns>
		Task<List<PerformanceTrendTelemetry>> GetPerformanceTrendTestProjectsAsync(bool excludeOrphanedSummaryTypes = true, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the distinct platforms for a specific filter criteria.
		/// </summary>
		/// <param name="filter">The filter to consider when obtaining platforms.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of narrow performance trend telemetry that describes the platforms that meet the filter criteria.</returns>
		Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryPlatformsAsync(PerformanceTrendFilter filter, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the distinct commits for a specific filter criteria.
		/// </summary>
		/// <param name="filter">The filter to consider when obtaining commits.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of narrow performance trend telemetry that describes the commits that meet the filter criteria.</returns>
		Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryCommitsAsync(PerformanceTrendFilter filter, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the list of performance trend types registered to the service.
		/// </summary>
		/// <returns>A list of possible performance trends.</returns>
		List<string> GetPerformanceTrendTypes();
	}
}
