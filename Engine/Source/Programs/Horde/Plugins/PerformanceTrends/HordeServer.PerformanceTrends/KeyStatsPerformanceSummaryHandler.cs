// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using Dapper;
using EpicGames.Analytics;
using EpicGames.Analytics.Generated;
using EpicGames.Analytics.PerformanceTrends;
using HordeServer.Analytics;
using Microsoft.Extensions.Logging;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Summary Handler for the KeyStats type.
	/// </summary>
	public class KeyStatsPerformanceSummaryHandler : AbstractPerformanceSummaryTrendHandler<KeyStatTelemetryRecord>
	{
		/// <summary>
		/// Static constructor — registers Dapper column mappings once per type.
		/// </summary>
		static KeyStatsPerformanceSummaryHandler()
		{
			ColumnHandlers.RegisterDapperColumnMapping<KeyStatTelemetryRecord>();
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		public KeyStatsPerformanceSummaryHandler()
		{
		}

		#region -- AbstractPerformanceSummaryTrendHandler API --

		/// <inheritdoc/>
		public override string PerformanceSummaryType => "KeyStats";

		/// <inheritdoc/>
		protected override string? SchemaName => EpicGames_Analytics_KeyStatTelemetryRecordGen.SchemaName;

		/// <inheritdoc/>
		protected override string TableName => EpicGames_Analytics_KeyStatTelemetryRecordGen.TableName;

		/// <inheritdoc/>
		public override async Task<IEnumerable<PerformanceTrendTelemetry>> ProcessGeneralMetricRequest(DbConnection dbConnection, PerformanceTrendFilter filter, ILogger logger, CancellationToken cancellationToken)
		{
			List<KeyStatTelemetryRecord> results = [];

			using (DbCommand cmd = dbConnection.CreateCommand())
			{
				WhereClauseResult whereClauseResult = filter.BuildWhereClause();
				string whereSql = whereClauseResult.Clause;
				whereClauseResult.ApplyTo(cmd);

				string sql = $@"
								SELECT
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.EventName},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.StreamId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.JobId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.TemplateId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.StepId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.CommitId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.CommitIdOrdered},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.CsvId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.GpuTimeAvg},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.PhysicalUsedMbMax},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.Collated},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.DynamicResolutionVsmNanite},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.MVP},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.HitchesMin},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.HitchTimePercent},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.GameThreadtimeAvg},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.RenderThreadtimeAvg},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.FrametimeAvg},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.SummaryName},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.DynamicResolutionPercentageAvg},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.EngineVersion},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.EngineReleaseVersion},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.BuildVersion},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.Platform},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.TestId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.TestName},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.TestConfigName},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.GauntletTestType},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.GauntletSubTest},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.IsBuildMachine},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.HordeUrlStr},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.StartTimestamp},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.EndTimestamp},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.SessionId},
									{EpicGames_Analytics_KeyStatTelemetryRecordGen.SchemaVersion},
									COALESCE({EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StreamId}, {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch}) AS {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.ComputedStream},
									{EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch}
								FROM {GetTableName(filter)}
								{whereSql}
								LIMIT {filter.RecordCount}";

				// We can suppress these as the issue here is:
				// - recordCount - whilst it is user provided, it's bound by hard limits.
				// - whereSql - which is not user provided, and represents compliant parameter bindings.
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandType = CommandType.Text;
				using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
				{
					results = reader.Parse<KeyStatTelemetryRecord>().AsList();

					await reader.CloseAsync();
				}
			}

			return results;
		}

		/// <inheritdoc/>
		protected override IReadOnlyList<PerformanceTrendTelemetry> ParseTypeFromReader(DbDataReader reader)
		{
			return reader.Parse<KeyStatTelemetryRecord>().ToList();
		}

		#endregion -- AbstractPerformanceSummaryTrendHandler API --
	}
}
