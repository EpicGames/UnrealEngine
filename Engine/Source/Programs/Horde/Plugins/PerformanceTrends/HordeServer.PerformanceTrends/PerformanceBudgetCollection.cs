// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.PerformanceTrends;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Performance Budget Collection implementation.
	/// </summary>
	class PerformanceBudgetCollection : IPerformanceBudgetCollection
	{
		#region -- Private Members --

		readonly IMongoCollection<PerformanceBudgetDocument> _performanceBudgetDocuments;
		readonly ILogger<PerformanceBudgetCollection> _logger;

		#endregion -- Private Members --

		#region -- Constructor --

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="mongoService">The underlying mongo service to use.</param>
		/// <param name="logger">The logger.</param>
		public PerformanceBudgetCollection(IMongoService mongoService, ILogger<PerformanceBudgetCollection> logger)
		{
			List<MongoIndex<PerformanceBudgetDocument>> indexes = new List<MongoIndex<PerformanceBudgetDocument>>();

			// Primary query index: ComputedStream + TestProject (most common filter combination)
			indexes.Add(x => x.Ascending(d => d.ComputedStream).Ascending(d => d.TestProject));

			// Index for "all platforms" budgets - used when filtering by platform
			// Query: AppliesToAllPlatforms == true OR Platforms.Contains(platform)
			indexes.Add(x => x.Ascending(d => d.ComputedStream).Ascending(d => d.AppliesToAllPlatforms));

			// Multikey index for platform lookups - enables efficient AnyEq queries on the Platforms array
			indexes.Add(x => x.Ascending(d => d.ComputedStream).Ascending(d => d.Platforms));

			_performanceBudgetDocuments = mongoService.GetCollection<PerformanceBudgetDocument>("PerformanceBudgets", indexes);
			_logger = logger;
		}

		#endregion -- Constructor --

		#region -- Interface API --

		/// <inheritdoc/>
		public async Task<IPerformanceBudget?> AddPerformanceBudgetAsync(UserId? owner, PerformanceBudgetAddRequest request, CancellationToken cancellationToken)
		{
			if (!request.IsValidAddRequest())
			{
				_logger.LogWarning("Invalid add performance budget request. IsValidName: {IsValidName}; IsValidDescription: {IsValidDescription}; IsValidThresholds: {IsValidThresholds}",
					request.IsValidName(), request.IsValidDescription(), request.IsValidThresholds());
				return null;
			}

			List<MetricThresholdDocument> thresholds = request.Thresholds.Select(t => new MetricThresholdDocument
			{
				TestType = t.TestType,
				MetricName = t.MetricName,
				ThresholdValue = t.ThresholdValue,
				LargerIsWorse = t.LargerIsWorse
			}).ToList();

			List<string>? platforms = request.Platforms?.ToList();
			bool appliesToAllPlatforms = platforms == null || platforms.Count == 0;

			PerformanceBudgetDocument newBudget = new PerformanceBudgetDocument()
			{
				Id = new PerformanceBudgetId(BinaryIdUtils.CreateNew()),
				Name = request.Name,
				Description = request.Description,
				Owner = owner,
				ComputedStream = request.ComputedStream,
				TestProject = request.TestProject,
				Platforms = platforms,
				AppliesToAllPlatforms = appliesToAllPlatforms,
				Thresholds = thresholds,
				UpdateTimeUtc = DateTime.UtcNow
			};

			await _performanceBudgetDocuments.InsertOneAsync(newBudget, null, cancellationToken);
			return newBudget;
		}

		/// <inheritdoc/>
		public async Task<IPerformanceBudget?> UpdatePerformanceBudgetAsync(PerformanceBudgetId budgetId, PerformanceBudgetUpdateRequest request, CancellationToken cancellationToken)
		{
			if (!request.IsValidUpdateRequest())
			{
				_logger.LogWarning("Invalid update performance budget request. IsValidName: {IsValidName}; IsValidDescription: {IsValidDescription}; IsValidThresholds: {IsValidThresholds}",
					request.IsValidName(), request.IsValidDescription(),request.IsValidThresholds());
				return null;
			}

			if (!request.HasUpdates())
			{
				_logger.LogInformation("No updates to apply for budgetId ({BudgetId}). Returning.", budgetId);
				return null;
			}

			FilterDefinition<PerformanceBudgetDocument> filter = Builders<PerformanceBudgetDocument>.Filter.Expr(x => x.Id == budgetId);
			List<UpdateDefinition<PerformanceBudgetDocument>> updates = new List<UpdateDefinition<PerformanceBudgetDocument>>();

			if (request.Name != null)
			{
				updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.Name, request.Name));
			}

			if (request.Description != null)
			{
				updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.Description, String.IsNullOrEmpty(request.Description) ? null : request.Description));
			}

			if (request.Platforms != null)
			{
				// Empty list clears platforms (applies to all), null means no change
				List<string>? platforms = request.Platforms.Count == 0 ? null : request.Platforms.ToList();
				bool appliesToAllPlatforms = platforms == null || platforms.Count == 0;
				updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.Platforms, platforms));
				updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.AppliesToAllPlatforms, appliesToAllPlatforms));
			}

			if (request.Thresholds != null)
			{
				List<MetricThresholdDocument> thresholds = request.Thresholds.Select(t => new MetricThresholdDocument
				{
					TestType = t.TestType,
					MetricName = t.MetricName,
					ThresholdValue = t.ThresholdValue,
					LargerIsWorse = t.LargerIsWorse
				}).ToList();
				updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.Thresholds, thresholds));
			}

			// We have no updates, just return null.
			if (updates.Count == 0)
			{
				_logger.LogInformation("No updates to apply for budgetId ({BudgetId}). Returning.", budgetId);
				return null;
			}

			updates.Add(Builders<PerformanceBudgetDocument>.Update.Set(x => x.UpdateTimeUtc, DateTime.UtcNow));

			UpdateDefinition<PerformanceBudgetDocument> update = Builders<PerformanceBudgetDocument>.Update.Combine(updates);
			IPerformanceBudget? budget = await _performanceBudgetDocuments.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<PerformanceBudgetDocument, PerformanceBudgetDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);

			if (budget == null)
			{
				_logger.LogInformation("Unable to update Performance Budget with Id: {BudgetId}.", budgetId);
			}

			return budget;
		}

		/// <inheritdoc/>
		public async Task<IPerformanceBudget?> GetPerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken)
		{
			FilterDefinition<PerformanceBudgetDocument> filter = Builders<PerformanceBudgetDocument>.Filter.Expr(x => x.Id == budgetId);
			IPerformanceBudget? document = await _performanceBudgetDocuments.Find(filter).FirstOrDefaultAsync(cancellationToken);

			return document;
		}

		/// <inheritdoc/>
		public async Task<IEnumerable<IPerformanceBudget>> GetPerformanceBudgetsAsync(string computedStream, string? testProject = null, string? platform = null, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<PerformanceBudgetDocument> filterBuilder = Builders<PerformanceBudgetDocument>.Filter;
			FilterDefinition<PerformanceBudgetDocument> filter = filterBuilder.Empty;

			filter &= filterBuilder.Eq(x => x.ComputedStream, computedStream);

			if (!String.IsNullOrEmpty(testProject))
			{
				filter &= filterBuilder.Eq(x => x.TestProject, testProject);
			}

			if (!String.IsNullOrEmpty(platform))
			{
				// Return budgets that either apply to all platforms OR include the specified platform
				// AppliesToAllPlatforms is indexed for efficient lookup of "all platforms" budgets
				// Platforms has a multikey index for efficient AnyEq queries
				filter &= filterBuilder.Or(
					filterBuilder.Eq(x => x.AppliesToAllPlatforms, true),
					filterBuilder.AnyEq(x => x.Platforms, platform)
				);
			}

			List<PerformanceBudgetDocument> documents = await _performanceBudgetDocuments.Find(filter).ToListAsync(cancellationToken);

			return documents;
		}

		/// <inheritdoc/>
		public async Task<bool> DeletePerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken)
		{
			FilterDefinition<PerformanceBudgetDocument> filter = Builders<PerformanceBudgetDocument>.Filter.Expr(x => x.Id == budgetId);
			DeleteResult result = await _performanceBudgetDocuments.DeleteOneAsync(filter, cancellationToken);

			return result.DeletedCount == 1;
		}

		#endregion -- Interface API --

		#region -- Internal Types --

		/// <summary>
		/// Metric threshold document for storage in MongoDB.
		/// </summary>
		class MetricThresholdDocument : IMetricThreshold
		{
			/// <inheritdoc/>
			[BsonElement("tt")]
			public string TestType { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("mn")]
			public string MetricName { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("tv")]
			public double ThresholdValue { get; set; }

			/// <inheritdoc/>
			[BsonElement("liw")]
			public bool LargerIsWorse { get; set; }
		}

		/// <summary>
		/// Performance budget document.
		/// </summary>
		class PerformanceBudgetDocument : IPerformanceBudget
		{
			/// <inheritdoc/>
			[BsonRequired, BsonId]
			public PerformanceBudgetId Id { get; set; }

			/// <inheritdoc/>
			[BsonElement("n")]
			public string Name { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("d")]
			public string? Description { get; set; }

			/// <inheritdoc/>
			[BsonElement("o")]
			public UserId? Owner { get; set; }

			/// <inheritdoc/>
			[BsonElement("cs")]
			public string ComputedStream { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("tp")]
			public string TestProject { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("pls")]
			public List<string>? Platforms { get; set; }

			/// <summary>
			/// Computed flag indicating this budget applies to all platforms.
			/// Set to true when Platforms is null or empty. Used for efficient indexing.
			/// </summary>
			[BsonElement("aap")]
			public bool AppliesToAllPlatforms { get; set; }

			/// <summary>
			/// The platforms as a read-only list for the interface.
			/// </summary>
			IReadOnlyList<string>? IPerformanceBudget.Platforms => Platforms;

			/// <inheritdoc/>
			[BsonElement("th")]
			public List<MetricThresholdDocument> Thresholds { get; set; } = new();

			/// <summary>
			/// The thresholds as a read-only list for the interface.
			/// </summary>
			IReadOnlyList<IMetricThreshold> IPerformanceBudget.Thresholds => Thresholds;

			/// <inheritdoc/>
			[BsonElement("ut")]
			public DateTime UpdateTimeUtc { get; set; }
		}

		#endregion -- Internal Types --
	}
}
