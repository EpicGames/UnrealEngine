// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using EpicGames.Analytics.Generated;
using HordeServer.Analytics;
using HordeServer.Dialects;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Filter object for common performance trend queries.
	/// </summary>
	public struct PerformanceTrendFilter
	{
		/// <summary>
		/// The maximum record count to return from a performance trend.
		/// </summary>
		public const int MaximumRecordCount = 10000;

		private int _recordCount;

		/// <summary>
		/// The maximum number of records to obtain.
		/// </summary>
		public int RecordCount
		{
			get => _recordCount;
			set => _recordCount = Math.Clamp(value, 0, MaximumRecordCount);
		}

		/// <summary>
		/// The summary name to filter by.
		/// </summary>
		public string? SummaryName { get; set; }

		/// <summary>
		/// The platforms to filter by.
		/// </summary>
		public string[]? Platforms { get; set; }

		/// <summary>
		/// The minimum changelist to obtain.
		/// </summary>
		public int? MinChangelist { get; set; }

		/// <summary>
		/// The maximum changelist to obtain.
		/// </summary>
		public int? MaxChangelist { get; set; }

		/// <summary>
		/// The minimum date to obtain.
		/// </summary>
		public DateTime? MinDate { get; set; }

		/// <summary>
		/// The maximum date to obtain.
		/// </summary>
		public DateTime? MaxDate { get; set; }

		/// <summary>
		/// The test project to limit the selection to.
		/// </summary>
		public string? TestProject { get; set; }

		/// <summary>
		/// The test identity to limit the selection to.
		/// </summary>
		public string? TestIdentity { get; set; }

		/// <summary>
		/// The test types to limit the selection to.
		/// </summary>
		public string[]? TestTypes { get; set; }

		/// <summary>
		/// The minimum schema version to apply.
		/// </summary>
		public int? MinSchemaVersion { get; set; }

		/// <summary>
		/// The set of compute streams to limit the selection to.
		/// </summary>
		/// <remarks>
		/// This is a computed stream in that it represents a special COALESCE action for the branch and streamId.
		/// </remarks>
		public string[]? ComputedStreams { get; set; }

		/// <summary>
		/// Backend dialect used by handlers for WHERE-clause materialisation, table-name formatting, and parameter binding. Set by the calling service so handlers don't need to thread it through their method signatures. Falls back to positional <c>?</c> placeholders when null (ODBC/test behavior).
		/// </summary>
		public ISqlDialect? Dialect { get; set; }

		/// <summary>
		/// Constructs a <see cref="WhereClauseResult"/> based on the AND of every enabled filter parameter.
		/// </summary>
		/// <returns>A where clause result containing the sql where clause, and the bound <see cref="ParameterEntry"/> list.</returns>
		public WhereClauseResult BuildWhereClause()
		{
			WhereClauseBuilder builder = new WhereClauseBuilder(Dialect ?? PositionalQuestionMarkDialect.Instance)
				.AndIfNotEmpty(TestProject, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestName)
				.AndIfNotEmpty(SummaryName, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.SummaryName)
				.AndIfNotEmpty(TestIdentity, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestIdentity)
				.AndIfNotNull(MinChangelist, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitIdOrdered, ">=", DbType.Int32)
				.AndIfNotNull(MaxChangelist, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitIdOrdered, "<=", DbType.Int32)
				.AndIfNotNull(MinDate, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StartTimestamp, ">=", DbType.DateTime)
				.AndIfNotNull(MaxDate, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StartTimestamp, "<=", DbType.DateTime)
				.AndIfNotNull(MinSchemaVersion, EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.SchemaVersion, ">=", DbType.Int32);

			if (TestTypes != null && TestTypes.Length > 0 && TestTypes.Any(x => !String.IsNullOrEmpty(x)))
			{
				builder.BeginGroupAnd();

				for (int i = 0; i < TestTypes.Length; ++i)
				{
					if (!String.IsNullOrEmpty(TestTypes[i]))
					{
						builder.Or(EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.GauntletSubTest, "=", TestTypes[i], DbType.AnsiString);
					}
				}

				builder.EndGroup();
			}

			if (Platforms != null && Platforms.Length > 0 && Platforms.Any(x => !String.IsNullOrEmpty(x)))
			{
				builder.BeginGroupAnd();

				for (int i = 0; i < Platforms.Length; ++i)
				{
					if (!String.IsNullOrEmpty(Platforms[i]))
					{
						builder.Or(EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Platform, "=", Platforms[i], DbType.AnsiString);
					}
				}

				builder.EndGroup();
			}

			if (ComputedStreams != null && ComputedStreams.Length > 0 && ComputedStreams.Any(x => !String.IsNullOrEmpty(x)))
			{
				builder.BeginGroupAnd();

				for (int i = 0; i < ComputedStreams.Length; ++i)
				{
					if (!String.IsNullOrEmpty(ComputedStreams[i]))
					{
						builder.Or($"COALESCE({EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.StreamId}, {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.Branch})", "=", ComputedStreams[i], DbType.AnsiString);
					}
				}
				builder.EndGroup();
			}

			return builder.Build();
		}
	}
}
