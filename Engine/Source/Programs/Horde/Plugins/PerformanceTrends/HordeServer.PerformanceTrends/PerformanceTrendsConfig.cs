// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer
{
	/// <summary>
	/// Configuration for the PerformanceTrends plugin
	/// </summary>
	public class PerformanceTrendsConfig : StructuredAnalyticsConfig
	{
		/// <summary>
		/// The minimum Performance Trend Schema Version to allow.
		/// </summary>
		public int? MinQuerySchemaVersion { get; set; }

		/// <summary>
		/// Whether to hide external results. A result is considered external if the source data did not originate from a Horde job.
		/// </summary>
		public bool HideExternalResults { get; set; }
	}
}
