// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Analytics.PerformanceTrends;
using EpicGames.Horde.PerformanceTrends;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Users;
using HordeServer.PerformanceTrends.Responses;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Performance Trends Controller
	/// </summary>
	[Authorize]
	[ApiController]
	public class PerformanceTrendsController : HordeControllerBase
	{
		private readonly IPerformanceTrendsService _performanceTrendsService;
		private readonly IPerformanceBudgetCollection _performanceBudgetCollection;
		private readonly IUserCollection _userCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly IOptionsSnapshot<PerformanceTrendsConfig> _performanceTrendsConfig;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="performanceTrendsService">Performance trend service to use in the controller.</param>
		/// <param name="performanceBudgetCollection">Performance budget collection to use in the controller.</param>
		/// <param name="userCollection">The user collection for resolving owner details.</param>
		/// <param name="performanceTrendsConfig">Performance config for controller.</param>
		/// <param name="buildConfig">Build config for authorization.</param>
		public PerformanceTrendsController(IPerformanceTrendsService performanceTrendsService, IPerformanceBudgetCollection performanceBudgetCollection, IUserCollection userCollection, IOptionsSnapshot<PerformanceTrendsConfig> performanceTrendsConfig, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_performanceTrendsService = performanceTrendsService;
			_performanceBudgetCollection = performanceBudgetCollection;
			_userCollection = userCollection;
			_performanceTrendsConfig = performanceTrendsConfig;
			_buildConfig = buildConfig;
		}

		#region -- Performance Trends API --

		/// <summary>
		/// Gets performance metrics.
		/// </summary>
		/// <param name="count">The number of records to obtain.</param>
		/// <param name="type">The type of summary telemetry to obtain.</param>
		/// <param name="testProject">The test project to include in the results.</param>
		/// <param name="testIdentity">The test identity for the performance telemetry to include in the results.</param>
		/// <param name="testTypes">The test types for the performance telemetry to include in the results.</param>
		/// <param name="platforms">List of platforms to filter for.</param>
		/// <param name="streams">The streams to filter for.</param>
		/// <param name="startCommitIdOrdered">The ordered commit id to start the filter range with.</param>
		/// <param name="endCommitIdOrdered">The ordered commit id to end the filter range with.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of metric entries.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/metrics")]
		[ProducesResponseType(typeof(List<PerformanceTrendTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetPerformanceMetricsAsync([FromQuery] string type, [FromQuery] string[] streams, [FromQuery] int count = 100, [FromQuery] string? testProject = null, [FromQuery] string? testIdentity = null, [FromQuery] string[]? testTypes = null, [FromQuery] string[]? platforms = null, [FromQuery] int? startCommitIdOrdered = null, [FromQuery] int? endCommitIdOrdered = null, CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streams == null || streams.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			string[] authorizedStreams = FilterAuthorizedStreams(streams);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			PerformanceTrendFilter filter = new PerformanceTrendFilter() { RecordCount = count };

			if (endCommitIdOrdered != null)
			{
				filter.MaxChangelist = endCommitIdOrdered;
			}

			if (startCommitIdOrdered != null)
			{
				filter.MinChangelist = startCommitIdOrdered;
			}

			if (!String.IsNullOrEmpty(testProject))
			{
				filter.TestProject = testProject;
			}

			if (!String.IsNullOrEmpty(testIdentity))
			{
				filter.TestIdentity = testIdentity;
			}

			if (testTypes != null && testTypes.Length > 0)
			{
				filter.TestTypes = testTypes;
			}

			filter.ComputedStreams = authorizedStreams;

			if (platforms != null && platforms.Length > 0)
			{
				filter.Platforms = platforms;
			}

			List<PerformanceTrendTelemetry> telemetry = await _performanceTrendsService.GetPerformanceSummaryRecordsAsync(filter, type, cancellationToken);

			return Ok(telemetry.Cast<object>().ToList());
		}

		/// <summary>
		/// Gets all the known test projects across all summary table types.
		/// </summary>
		/// <param name="excludeOrphanedSummaryTypes">Whether to exclude results that have invalid summary types (and as a result would be orphaned).</param>
		/// <param name="streams">Optional streams to filter results by. If provided, only test projects with matching authorized streams are returned. If not provided, results are filtered implicitly based on the user's authentication.</param>
		/// <param name="cancellationToken">The token to use throughout.</param>
		/// <returns>Test projects and their possible summary table types.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/testprojects")]
		public async Task<ActionResult<List<TestProjectResponse>>> GetPerformanceTestProjectsAsync([FromQuery] bool excludeOrphanedSummaryTypes = true, [FromQuery] string[]? streams = null, CancellationToken cancellationToken = default)
		{
			// Determine authorized streams if explicit filter is provided
			HashSet<string>? authorizedStreamSet = null;
			if (streams != null && streams.Length > 0)
			{
				string[] authorizedStreams = FilterAuthorizedStreams(streams);
				if (authorizedStreams.Length == 0)
				{
					return Forbid("User is not authorized to view any of the requested streams.");
				}
				authorizedStreamSet = new HashSet<string>(authorizedStreams, StringComparer.OrdinalIgnoreCase);
			}

			List<PerformanceTrendTelemetry> telemetry = await _performanceTrendsService.GetPerformanceTrendTestProjectsAsync(excludeOrphanedSummaryTypes, cancellationToken);
			List<TestProjectResponse> response = new List<TestProjectResponse>(telemetry.Count);

			for (int i = 0; i < telemetry.Count; ++i)
			{
				TestProjectResponse testProject = TestProjectResponse.CreateTestProjectResponse(telemetry[i]);

				// If explicit stream filter is provided, only include matching authorized streams
				if (authorizedStreamSet != null)
				{
					if (String.IsNullOrEmpty(testProject.ComputedStream) || !authorizedStreamSet.Contains(testProject.ComputedStream))
					{
						continue;
					}
				}
				else
				{
					// No explicit filter - check authorization for each result's stream
					if (!IsAuthorizedForStream(testProject.ComputedStream))
					{
						continue;
					}
				}

				response.Add(testProject);
			}

			return response;
		}

		/// <summary>
		/// Gets the distinct platforms given the filter parameters.
		/// </summary>
		/// <param name="metricSummaryType">The summary metricSummaryType to obtain distinct platforms for.</param>
		/// <param name="streams">The streams to filter for. Required parameter.</param>
		/// <param name="testProject">The test project to filter for.</param>
		/// <param name="testIdentity">The test identity to filter for.</param>
		/// <param name="testTypes">The test types to filter for.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>A list of all the distinct platforms for a summary metricSummaryType, given the filter criteria.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/platforms")]
		public async Task<ActionResult<List<TestProjectPlatformResponse>>> GetPlatformsAsync([FromQuery] string metricSummaryType, [FromQuery] string[] streams, [FromQuery] string? testProject = null, [FromQuery] string? testIdentity = null, [FromQuery] string[]? testTypes = null, CancellationToken cancellationToken = default)
		{
			if (String.IsNullOrEmpty(metricSummaryType))
			{
				return BadRequest("Insufficient required parameters: metricSummaryType");
			}

			// Validate streams parameter is provided
			if (streams == null || streams.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			string[] authorizedStreams = FilterAuthorizedStreams(streams);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			PerformanceTrendFilter filter = new PerformanceTrendFilter();

			filter.SummaryName = metricSummaryType;

			if (!String.IsNullOrEmpty(testProject))
			{
				filter.TestProject = testProject;
			}

			if (!String.IsNullOrEmpty(testIdentity))
			{
				filter.TestIdentity = testIdentity;
			}

			if (testTypes != null && testTypes.Length > 0)
			{
				filter.TestTypes = testTypes;
			}

			filter.ComputedStreams = authorizedStreams;

			List<PerformanceTrendTelemetry> distinctPlatforms = await _performanceTrendsService.GetPerformanceSummaryPlatformsAsync(filter, cancellationToken);

			// Group platforms by stream and test type to return compressed responses
			List<TestProjectPlatformResponse> groupedByStreamAndTestType = distinctPlatforms
				.GroupBy(p => new { p.ComputedStream, p.GauntletSubTest })
				.Select(g => new TestProjectPlatformResponse(
					filter.SummaryName,
					filter.TestProject,
					filter.TestIdentity,
					g.Key.GauntletSubTest,
					g.Key.ComputedStream,
					g.Select(p => p.Platform).Where(p => p != null).Cast<string>().ToArray()))
				.ToList();

			return groupedByStreamAndTestType;
		}

		/// <summary>
		/// Gets the distinct commits given the filter parameters.
		/// </summary>
		/// <param name="metricSummaryType">The summary metricSummaryType to obtain distinct commits for.</param>
		/// <param name="streams">The streams to filter for. Required parameter.</param>
		/// <param name="testProject">The test project to filter for.</param>
		/// <param name="testIdentity">The test identity to filter for.</param>
		/// <param name="testTypes">The test types to filter for.</param>
		/// <param name="platforms">The platforms to filter for.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>An ascending ordered list of all the distinct commits for a summary metricSummaryType, given the filter criteria and where the commit id is not <see cref="HordeContextTelemetryRecord.InvalidCommitId"/>.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/commits")]
		public async Task<ActionResult<List<TestProjectCommitResponse>>> GetCommitsAsync([FromQuery] string metricSummaryType, [FromQuery] string[] streams, [FromQuery] string? testProject = null, [FromQuery] string? testIdentity = null, [FromQuery] string[]? testTypes = null, [FromQuery] string[]? platforms = null, CancellationToken cancellationToken = default)
		{
			if (String.IsNullOrEmpty(metricSummaryType))
			{
				return BadRequest("Insufficient required parameters: metricSummaryType");
			}

			// Validate streams parameter is provided
			if (streams == null || streams.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			string[] authorizedStreams = FilterAuthorizedStreams(streams);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			PerformanceTrendFilter filter = new PerformanceTrendFilter();

			filter.SummaryName = metricSummaryType;

			if (!String.IsNullOrEmpty(testProject))
			{
				filter.TestProject = testProject;
			}

			if (!String.IsNullOrEmpty(testIdentity))
			{
				filter.TestIdentity = testIdentity;
			}

			if (testTypes != null && testTypes.Length > 0)
			{
				filter.TestTypes = testTypes;
			}

			filter.ComputedStreams = authorizedStreams;

			if (platforms != null && platforms.Length > 0)
			{
				filter.Platforms = platforms;
			}

			List<PerformanceTrendTelemetry> distinctCommits = await _performanceTrendsService.GetPerformanceSummaryCommitsAsync(filter, cancellationToken);

			// Group commits by stream and test type to return compressed responses
			List<TestProjectCommitResponse> groupedByStreamAndTestType = distinctCommits
				.GroupBy(c => new { c.ComputedStream, c.GauntletSubTest })
				.Select(g => new TestProjectCommitResponse(
					filter.SummaryName,
					filter.TestProject,
					filter.TestIdentity,
					g.Key.GauntletSubTest,
					g.Key.ComputedStream,
					g.Select(c => c.CommitIdOrdered ?? HordeContextTelemetryRecord.InvalidCommitId).Where(c => c != HordeContextTelemetryRecord.InvalidCommitId).OrderBy(id => id).ToArray()))
				.ToList();

			return groupedByStreamAndTestType;
		}

		/// <summary>
		/// Gets the types of performance trends available.
		/// </summary>
		/// <returns>A list of all the performance trend types that this server supports.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/types")]
		public ActionResult<List<string>> GetPerformanceTrendTypes()
		{
			return _performanceTrendsService.GetPerformanceTrendTypes();
		}

		#endregion -- Performance Trends API --

		#region -- Budget CRUD Endpoints --

		/// <summary>
		/// Creates a new performance budget group.
		/// </summary>
		/// <param name="request">The add request containing budget details including name, description, and metric thresholds.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The newly created budget if successful.</returns>
		/// <remarks>The requesting user will become the implicit owner of the new budget.</remarks>
		[HttpPost]
		[Route("/api/v1/performancetrends/budgets")]
		[ProducesResponseType(typeof(PerformanceBudgetResponse), 200)]
		public async Task<ActionResult<PerformanceBudgetResponse>?> AddPerformanceBudgetAsync([FromBody] PerformanceBudgetAddRequest request, CancellationToken cancellationToken = default)
		{
			// Validate inputs
			if (String.IsNullOrEmpty(request.Name))
			{
				return BadRequest("No name provided. Name is a required parameter.");
			}

			if (String.IsNullOrEmpty(request.TestProject))
			{
				return BadRequest("No test project provided. Test project is a required parameter.");
			}

			if (request.Thresholds == null || request.Thresholds.Count == 0)
			{
				return BadRequest("No thresholds provided. At least one metric threshold is required.");
			}

			if (!request.IsValidAddRequest())
			{
				return BadRequest($"AddRequest was not valid. Check name length (Max:{PerformanceBudgetRestrictions.MaxNameLength}), description length (Max:{PerformanceBudgetRestrictions.MaxDescriptionLength}), and thresholds count (Max:{PerformanceBudgetRestrictions.MaxThresholdsCount}).");
			}

			// Validate stream exists and user has access
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeStream(request.ComputedStream);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			UserId? requestingUser = User.GetUserId();
			IPerformanceBudget? budget = await _performanceBudgetCollection.AddPerformanceBudgetAsync(requestingUser, request, cancellationToken);

			if (budget == null)
			{
				return StatusCode(StatusCodes.Status500InternalServerError);
			}

			return await CreatePerformanceBudgetResponseWithOwnerAsync(budget, cancellationToken);
		}

		/// <summary>
		/// Updates an existing performance budget group.
		/// </summary>
		/// <param name="budgetId">The id of the budget to update.</param>
		/// <param name="request">The update request containing fields to modify (name, description, platforms, thresholds).</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The updated budget if successful.</returns>
		/// <remarks>Users can only update budgets that don't have an owner, ones they own, or any budget if they are an admin.</remarks>
		[HttpPut]
		[Route("/api/v1/performancetrends/budgets/{budgetId}")]
		[ProducesResponseType(typeof(PerformanceBudgetResponse), 200)]
		public async Task<ActionResult<PerformanceBudgetResponse>?> UpdatePerformanceBudgetAsync(PerformanceBudgetId budgetId, [FromBody] PerformanceBudgetUpdateRequest request, CancellationToken cancellationToken = default)
		{
			// Validate inputs
			if (!request.HasUpdates())
			{
				return BadRequest("No updates provided. Please provide at least one field to update (name, description, platforms, or thresholds).");
			}

			if (!request.IsValidUpdateRequest())
			{
				return BadRequest($"UpdateRequest was not valid. Check name length (Max:{PerformanceBudgetRestrictions.MaxNameLength}), description length (Max:{PerformanceBudgetRestrictions.MaxDescriptionLength}), and thresholds count (Max:{PerformanceBudgetRestrictions.MaxThresholdsCount}).");
			}

			// Verify budget exists
			IPerformanceBudget? existingBudget = await _performanceBudgetCollection.GetPerformanceBudgetAsync(budgetId, cancellationToken);
			if (existingBudget == null)
			{
				return NotFound($"Could not find PerformanceBudget with Id: {budgetId}");
			}

			// Validate stream exists and user has access
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeStream(existingBudget.ComputedStream);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			// Verify the user is able to edit this budget
			UserId? requestingUser = User.GetUserId();
			bool canEdit = existingBudget.Owner == requestingUser || User.HasAdminClaim();

			if (!canEdit)
			{
				return StatusCode(StatusCodes.Status403Forbidden, $"User is not the owner of PerformanceBudget Id: {budgetId}.");
			}

			IPerformanceBudget? budget = await _performanceBudgetCollection.UpdatePerformanceBudgetAsync(budgetId, request, cancellationToken);

			if (budget == null)
			{
				return StatusCode(StatusCodes.Status500InternalServerError);
			}

			return await CreatePerformanceBudgetResponseWithOwnerAsync(budget, cancellationToken);
		}

		/// <summary>
		/// Gets a performance budget by ID.
		/// </summary>
		/// <param name="budgetId">The id of the budget to retrieve.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The budget if found.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/budgets/{budgetId}")]
		[ProducesResponseType(typeof(PerformanceBudgetResponse), 200)]
		public async Task<ActionResult<PerformanceBudgetResponse>?> GetPerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken = default)
		{
			IPerformanceBudget? budget = await _performanceBudgetCollection.GetPerformanceBudgetAsync(budgetId, cancellationToken);
			if (budget == null)
			{
				return NotFound(budgetId);
			}

			// Validate stream exists and user has access
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeStream(budget.ComputedStream);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			return await CreatePerformanceBudgetResponseWithOwnerAsync(budget, cancellationToken);
		}

		/// <summary>
		/// Gets performance budget groups matching the specified criteria.
		/// </summary>
		/// <param name="computedStream">The computed stream to filter budgets by (e.g., stream-main (Horde semantics), or "++Stream+Main" (branch semantics)).</param>
		/// <param name="testProject">Optional test project name filter.</param>
		/// <param name="platform">Optional platform filter. Returns budgets that include this platform or have no platform restriction.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>A list of matching budget groups.</returns>
		[HttpGet]
		[Route("/api/v1/performancetrends/budgets")]
		[ProducesResponseType(typeof(List<PerformanceBudgetResponse>), 200)]
		public async Task<ActionResult<List<PerformanceBudgetResponse>>> GetPerformanceBudgetsAsync([FromQuery] string computedStream, [FromQuery] string? testProject = null, [FromQuery] string? platform = null, CancellationToken cancellationToken = default)
		{
			// Validate stream exists and user has access
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeStream(computedStream);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			IEnumerable<IPerformanceBudget> budgets = await _performanceBudgetCollection.GetPerformanceBudgetsAsync(computedStream, testProject, platform, cancellationToken);
			List<PerformanceBudgetResponse> returnObjects = new List<PerformanceBudgetResponse>(budgets.Count());

			foreach (IPerformanceBudget budget in budgets)
			{
				PerformanceBudgetResponse response = await CreatePerformanceBudgetResponseWithOwnerAsync(budget, cancellationToken);
				returnObjects.Add(response);
			}

			return returnObjects;
		}

		/// <summary>
		/// Deletes a performance budget.
		/// </summary>
		/// <param name="budgetId">The id of the budget to delete.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>Ok if successful.</returns>
		/// <remarks>Users can only delete budgets that don't have an owner, ones they own, or any budget if they are an admin.</remarks>
		[HttpDelete]
		[Route("/api/v1/performancetrends/budgets/{budgetId}")]
		public async Task<ActionResult> DeletePerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken = default)
		{
			// Verify budget exists
			IPerformanceBudget? existingBudget = await _performanceBudgetCollection.GetPerformanceBudgetAsync(budgetId, cancellationToken);

			if (existingBudget == null)
			{
				return NotFound($"Could not find PerformanceBudget with Id: {budgetId}");
			}

			// Validate stream exists and user has access
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeStream(existingBudget.ComputedStream);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			// Verify user is able to delete this budget
			UserId? requestingUser = User.GetUserId();
			bool canDelete = existingBudget.Owner == requestingUser || User.HasAdminClaim();

			if (!canDelete)
			{
				return StatusCode(StatusCodes.Status403Forbidden, $"User is not the owner of PerformanceBudget Id: {budgetId}.");
			}

			if (!await _performanceBudgetCollection.DeletePerformanceBudgetAsync(budgetId, cancellationToken))
			{
				return BadRequest($"Unable to delete budget ({budgetId}).");
			}

			return Ok();
		}

		#endregion -- Budget CRUD Endpoints --

		#region -- Private Helpers --

		private async Task<PerformanceBudgetResponse> CreatePerformanceBudgetResponseWithOwnerAsync(IPerformanceBudget budget, CancellationToken cancellationToken)
		{
			if (budget.Owner != null)
			{
				GetThinUserInfoResponse? user = (await _userCollection.GetCachedUserAsync(budget.Owner.Value, cancellationToken))?.ToThinApiResponse();

				if (user != null)
				{
					return new PerformanceBudgetResponse(budget, user);
				}
			}

			return new PerformanceBudgetResponse(budget);
		}

		/// <summary>
		/// Helper method to verify and authorize stream.
		/// </summary>
		/// <param name="computedStream">The computed stream identifier to verify the user against (e.g., "++Stream+Main" (branch semantics) or "stream-main" (Horde semantics)).</param>
		/// <returns>Tuple representing whether validation was success, and the action result. Action result is Ok() on success.</returns>
		private (bool isValidationSuccessful, ActionResult actionResult) VerifyAndAuthorizeStream(string computedStream)
		{
			// Use the main authorization method (respects DisableStreamBasedAuth and HideExternalResults)
			if (IsAuthorizedForStream(computedStream))
			{
				return (isValidationSuccessful: true, actionResult: Ok());
			}

			// Authorization failed - determine appropriate error response
			// For non-Horde streams blocked by HideExternalResults, return Forbid
			if (IsNonHordeStream(computedStream))
			{
				return (isValidationSuccessful: false, actionResult: Forbid($"Access denied to external stream: {computedStream}"));
			}

			// For Horde streams, check if stream exists to differentiate NotFound vs Forbid
			StreamId streamId = new StreamId(computedStream);
			if (!_buildConfig.Value.TryGetStream(streamId, out _))
			{
				return (isValidationSuccessful: false, actionResult: NotFound($"Stream not found: {computedStream}"));
			}

			// Stream exists but user not authorized
			return (isValidationSuccessful: false, actionResult: Forbid(StreamAclAction.ViewStream, streamId));
		}

		/// <summary>
		/// Determines if the given stream identifier is a non-Horde stream format.
		/// </summary>
		/// <param name="stream">The stream identifier to check.</param>
		/// <returns>True if this is a non-Horde stream (starts with '++'), false otherwise.</returns>
		private static bool IsNonHordeStream(string stream)
		{
			// Perforce branch format (++Project+Stream) cannot be attributed to Horde ACLs
			return !String.IsNullOrEmpty(stream) && stream.StartsWith("++", StringComparison.Ordinal);
		}

		/// <summary>
		/// Filters the provided streams array to only include streams the user is authorized to access.
		/// </summary>
		/// <param name="streams">The array of stream identifiers to filter.</param>
		/// <returns>An array containing only the authorized streams. May be empty if no streams are authorized.</returns>
		private string[] FilterAuthorizedStreams(string[]? streams)
		{
			if (streams == null || streams.Length == 0)
			{
				return Array.Empty<string>();
			}

			List<string> authorized = new List<string>(streams.Length);

			foreach (string stream in streams)
			{
				if (String.IsNullOrEmpty(stream))
				{
					continue;
				}

				if (IsAuthorizedForStream(stream))
				{
					authorized.Add(stream);
				}
			}

			return authorized.ToArray();
		}

		/// <summary>
		/// Checks if the user is authorized to access a single stream.
		/// </summary>
		/// <param name="stream">The stream identifier to check authorization for.</param>
		/// <returns>True if authorized (or stream is null/empty/non-Horde), false if not authorized.</returns>
		/// <remarks>This is possibly overridden by StructuredAnalyticsConfig.DisableStreamBasedAuth.</remarks>
		private bool IsAuthorizedForStream(string? stream)
		{
			if (_performanceTrendsConfig.Value.DisableStreamBasedAuth)
			{
				return true;
			}

			// Null or empty stream - allow (no stream to restrict)
			if (String.IsNullOrEmpty(stream))
			{
				return false;
			}

			// Non-Horde streams (++prefix): allow through, can't attribute to ACL
			if (IsNonHordeStream(stream))
			{
				if (_performanceTrendsConfig.Value.HideExternalResults)
				{
					return false;
				}

				return true;
			}

			// Attempt Horde ACL validation
			StreamId streamId = new StreamId(stream);
			if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? config) && config.Authorize(StreamAclAction.ViewStream, User))
			{
				return true;
			}

			return false;
		}

		#endregion -- Private Helpers --
	}
}