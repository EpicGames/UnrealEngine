// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.PerformanceTrends;
using HordeServer.PerformanceTrends;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the PerformanceTrends plugin
	/// </summary>
	[Plugin("PerformanceTrends", EnabledByDefault = false, GlobalConfigType = typeof(PerformanceTrendsConfig), ServerConfigType = typeof(PerformanceTrendsServerConfig))]
	public class PerformanceTrendsPlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public PerformanceTrendsPlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			serviceCollection.AddSingleton<IPerformanceTrendsService, PerformanceTrendsService>();

			serviceCollection.AddSingleton<IPerformanceSummaryTrendHandler, KeyStatsPerformanceSummaryHandler>();

			serviceCollection.AddSingleton<IPerformanceBudgetCollection, PerformanceBudgetCollection>();

			serviceCollection.AddHttpClient();
		}
	}

	/// <summary>
	/// Helper methods for PerformanceTrends plugin config
	/// </summary>
	public static class PerformanceTrendsExtensions
	{
		/// <summary>
		/// Configures the PerformanceTrends plugin
		/// </summary>
		public static void AddPerformanceTrendsConfig(this IDictionary<PluginName, IPluginConfig> dictionary, PerformanceTrendsConfig config)
			=> dictionary[new PluginName("PerformanceTrends")] = config;

		/// <summary>
		/// Gets configuration for the PerformanceTrends plugin
		/// </summary>
		public static PerformanceTrendsConfig GetPerformanceTrendsConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (PerformanceTrendsConfig)dictionary[new PluginName("PerformanceTrends")];
	}
}
