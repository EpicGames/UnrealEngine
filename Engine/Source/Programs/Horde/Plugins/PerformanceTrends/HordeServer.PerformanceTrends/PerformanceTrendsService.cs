// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data.Common;
using EpicGames.Analytics.PerformanceTrends;
using HordeServer.Analytics;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.PerformanceTrends
{
	/// <summary>
	/// Horde Performance Trends service used for handling requests for performance trend data, and delegating them to the corresponding handlers.
	///
	/// This service owns the primary Api around access, and filtering.
	/// </summary>
	public class PerformanceTrendsService : AbstractAnalyticsService<PerformanceTrendsConfig>, IPerformanceTrendsService
	{
		private readonly Dictionary<string, IPerformanceSummaryTrendHandler> _handlersByType = new Dictionary<string, IPerformanceSummaryTrendHandler>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="config">Configuration object that instruments the service.</param>
		/// <param name="serviceProvider">DI service provider used to resolve the keyed analytics data source named by <c>BackendName</c>. The data source bundles the table-name formatter and SQL dialect.</param>
		/// <param name="logger">Logger.</param>
		/// <param name="handlers">The performance summary trend handlers.</param>
		public PerformanceTrendsService(IOptionsMonitor<PerformanceTrendsConfig> config, IServiceProvider serviceProvider, ILogger<PerformanceTrendsService> logger, IEnumerable<IPerformanceSummaryTrendHandler> handlers) : base(config, serviceProvider, logger)
		{
			TypeHandlers.RegisterTypeHandlers();

			foreach (IPerformanceSummaryTrendHandler handler in handlers)
			{
				if (!_handlersByType.ContainsKey(handler.PerformanceSummaryType))
				{
					_handlersByType.Add(handler.PerformanceSummaryType, handler);
				}
				else
				{
					Logger.LogError("Multiple performance handlers of the same summary type have been introduced: {PerformanceSummary}. Skipping registration, and enacting graceful degradation.", handler.PerformanceSummaryType);
				}
			}
		}

		#region -- IPerformanceTrendsService API --

		/// <inheritdoc/>
		public async Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryRecordsAsync(PerformanceTrendFilter filter, string type, CancellationToken cancellationToken)
		{
			IEnumerable<PerformanceTrendTelemetry> rows;

			if (Config.CurrentValue.MinQuerySchemaVersion != null)
			{
				filter.MinSchemaVersion = Config.CurrentValue.MinQuerySchemaVersion;
			}
			filter.Dialect = Dialect;

			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				if (!String.IsNullOrEmpty(type) && _handlersByType.TryGetValue(type, out IPerformanceSummaryTrendHandler? handler) && handler != null)
				{
					rows = await handler.ProcessGeneralMetricRequest(dbConnection, filter, Logger, cancellationToken);
				}
				else
				{
					Logger.LogWarning("Could not find a handler of the requested type: {PerformanceSummaryTrendHandlerType}", type);

					return [];
				}
			}

			return rows.ToList();
		}

		/// <inheritdoc/>
		public async Task<List<PerformanceTrendTelemetry>> GetPerformanceTrendTestProjectsAsync(bool excludeOrphanedSummaryTypes = true, CancellationToken cancellationToken = default)
		{
			List<PerformanceTrendTelemetry> returnList = new List<PerformanceTrendTelemetry>(16);

			PerformanceTrendFilter filter = new PerformanceTrendFilter();

			if (Config.CurrentValue.MinQuerySchemaVersion != null)
			{
				filter.MinSchemaVersion = Config.CurrentValue.MinQuerySchemaVersion;
			}
			filter.Dialect = Dialect;

			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				foreach (IPerformanceSummaryTrendHandler handler in _handlersByType.Values)
				{
					IEnumerable<PerformanceTrendTelemetry> rows = await handler.ProcessDistinctTestProjectRequestAsync(dbConnection, filter, Logger, cancellationToken);

					foreach (PerformanceTrendTelemetry row in rows)
					{
						if (!excludeOrphanedSummaryTypes || (!String.IsNullOrEmpty(row.SummaryName) && _handlersByType.ContainsKey(row.SummaryName)))
						{
							returnList.Add(row);
						}
					}
				}
			}

			return returnList;
		}

		/// <inheritdoc/>
		public async Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryPlatformsAsync(PerformanceTrendFilter filter, CancellationToken cancellationToken)
		{
			IEnumerable<PerformanceTrendTelemetry> rows;

			if (Config.CurrentValue.MinQuerySchemaVersion != null)
			{
				filter.MinSchemaVersion = Config.CurrentValue.MinQuerySchemaVersion;
			}
			filter.Dialect = Dialect;

			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				if (!String.IsNullOrEmpty(filter.SummaryName) && _handlersByType.TryGetValue(filter.SummaryName, out IPerformanceSummaryTrendHandler? handler) && handler != null)
				{
					rows = await handler.GetDistinctPlatformsAsync(dbConnection, filter, Logger, cancellationToken);
				}
				else
				{
					Logger.LogWarning("Could not find a handler of the requested type: {PerformanceSummaryTrendHandlerType}", filter.SummaryName);

					return [];
				}
			}

			return rows.ToList();
		}

		/// <inheritdoc/>
		public async Task<List<PerformanceTrendTelemetry>> GetPerformanceSummaryCommitsAsync(PerformanceTrendFilter filter, CancellationToken cancellationToken)
		{
			IEnumerable<PerformanceTrendTelemetry> rows;

			if (Config.CurrentValue.MinQuerySchemaVersion != null)
			{
				filter.MinSchemaVersion = Config.CurrentValue.MinQuerySchemaVersion;
			}
			filter.Dialect = Dialect;

			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				if (!String.IsNullOrEmpty(filter.SummaryName) && _handlersByType.TryGetValue(filter.SummaryName, out IPerformanceSummaryTrendHandler? handler) && handler != null)
				{
					rows = await handler.GetDistinctCommitsAsync(dbConnection, filter, Logger, cancellationToken);
				}
				else
				{
					Logger.LogWarning("Could not find a handler of the requested type: {PerformanceSummaryTrendHandlerType}", filter.SummaryName);

					return [];
				}
			}

			return rows.ToList();
		}

		/// <inheritdoc/>
		public List<string> GetPerformanceTrendTypes()
		{
			return _handlersByType.Keys.ToList();
		}

		#endregion -- IPerformanceTrendsService API --
	}
}
