// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.PerformanceTrends.Responses
{
	/// <summary>
	///  Models a response for a project's commit set.
	/// </summary>
	/// <remarks>
	///		A commit response references an <see cref="OwningTestProject"/>. This means that we have a series of commits that are applicable to the "prefix" modeled by the <see cref="TestProjectResponse"/>.
	/// </remarks>
	public class TestProjectCommitResponse
	{
		/// <summary>
		/// The owning test project.
		/// </summary>
		public TestProjectResponse OwningTestProject { get; init; }

		/// <summary>
		/// The commit ids associated with the <see cref="OwningTestProject"/>.
		/// </summary>
		public int[] CommitIds { get; init; }

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="summaryType">The summary type associated with this test project.</param>
		/// <param name="testName">The name of the test project.</param>
		/// <param name="testIdentity">The test identity of the performance trend telemetry.</param>
		/// <param name="testType">The test type of the performance trend telemetry.</param>
		/// <param name="stream">The stream in which this commit was present.</param>
		/// <param name="commitIds">The commit ids of the performance trend telemetry.</param>
		public TestProjectCommitResponse(string summaryType, string? testName, string? testIdentity, string? testType, string? stream, int[] commitIds)
		{
			OwningTestProject = new TestProjectResponse(summaryType, testName, testIdentity, testType, stream);
			CommitIds = commitIds;
		}
	}
}
