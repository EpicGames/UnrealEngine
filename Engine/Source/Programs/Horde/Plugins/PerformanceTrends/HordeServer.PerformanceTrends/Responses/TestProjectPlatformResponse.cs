// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.PerformanceTrends.Responses
{
	/// <summary>
	///  Models a response for a project's platform set.
	/// </summary>
	/// <remarks>
	///		A platform response references an <see cref="OwningTestProject"/>. This means that we have a series of platforms that are applicable to the "prefix" modeled by the <see cref="TestProjectResponse"/>.
	/// </remarks>
	public class TestProjectPlatformResponse
	{
		/// <summary>
		/// The owning test project.
		/// </summary>
		public TestProjectResponse OwningTestProject { get; init; }

		/// <summary>
		/// The platforms associated with the <see cref="OwningTestProject"/>.
		/// </summary>
		public string[] Platforms { get; init; }

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="summaryType">The summary type associated with this test project.</param>
		/// <param name="testName">The name of the test project.</param>
		/// <param name="testIdentity">The test identity of the performance trend telemetry.</param>
		/// <param name="testType">The test type of the performance trend telemetry.</param>
		/// <param name="stream">The stream of the performance trend telemetry.</param>
		/// <param name="platforms">The platforms of the performance trend telemetry.</param>
		public TestProjectPlatformResponse(string summaryType, string? testName, string? testIdentity, string? testType, string? stream, string[] platforms)
		{
			OwningTestProject = new TestProjectResponse(summaryType, testName, testIdentity, testType, stream);
			Platforms = platforms;
		}
	}
}
