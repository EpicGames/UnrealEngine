// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Analytics.PerformanceTrends;

namespace HordeServer.PerformanceTrends.Responses
{
	/// <summary>
	/// Models a response for a test project.
	/// </summary>
	/// <remarks>
	///		This should be considered a fundamental prefix.
	///		If <see cref="SummaryType"/> is the only thing set, it means that across all projects, we have at least one result of this summary type.
	///		If <see cref="TestName"/> is set, it means that results exist for the <see cref="SummaryType"/> and <see cref="TestName"/> combination.
	///		If <see cref="TestIdentity"/> is set, it means that results exist for the <see cref="SummaryType"/>, <see cref="TestName"/> and <see cref="TestIdentity"/> combination.
	///		If <see cref="TestType"/> is set, it means that results exist for the <see cref="SummaryType"/>, <see cref="TestName"/>, <see cref="TestIdentity"/>, and <see cref="TestType"/> combination.
	///		If <see cref="ComputedStream"/> is set, it means that results exist for the <see cref="SummaryType"/>, <see cref="TestName"/>, <see cref="TestIdentity"/>, <see cref="TestType"/>, and <see cref="ComputedStream"/> combination.
	///		
	///		Fundamentally this response object is used as a semantic reasoning object, and input into subsequent inquiries into the data set.
	///	</remarks>
	public class TestProjectResponse
	{
		/// <summary>
		/// The summary type this test project belongs to.
		/// </summary>
		public string SummaryType { get; init; }

		/// <summary>
		/// The name of the test project.
		/// </summary>
		public string? TestName { get; init; }

		/// <summary>
		/// The test identity of the performance trend telemetry.
		/// </summary>
		public string? TestIdentity { get; init; }

		/// <summary>
		/// The test type of the performance trend telemetry.
		/// </summary>
		/// <remarks>Historically, this has been called GauntletSubTest.</remarks>
		public string? TestType { get; init; }

		/// <summary>
		/// The computed stream of the performance trend telemetry.
		/// </summary>
		/// <remarks>
		/// This represents a coalesced representation of the stream using the reported Stream from Horde, and the Branch of the build.
		/// </remarks>
		public string? ComputedStream { get; init; }

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="summaryType">The summary type associated with this test project.</param>
		/// <param name="testName">The name of the test project.</param>
		/// <param name="testIdentity">The test identity of the performance trend telemetry.</param>
		/// <param name="testType">The test type of the performance trend telemetry.</param>
		/// <param name="computedStream">The stream of the performance trend telemetry.</param>
		public TestProjectResponse(string summaryType, string? testName, string? testIdentity, string? testType, string? computedStream)
		{
			SummaryType = summaryType;
			TestName = testName;
			TestIdentity = testIdentity;
			TestType = testType;
			ComputedStream = computedStream;
		}

		/// <summary>
		/// Creates a Test Project Response from a base performance trend telemetry event.
		/// </summary>
		/// <param name="telemetry">The telemetry event to base the Test Project Response on.</param>
		/// <returns>The test project response.</returns>
		public static TestProjectResponse CreateTestProjectResponse(PerformanceTrendTelemetry telemetry)
		{
			TestProjectResponse newTestProject = new TestProjectResponse(telemetry.SummaryName, telemetry.TestName, telemetry.TestIdentity, telemetry.GauntletSubTest, telemetry.ComputedStream);

			return newTestProject;
		}
	}
}
