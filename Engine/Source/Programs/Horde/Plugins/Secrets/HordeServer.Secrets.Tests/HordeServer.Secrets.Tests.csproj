<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer.Tests</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.SSO" Version="3.7.501.12" />
    <PackageReference Include="AWSSDK.SSOOIDC" Version="3.7.502" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\HordeServer.Secrets\HordeServer.Secrets.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Tests.Shared\HordeServer.Tests.Shared.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>

</Project>
