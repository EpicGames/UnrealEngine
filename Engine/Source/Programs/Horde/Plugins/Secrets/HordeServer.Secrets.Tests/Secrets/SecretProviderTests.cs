// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Secrets;
using HordeServer.Secrets;
using HordeServer.Secrets.Providers;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Secrets
{
	[TestClass]
	public class SecretProviderTests
	{
		private class OptionsSnapshotStub<T>(T value) : IOptionsSnapshot<T> where T : class
		{
			public T Value { get; } = value;
			public T Get(string? name)
			{
				return Value;
			}
		}

		private class InMemoryProvider : ISecretProvider
		{
			private Dictionary<string, string> Secrets { get; } = new()
			{
				{ "secret/username", "user01" },
				{ "secret/password", "password01" }
			};
			public string Name { get; } = "InMemory";

			public Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
			{
				return Task.FromResult(Secrets[path]);
			}
		}

		private class StubHttpClientFactory : IHttpClientFactory
		{
			public HttpClient CreateClient(string name)
			{
				return new HttpClient();
			}
		}
		private static SecretCollection CreateSecretCollection()
		{
			SecretCollectionInternal secretCollectionInternal = new([
				new InMemoryProvider(),
				new HcpVaultSecretProvider(new StubHttpClientFactory(), NullLogger<HcpVaultSecretProvider>.Instance),
				new AzureKeyVaultSecretProvider()
			]);
			string json = """
				{
					"secrets": [
						{
							"id": "secret",
							"data": {
								"username": "j.doe",
								"password": "password123"
							}
						},
						{
							"id": "secretUsingProviderConfig",
							"sources": [
								{
									"providerConfig": "in-memory",
									"path": "secret/username",
									"key": "username"
								},
								{
									"providerConfig": "in-memory",
									"path": "secret/password",
									"key": "password"
								}
							]
						},
						{
							"id": "secretUsingHcpVault",
							"sources": [
								{
									"providerConfig": "vault-pre-shared-key",
									"path": "/v1/secret/data/example",
									"format": "json"
								}
							]
						},
						{
							"id": "missingSecretUsingHcpVault",
							"sources": [
								{
									"providerConfig": "vault-pre-shared-key",
									"path": "/v1/secret/data/notfound",
									"format": "json"
								}
							]
						},
						{
							"id": "multipleSecretUsingHcpVault",
							"sources": [
								{
									"providerConfig": "vault-pre-shared-key",
									"path": "/v1/secret/data/first",
									"format": "json"
								},
								{
									"providerConfig": "vault-pre-shared-key",
									"path": "/v1/secret/data/second",
									"format": "json"
								},
								{
									"providerConfig": "vault-pre-shared-key",
									"path": "/v1/secret/data/third",
									"format": "json"
								}
							]
						},
						{
							"id": "secretUsingHcpVaultAwsAuth",
							"sources": [
								{
									"providerConfig": "vault-aws-assume-role",
									"path": "/v1/secret/data/example",
									"format": "json"
								}
							]
						},
						{
							"id": "secretUsingAzureKeyVault",
							"sources": [
								{
									"providerConfig": "azure-key-vault",
									"path": "test",
									"key": "secret"
								}
							]
						},
						{
							"id": "emptySecret",
							"sources": [
							]
						}
					],
					"providerConfigs": [
						{
							"name": "in-memory",
							"provider": "InMemory"
						},
						{
							"name": "vault-pre-shared-key",
							"provider": "HcpVault",
							// To set up a local HCP Vault dev server with test data:
							// 1. install the vault executable https://developer.hashicorp.com/vault/install
							// 2. in a shell run: vault server -dev
							//    make a note of the token value called 'Root Token' printed to stdout 
							// 3. in a second shell set the environment variable VAULT_ADDR
							//    the addr will likely be http://127.0.0.1:8200
							// 4. in the second shell login in with the token: vault login
							// 5. in the second shell add the data:
							//    vault kv put secret/example username="User01" password="Password123"
							//    vault kv put secret/first one="1"
							//    vault kv put secret/second two="2"
							//    vault kv put secret/third three="3"
							// 6. update the PreSharedKey property below 
							"hcpVault": {
								"credentials": "PreSharedKey",
								"endpoint": "http://127.0.0.1:8200",
								"preSharedKey": "replace-with-token"
							}
						},
						{
							"name": "vault-pre-shared-key-provider",
							"provider": "HcpVault",
							"hcpVault": {
								"credentials": "PreSharedKey",
								"endpoint": "https://vault.example.com",
								"preSharedKey": "Abc123Def"
							}
						},
						{
							"name": "vault-aws-assume-role",
							"provider": "HcpVault",
							"hcpVault": {
								"Credentials": "AwsAuth",
								"endpoint": "https://vault.example.com/",
								"awsIamServerId": "vault.example.com",
								"awsArnRole": "arn:aws:iam::1234567890:role/vault-example-role",
								"role": "vault-example"
							}
						},
						{
							"name": "azure-key-vault",
							"provider": "AzureKeyVault",
							"azureKeyVault": {
								"vaultUri": "https://example.vault.azure.net/"
							}
						}
					]
				}
			""";
			SecretsConfig secretsConfig = JsonSerializer.Deserialize<SecretsConfig>(json, JsonUtils.DefaultSerializerOptions)!;
			SecretCollection secretCollection = new(secretCollectionInternal, new OptionsSnapshotStub<SecretsConfig>(secretsConfig));
			GlobalConfig config = new();
			config.Plugins.AddSecretsConfig(secretsConfig);
			config.PostLoad(new ServerSettings(), [], []);
			return secretCollection;
		}

		[TestMethod]
		public async Task GetSecretAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("Secret"), CancellationToken.None);
			Assert.AreEqual("j.doe", secret?.Data["username"]);
			Assert.AreEqual("password123", secret?.Data["password"]);
		}

		[TestMethod]
		public async Task GetSecretFromProviderConfigAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("SecretUsingProviderConfig"), CancellationToken.None);
			Assert.AreEqual("user01", secret?.Data["username"]);
			Assert.AreEqual("password01", secret?.Data["password"]);
		}

		[TestMethod]
		public async Task GetEmptySecretAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection();
			ISecret? secret = await secretCollection.GetAsync(new SecretId("EmptySecret"), CancellationToken.None);
			Assert.AreEqual(0, secret?.Data.Count);
		}

		[Ignore("Requires access to a running HCP Vault server and a pre-shared key")]
		[TestMethod]
		public async Task GetSecretFromVaultAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("SecretUsingHcpVault"), CancellationToken.None);
			Assert.AreEqual("User01", secret?.Data["username"]);
			Assert.AreEqual("Password123", secret?.Data["password"]);
		}

		[Ignore("Requires access to a running HCP Vault server and a pre-shared key")]
		[TestMethod]
		public async Task GetMultipleSecretFromVaultAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("MultipleSecretUsingHcpVault"), CancellationToken.None);
			Assert.AreEqual("1", secret?.Data["one"]);
			Assert.AreEqual("2", secret?.Data["two"]);
			Assert.AreEqual("3", secret?.Data["three"]);
		}

		[Ignore("Requires access to a running HCP Vault server and a pre-shared key")]
		[TestMethod]
		public async Task SecretNotFoundFromVaultAsync()
		{
			SecretCollection secretCollection = CreateSecretCollection(); 
			InvalidOperationException ex = await Assert.ThrowsExactlyAsync<InvalidOperationException>(() =>
				secretCollection.GetAsync(new SecretId("MissingSecretUsingHcpVault"), CancellationToken.None)
			);
			Assert.AreEqual("Unable to fetch secret '/v1/secret/data/notfound' from HCP Vault Status = NotFound Body={\"errors\":[]}\n", ex.Message);
		}

		[Ignore("Requires access to a running HCP Vault server and supported AWS IAM")]
		[TestMethod]
		public async Task GetSecretFromVaultWithAwsAuthAsync()
		{
			// To run this test from your workstation
			// 1. log into AWS with: aws sso login
			// 2. set the environment variable AWS_PROFILE to a profile that can call AssumeRole for the required ARN
			// 3. restart IDE to pick up the environment change
			// 4. update the SecretProviderConfig options for the SecretId used in this test 
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("SecretUsingHcpVaultAwsAuth"), CancellationToken.None);
			Assert.AreEqual("test", secret?.Data["test"]);
		}

		[Ignore("Requires access to a Key Vault in Azure and an authorized Entra ID")]
		[TestMethod]
		public async Task GetSecretFromAzureKeyVaultAsync()
		{
			// To run this test from Visual Studio
			// 1. add your Azure Entra account to Visual Studio
			// 2. select that account in Azure Authentication settings
			// 3. update Vault URI in the SecretProviderConfig options for the SecretId used in this test 
			// 4. grant permissions for the Entra account to access the key vault and secret
			SecretCollection secretCollection = CreateSecretCollection(); 
			ISecret? secret = await secretCollection.GetAsync(new SecretId("SecretUsingAzureKeyVault"), CancellationToken.None);
			Assert.AreEqual("test", secret?.Data["secret"]);
		}

		[TestMethod]
		public async Task GetSecretFromMinimalJsonAsync()
		{
			string json = """
				{
					"secrets": [
						{
							"id": "basic-secret",
							"data": {
								"username": "j.doe",
								"password": "password123"
							}
						}
					]
				}
			""";
			SecretsConfig secretsConfig = JsonSerializer.Deserialize<SecretsConfig>(json, JsonUtils.DefaultSerializerOptions)!;
			SecretCollectionInternal secretCollectionInternal = new([]);
			SecretCollection secretCollection = new(secretCollectionInternal, new OptionsSnapshotStub<SecretsConfig>(secretsConfig));
			GlobalConfig config = new();
			config.Plugins.AddSecretsConfig(secretsConfig);
			config.PostLoad(new ServerSettings(), [], []);
			ISecret? secret = await secretCollection.GetAsync(new SecretId("basic-secret"), CancellationToken.None);
			Assert.AreEqual("j.doe", secret?.Data["username"]);
			Assert.AreEqual("password123", secret?.Data["password"]);
		}
	}
}
