// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Secrets;
using HordeServer.Secrets;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Secrets
{
	[TestClass]
	public class SecretResolveTests
	{
		private static ISecretCollection s_secretCollection = default!;

		private class OptionsSnapshotStub<T>(T value) : IOptionsSnapshot<T> where T : class
		{
			public T Value { get; } = value;
			public T Get(string? name)
			{
				return Value;
			}
		}

		[ClassInitialize]
		public static void CreateSecretCollection(TestContext _)
		{
			string json = """
				{
					"secrets": [
						{ "id": "mySecret", "data": { "username": "j.doe", "password": "password123" } },
						{ "id": "misconfiguredSecret", "sources": [ { "provider": "secretProviderNotFound" } ] }
					]
				}
			""";
			SecretsConfig secretsConfig = JsonSerializer.Deserialize<SecretsConfig>(json, JsonUtils.DefaultSerializerOptions)!;
			SecretCollection secretCollection = new(new SecretCollectionInternal([]), new OptionsSnapshotStub<SecretsConfig>(secretsConfig));
			GlobalConfig config = new();
			config.Plugins.AddSecretsConfig(secretsConfig);
			config.PostLoad(new ServerSettings(), [], []);
			s_secretCollection = secretCollection;
		}

		[TestMethod]
		[DataRow("horde:secret:mySecret.username", "username", "j.doe", DisplayName = "Get Username")]
		[DataRow("horde:secret:mySecret.password", "password", "password123", DisplayName = "Get Password")]
		public async Task ResolveSecretAsync(string value, string expectedName, string expectedValue)
		{
			ISecretProperty? secret = await s_secretCollection.ResolveAsync(value);
			Assert.IsNotNull(secret);
			Assert.AreEqual(expectedName, secret.Name);
			Assert.AreEqual(expectedValue, secret.Value);
		}

		[TestMethod]
		[DataRow("", DisplayName = "Empty string")]
		[DataRow(null, DisplayName = "Null string")]
		[DataRow("invalid string", DisplayName = "Incorrect format")]
		public async Task ResolveInvalidSecretAsync(string value)
		{
			ISecretProperty? secret = await s_secretCollection.ResolveAsync(value);
			Assert.IsNull(secret);
		}

		[TestMethod]
		[DataRow("horde:secret:doesNotExist.password", DisplayName = "Secret does not exist")]
		[DataRow("horde:secret:mySecret.doesNotExist", DisplayName = "Secret property does not exist")]
		public async Task ResolveSecretNotFoundAsync(string value)
		{
			await Assert.ThrowsExactlyAsync<KeyNotFoundException>(async () => _ = await s_secretCollection.ResolveAsync(value));
		}

		[TestMethod]
		public async Task ResolveSecretWithMisconfiguredConfigAsync()
		{
			await Assert.ThrowsExactlyAsync<KeyNotFoundException>(async () => _ = await s_secretCollection.ResolveAsync("horde:secret:misconfiguredSecret.property"));
		}
	}
}
