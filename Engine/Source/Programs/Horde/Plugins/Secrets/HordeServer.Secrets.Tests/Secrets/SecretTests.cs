// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Secrets;
using HordeServer.Plugins;
using HordeServer.Secrets;
using HordeServer.Server;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Secrets
{
	[TestClass]
	public class SecretTests
	{
		private static ISecretCollection s_secretCollection = null!;
		private static GlobalConfig s_globalConfig = null!;

		private class OptionsSnapshotStub<T>(T value) : IOptionsSnapshot<T> where T : class
		{
			public T Value { get; } = value;
			public T Get(string? name)
			{
				return Value;
			}
		}

		[ClassInitialize]
		public static void CreateSecretCollection(TestContext _)
		{
			SecretsConfig secretsConfig = new()
			{
				Secrets = [new SecretConfig
				{
					Id = new SecretId("User"),
					Data = new Dictionary<string, string>
					{
						{ "username", "j.doe" },
						{ "password", "password123" }
					}
				}]
			};
			SecretCollection secretCollection = new(new SecretCollectionInternal([]), new OptionsSnapshotStub<SecretsConfig>(secretsConfig));
			GlobalConfig config = new();
			config.Plugins.AddSecretsConfig(secretsConfig);
			config.PostLoad(new ServerSettings(), [], []);
			s_globalConfig = config;
			s_secretCollection = secretCollection;
		}

		[TestMethod]
		[DataRow("username", "j.doe", DisplayName = "Get Username")]
		[DataRow("password", "password123", DisplayName = "Get Password")]
		public async Task GetSecretWithConfigAsync(string expectedName, string expectedValue)
		{
			ISecret? secret = await s_secretCollection.GetAsync(new SecretId("User"), s_globalConfig.Plugins[new PluginName("Secrets")]);
			Assert.IsNotNull(secret);
			Assert.AreEqual(expectedValue, secret.Data[expectedName]);
		}

		[TestMethod]
		public async Task GetSecretWithInvalidConfigAsync()
		{
			await Assert.ThrowsExactlyAsync<InvalidCastException>(async () => _ = await s_secretCollection.GetAsync(new SecretId(), new BuildConfig()));
		}
	}
}
