<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <FrameworkReference Include="Microsoft.AspNetCore.App" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.SimpleSystemsManagement" Version="3.7.304.24" />
    <PackageReference Include="Azure.Identity" Version="1.14.2" />
    <PackageReference Include="Azure.Security.KeyVault.Secrets" Version="4.8.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
    <ProjectReference Include="..\..\Storage\HordeServer.Storage\HordeServer.Storage.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
  </ItemGroup>

</Project>
