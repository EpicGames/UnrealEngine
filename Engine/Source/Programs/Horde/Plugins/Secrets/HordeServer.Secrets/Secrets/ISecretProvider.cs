// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Secrets
{
	/// <summary>
	/// Interface for a service that can provide secret data
	/// </summary>
	public interface ISecretProvider
	{
		/// <summary>
		/// Name of this provider
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Gets values for a particular secret
		/// </summary>
		/// <param name="path">Path to the secret to fetch</param>
		/// <param name="config">Configuration for the secret provider</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken);
	}
}
