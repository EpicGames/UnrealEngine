// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.SimpleSystemsManagement;
using Amazon.SimpleSystemsManagement.Model;

namespace HordeServer.Secrets.Providers
{
	/// <summary>
	/// Fetches secrets from the AWS parameter store
	/// </summary>
	public class AwsParameterStoreSecretProvider : ISecretProvider
	{
		/// <inheritdoc/>
		public string Name => "AwsParameterStore";

		readonly IAmazonSimpleSystemsManagement _systemsManagement;

		/// <summary>
		/// Constructor
		/// </summary>
		public AwsParameterStoreSecretProvider(IAmazonSimpleSystemsManagement systemsManagement)
		{
			_systemsManagement = systemsManagement;
		}

		/// <inheritdoc/>
		public async Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
		{
			GetParameterResponse response = await _systemsManagement.GetParameterAsync(new GetParameterRequest { Name = path, WithDecryption = true }, cancellationToken);
			if (response.Parameter == null)
			{
				throw new InvalidOperationException($"Unable to fetch secret '{path}' from AWS parameter store ({response.HttpStatusCode})");
			}
			return response.Parameter.Value;
		}
	}
}
