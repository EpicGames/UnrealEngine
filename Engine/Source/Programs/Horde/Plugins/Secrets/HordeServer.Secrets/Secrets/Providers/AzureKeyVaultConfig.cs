// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Secrets.Providers;

/// <summary>
/// Configuration for Azure Key Vault secrets.
/// </summary>
public class AzureKeyVaultConfig
{
	/// <summary>
	/// The Uri to the Azure Key Vault.
	/// </summary>
	public string? VaultUri { get; set; }
}