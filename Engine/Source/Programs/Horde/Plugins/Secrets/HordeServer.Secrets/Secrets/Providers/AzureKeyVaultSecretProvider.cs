// Copyright Epic Games, Inc. All Rights Reserved.

using Azure;
using Azure.Identity;
using Azure.Security.KeyVault.Secrets;

namespace HordeServer.Secrets.Providers;

/// <summary>
/// Fetches secrets from Azure Key Vault.
/// </summary>
public class AzureKeyVaultSecretProvider : ISecretProvider
{
	/// <inheritdoc/>
	public string Name => "AzureKeyVault";

	/// <inheritdoc/>
	public async Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
	{
		if (config?.AzureKeyVault == null)
		{
			throw new InvalidOperationException($"Unable to fetch secret {path} from Azure Key Vault (No Options)");
		}
		if (config.AzureKeyVault.VaultUri == null)
		{
			throw new InvalidOperationException($"Unable to fetch secret {path} from Azure Key Vault (No Vault URI)");
		}
		DefaultAzureCredential credential = new();
		SecretClient secretClient = new(new Uri(config.AzureKeyVault.VaultUri), credential);
		Response<KeyVaultSecret>? secret = await secretClient.GetSecretAsync(path, cancellationToken: cancellationToken);
		return secret.Value.Value;
	}
}