// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Secrets.Providers
{
	/// <summary>
	/// Credentials to use to log into HCP Vault.
	/// </summary>
	public enum HcpVaultCredentialsType
	{
		/// <summary>
		/// Use a pre-shared token provided by Vault for authentication.
		/// </summary>
		PreSharedKey,

		/// <summary>
		/// Use AWS IAM credentials to authenticate against the Vault server.
		/// </summary>
		AwsAuth
	}

	/// <summary>
	/// Configuration for HCP Vault secrets.
	/// </summary>
	public class HcpVaultConfig
	{
		/// <summary>
		/// The Vault authentication token to use when the type of credentials is <see cref="HcpVaultCredentialsType.PreSharedKey"/>.
		/// <remarks>
		/// More information on this token can be found in the API documentation for <see href="https://developer.hashicorp.com/vault/api-docs#authentication">X-Vault-Token</see>
		/// </remarks>
		/// </summary>
		public string? PreSharedKey { get; set; }

		/// <summary>
		/// The address of the Vault server.
		/// </summary>
		public string? EndPoint { get; set; }

		/// <summary>
		/// The optional AWS ARN to assume a role when logging into Vault.
		/// To be used when the type of credentials is <see cref="HcpVaultCredentialsType.AwsAuth"/>.
		/// <remarks>When not set the AWS SDK default credential search will be used.</remarks>
		/// </summary>
		public string? AwsArnRole { get; set; }

		/// <summary>
		/// The value for the X-Vault-AWS-IAM-Server-ID header when the Vault server is configured with
		/// <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#iam_server_id_header_value">iam_server_id_header_value</see>.
		/// Most likely to be the Vault server's DNS name.
		/// To be used when the type of credentials is <see cref="HcpVaultCredentialsType.AwsAuth"/>.
		/// </summary>
		public string? AwsIamServerId { get; set; }

		/// <summary>
		/// The name of the <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#role-4">role</see> in the Vault configuration
		/// for the login. This role name can be different to the name of the AWS IAM principal.
		/// To be used when the type of credentials is <see cref="HcpVaultCredentialsType.AwsAuth"/>.
		/// <remarks>When not set the name of IAM principal will be used.</remarks>
		/// </summary>
		public string? Role { get; set; }

		/// <summary>
		/// Type of credentials to use.
		/// </summary>
		public HcpVaultCredentialsType Credentials { get; set; }
	}
}
