// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Http.Json;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using Amazon.Runtime;
using Amazon.Runtime.Internal;
using Amazon.Runtime.Internal.Auth;
using Amazon.Runtime.Internal.Util;
using Amazon.SecurityToken;
using Amazon.SecurityToken.Model;
using Amazon.SecurityToken.Model.Internal.MarshallTransformations;
using Microsoft.Extensions.Logging;
using ILogger = Microsoft.Extensions.Logging.ILogger;

namespace HordeServer.Secrets.Providers
{
	/// <summary>
	/// Fetches secrets from a HashiCorp (HCP) Vault.
	/// </summary>
	public class HcpVaultSecretProvider : ISecretProvider
	{
		/// <summary>
		/// The login parameters to Vault when using AWS for authentication.
		/// <remarks>
		/// The parameters for the <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#login">v1/auth/aws/login</see> request.
		/// </remarks>
		/// </summary>
		private class AwsAuthRequest
		{
			/// <summary>
			/// The <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#iam_http_request_method">HTTP method</see>
			/// used in the signed request.
			/// </summary>
			[JsonPropertyName("iam_http_request_method")]
			public string Method { get; set; } = "POST";

			/// <summary>
			/// Base64-encoded <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#iam_request_body">body</see> of the signed request.
			/// </summary>
			[JsonPropertyName("iam_request_body")]
			public string Body { get; set; } = Convert.ToBase64String("Action=GetCallerIdentity&Version=2011-06-15"u8.ToArray());

			/// <summary>
			/// Based64-encoded key/value pairs of <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#iam_request_headers">headers</see>
			/// for the sts:GetCallerIdentity request.
			/// </summary>
			[JsonPropertyName("iam_request_headers")]
			public required string Headers { get; set; }

			/// <summary>
			/// Base64-encoded of the <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#iam_request_url">HTTP URL</see> used to sign
			/// the request.
			/// </summary>
			[JsonPropertyName("iam_request_url")]
			public string Url { get; set; } = Convert.ToBase64String("https://sts.amazonaws.com/"u8.ToArray());

			/// <summary>
			/// Name of the <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#role-4">role</see> the login is attempted against.
			/// This is not the same as the AWS ARN Role.
			/// </summary>
			[JsonPropertyName("role")]
			public string Role { get; set; } = String.Empty;
		}

		/// <summary>
		/// The <see href="https://developer.hashicorp.com/vault/api-docs/auth/aws#sample-response-12">response</see> from the login request.
		/// </summary>
		private class AuthResponse
		{
			[JsonPropertyName("auth")]
			public AuthInfo? AuthInfo { get; set; }
		}

		/// <summary>
		/// The data from a successful login response.
		/// </summary>
		private class AuthInfo
		{
			/// <summary>
			/// The token returned from a login request and to be used as the value for the X-Vault-Token header for following API calls.
			/// </summary>
			[JsonPropertyName("client_token")]
			public string? Token { get; set; }
		}

		/// <summary>
		/// The response from requesting a secret.
		/// </summary>
		private class SecretResponse
		{
			/// <summary>
			/// The key/value paris of all items for a given path to a secret.
			/// </summary>
			public Dictionary<string, Dictionary<string, object>> Data { get; set; } = new();
		}

		/// <inheritdoc/>
		public string Name => "HcpVault";

		private readonly IHttpClientFactory _httpClientFactory;
		private readonly ILogger _logger;
		private const string HttpClientName = "HordeSecretProviderHcpVault";
		private const string AwsAssumeRoleSessionName = "HordeSecretProviderVaultSession";

		/// <summary>
		/// Constructor
		/// </summary>
		public HcpVaultSecretProvider(IHttpClientFactory httpClientFactory, ILogger<HcpVaultSecretProvider> logger)
		{
			_httpClientFactory = httpClientFactory;
			_logger = logger;
		}

		private HttpClient GetHttpClient()
		{
			return _httpClientFactory.CreateClient(HttpClientName);
		}

		private static async Task<string> GetAwsCallerArnAsync(AWSCredentials credentials, CancellationToken cancellationToken)
		{
			using AmazonSecurityTokenServiceClient client = new(credentials);
			GetCallerIdentityRequest request = new();
			GetCallerIdentityResponse? response = await client.GetCallerIdentityAsync(request, cancellationToken);
			return response.Arn;
		}

		private async Task<ImmutableCredentials> GetAwsCredentialsAsync(string? role, CancellationToken cancellationToken)
		{
			ImmutableCredentials credentials;
			if (!String.IsNullOrEmpty(role))
			{
				string originalCaller = await GetAwsCallerArnAsync(FallbackCredentialsFactory.GetCredentials(), cancellationToken);
				using AmazonSecurityTokenServiceClient client = new();
				AssumeRoleRequest assumeRoleRequest = new() { RoleArn = role, RoleSessionName = AwsAssumeRoleSessionName };
				AssumeRoleResponse? assumeRole = await client.AssumeRoleAsync(assumeRoleRequest, cancellationToken);
				string newCaller = await GetAwsCallerArnAsync(assumeRole.Credentials, cancellationToken);
				_logger.LogDebug("Caller: {OriginalCaller} to AssumeRole: {NewCaller}", originalCaller, newCaller);
				credentials = await assumeRole.Credentials.GetCredentialsAsync();
			}
			else
			{
				AWSCredentials awsCredentials = FallbackCredentialsFactory.GetCredentials();
				string originalCaller = await GetAwsCallerArnAsync(awsCredentials, cancellationToken);
				credentials = await awsCredentials.GetCredentialsAsync();
				_logger.LogDebug("Caller: {OriginalCaller} to AccessKey: {AccessKey}", originalCaller, credentials.AccessKey);
			}
			return credentials;
		}
	
		private async Task<string> GetAwsSignedHeadersAsync(string? serverId, string? arnRole, CancellationToken cancellationToken)
		{
			ImmutableCredentials credentials = await GetAwsCredentialsAsync(arnRole, cancellationToken);
			IRequest? request = GetCallerIdentityRequestMarshaller.Instance.Marshall(new GetCallerIdentityRequest());
			request.ResourcePath = "/";
			request.Headers.Add("Content-Type", "application/x-www-form-urlencoded; charset=utf-8");
			if (serverId != null)
			{
				request.Headers.Add("X-Vault-AWS-IAM-Server-ID", serverId);
			}
			if (credentials.UseToken)
			{
				request.Headers.Add("X-Amz-Security-Token", credentials.Token);
			}
			AmazonSecurityTokenServiceConfig securityTokenServiceConfig = new();
#pragma warning disable CS0618 // Type or member is obsolete
			request.Endpoint = new Uri(securityTokenServiceConfig.DetermineServiceURL());
#pragma warning restore CS0618 // Type or member is obsolete
			new AWS4Signer().Sign(request, securityTokenServiceConfig, new RequestMetrics(), credentials.AccessKey, credentials.SecretKey);
			return JsonSerializer.Serialize(request.Headers);
		}

		private async Task<string> GetTokenFromAwsAsync(HcpVaultConfig config, CancellationToken cancellationToken)
		{
			string signedHeaders = await GetAwsSignedHeadersAsync(config.AwsIamServerId, config.AwsArnRole, cancellationToken);
			AwsAuthRequest payload = new()
			{
				Headers = Convert.ToBase64String(Encoding.UTF8.GetBytes(signedHeaders)),
				Role = config.Role ?? String.Empty
			};
			// https://developer.hashicorp.com/vault/api-docs/auth/aws#login 
			Uri requestUri = new(new Uri(config.EndPoint!), "v1/auth/aws/login");
			HttpClient httpClient = GetHttpClient();
			using HttpResponseMessage httpResponseMessage = await httpClient.PostAsJsonAsync(requestUri, payload, cancellationToken);
			if (httpResponseMessage.StatusCode != HttpStatusCode.OK)
			{
				string errorMessage = $"Status = {httpResponseMessage.StatusCode} Body={await httpResponseMessage.Content.ReadAsStringAsync(cancellationToken)}";
				throw new InvalidOperationException($"Unable to log into HCP Vault using AWS {errorMessage}");
			}
			AuthResponse? response = await httpResponseMessage.Content.ReadFromJsonAsync<AuthResponse>(cancellationToken);
			return response?.AuthInfo?.Token ?? throw new InvalidOperationException($"Unable to log into HCP Vault using AWS (No Token Content)");
		}

		private async Task<string> GetSecretAsync(Uri path, string token, CancellationToken cancellationToken)
		{
			// https://developer.hashicorp.com/vault/api-docs 
			using HttpRequestMessage request = new(HttpMethod.Get, path);
			HttpClient httpClient = GetHttpClient();
			httpClient.DefaultRequestHeaders.Add("X-Vault-Token", token);
			using HttpResponseMessage httpResponseMessage = await httpClient.SendAsync(request, cancellationToken);
			if (httpResponseMessage.StatusCode != HttpStatusCode.OK)
			{
				string errorMessage = $"Status = {httpResponseMessage.StatusCode} Body={await httpResponseMessage.Content.ReadAsStringAsync(cancellationToken)}";
				throw new InvalidOperationException($"Unable to fetch secret '{path.AbsolutePath}' from HCP Vault {errorMessage}");
			}
			SecretResponse? response = await httpResponseMessage.Content.ReadFromJsonAsync<SecretResponse>(cancellationToken);
			if (response != null)
			{
				if (response.Data.TryGetValue("data", out Dictionary<string, object>? value))
				{
					return JsonSerializer.Serialize(value);
				}
			}
			throw new InvalidOperationException($"Unable to fetch secret '{path.AbsolutePath}' from HCP Vault (No Content)");
		}

		/// <inheritdoc/>
		public async Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
		{
			if (config?.HcpVault == null)
			{
				throw new InvalidOperationException($"Unable to fetch secret {path} from HCP Vault (No Options)");
			}
			if (config.HcpVault.EndPoint == null)
			{
				throw new InvalidOperationException($"Unable to fetch secret {path} from HCP Vault (No HcpVaultEndPoint)");
			}
			string token;
			if (config.HcpVault.Credentials == HcpVaultCredentialsType.PreSharedKey)
			{
				token = config.HcpVault.PreSharedKey ?? throw new InvalidOperationException($"Unable to fetch secret {path} form HCP Vault (No Vault Token)");
			}
			else if (config.HcpVault.Credentials == HcpVaultCredentialsType.AwsAuth)
			{
				token = await GetTokenFromAwsAsync(config.HcpVault, cancellationToken);
			}
			else
			{
				throw new NotImplementedException();
			}
			return await GetSecretAsync(new Uri(new Uri(config.HcpVault.EndPoint), path), token, cancellationToken);
		}
	}
}
