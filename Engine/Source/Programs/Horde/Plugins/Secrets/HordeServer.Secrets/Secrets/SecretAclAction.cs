// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Secrets
{
	/// <summary>
	/// ACL actions for secrets
	/// </summary>
	public static class SecretAclAction
	{
		/// <summary>
		/// View a credential
		/// </summary>
		public static AclAction ViewSecret { get; } = new AclAction("ViewSecret");
	}
}
