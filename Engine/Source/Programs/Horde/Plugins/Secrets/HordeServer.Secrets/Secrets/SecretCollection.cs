// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.RegularExpressions;
using EpicGames.Horde.Secrets;
using Microsoft.Extensions.Options;

namespace HordeServer.Secrets
{
	/// <summary>
	/// Implementation of <see cref="ISecretCollection"/>
	/// </summary>
	public class SecretCollection : ISecretCollection
	{
		readonly SecretCollectionInternal _secretCollectionInternal;
		readonly IOptionsSnapshot<SecretsConfig> _secretsConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public SecretCollection(SecretCollectionInternal secretCollectionInternal, IOptionsSnapshot<SecretsConfig> secretsConfig)
		{
			_secretCollectionInternal = secretCollectionInternal;
			_secretsConfig = secretsConfig;
		}

		/// <inheritdoc/>
		public async Task<ISecretProperty?> ResolveAsync(string value, CancellationToken cancellationToken)
		{
			return await _secretCollectionInternal.ResolveAsync(value, _secretsConfig.Value, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<ISecret?> GetAsync(SecretId secretId, object config, CancellationToken cancellationToken)
		{
			SecretsConfig secretsConfig = config as SecretsConfig ?? throw new InvalidCastException($"{nameof(config)} is not a SecretsConfig");
			return await _secretCollectionInternal.GetAsync(secretId, secretsConfig, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<ISecret?> GetAsync(SecretId secretId, CancellationToken cancellationToken)
		{
			return await _secretCollectionInternal.GetAsync(secretId, _secretsConfig.Value, cancellationToken);
		}
	}

	/// <summary>
	/// Core implementation of <see cref="ISecretCollection"/> which exists as a global service.
	/// </summary>
	public class SecretCollectionInternal
	{
		class Secret : ISecret
		{
			public SecretId Id { get; }
			public IReadOnlyDictionary<string, string> Data { get; }

			public Secret(SecretId id, IReadOnlyDictionary<string, string> data)
			{
				Id = id;
				Data = data;
			}
		}

		class SecretProperty : ISecretProperty
		{
			public SecretId Id { get; }
			public string Name { get; }
			public string Value { get; }

			public SecretProperty(SecretId id, string name, string value)
			{
				Id = id;
				Name = name;
				Value = value;
			}
		}

		readonly Dictionary<string, ISecretProvider> _providers;

		/// <summary>
		/// Constructor
		/// </summary>
		public SecretCollectionInternal(IEnumerable<ISecretProvider> providers)
		{
			_providers = providers.ToDictionary(x => x.Name, x => x);
		}

		/// <summary>
		/// Resolve a secret to concrete values
		/// </summary>
		/// <param name="secretId">The secret to resolve</param>
		/// <param name="secretsConfig">Configuration for the secret</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ISecret?> GetAsync(SecretId secretId, SecretsConfig secretsConfig, CancellationToken cancellationToken)
		{
			if (!secretsConfig.TryGetSecret(secretId, out SecretConfig? secretConfig))
			{
				return null;
			}

			// secret provider configs are optional
			secretsConfig.TryGetSecretProviderConfigs(secretId, out Dictionary<string, SecretProviderConfig>? secretProviderConfigs);

			return await GetAsync(secretConfig, secretProviderConfigs, cancellationToken);
		}

		/// <summary>
		/// Resolve a secret to concrete values
		/// </summary>
		/// <param name="config">Configuration for the secret</param>
		/// <param name="secretProviderConfigs">Secret provider configs for the secret</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ISecret> GetAsync(SecretConfig config, Dictionary<string, SecretProviderConfig>? secretProviderConfigs, CancellationToken cancellationToken)
		{
			// Add the hard-coded secrets
			Dictionary<string, string> data = new Dictionary<string, string>();
			foreach ((string key, string value) in config.Data)
			{
				data.Add(key, value);
			}

			// Fetch all the secrets from external providers
			foreach (ExternalSecretConfig externalConfig in config.Sources)
			{
				ISecretProvider provider;
				SecretProviderConfig? providerConfig = null;
				if (!String.IsNullOrEmpty(externalConfig.Provider))
				{
					provider = _providers[externalConfig.Provider];
				}
				else if (secretProviderConfigs != null && secretProviderConfigs.TryGetValue(externalConfig.ProviderConfig, out providerConfig) && !String.IsNullOrEmpty(providerConfig.Provider))
				{
					provider = _providers[providerConfig.Provider];
				}
				else
				{
					throw new InvalidOperationException($"Missing or invalid secret provider identifier for '{externalConfig.ProviderConfig}'");
				}

				string secret = await provider.GetSecretAsync(externalConfig.Path, providerConfig, cancellationToken);
				if (externalConfig.Format == ExternalSecretFormat.Text)
				{
					data[externalConfig.Key ?? "default"] = secret;
				}
				else if (externalConfig.Format == ExternalSecretFormat.Json)
				{
					Dictionary<string, string>? values = JsonSerializer.Deserialize<Dictionary<string, string>>(secret, new JsonSerializerOptions { AllowTrailingCommas = true });
					if (values != null)
					{
						foreach ((string key, string value) in values)
						{
							data[key] = value;
						}
					}
				}
				else
				{
					throw new NotImplementedException();
				}
			}

			return new Secret(config.Id, data);
		}

		/// <summary>
		/// Resolves a string to a property of a secret
		/// </summary>
		/// <param name="value">A string that contains a secret ID and property name</param>
		/// <param name="secretsConfig">Configuration for the secret</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ISecretProperty?> ResolveAsync(string value, SecretsConfig secretsConfig, CancellationToken cancellationToken)
		{
			if (String.IsNullOrEmpty(value))
			{
				return null;
			}

			Match match = Regex.Match(value, @"^horde:secret:(?<SecretId>[^\.]+)\.(?<PropertyName>[^\.]+)?$");
			if (!match.Success)
			{
				return null;
			}

			SecretId secretId = new(match.Groups["SecretId"].Value);

			ISecret? secret = await GetAsync(secretId, secretsConfig, cancellationToken);
			if (secret == null)
			{
				throw new KeyNotFoundException($"The given secret '{secretId}' was not found");
			}

			string propertyName = match.Groups["PropertyName"].Value;
			if (!secret.Data.TryGetValue(propertyName, out string? propertyValue))
			{
				throw new KeyNotFoundException($"The given property '{propertyName}' for the secret '{secretId}' was not found");
			}

			ISecretProperty secretProperty = new SecretProperty(secretId, propertyName, propertyValue);
			return secretProperty;
		}
	}
}
