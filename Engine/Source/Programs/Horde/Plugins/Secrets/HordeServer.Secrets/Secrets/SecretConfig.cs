// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Secrets;
using HordeServer.Acls;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Secrets
{
	/// <summary>
	/// Configuration for a secret value
	/// </summary>
	public class SecretConfig
	{
		/// <summary>
		/// Identifier for this secret
		/// </summary>
		public SecretId Id { get; set; }

		/// <summary>
		/// Key/value pairs associated with this secret
		/// </summary>
		public Dictionary<string, string> Data { get; set; } = new Dictionary<string, string>();

		/// <summary>
		/// Providers to source key/value pairs from
		/// </summary>
		public List<ExternalSecretConfig> Sources { get; set; } = new List<ExternalSecretConfig>();

		/// <summary>
		/// Defines access to this particular secret
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Called after the config has been read
		/// </summary>
		/// <param name="parentAcl">Parent acl object</param>
		public void PostLoad(AclConfig parentAcl)
		{
			Acl.PostLoad(parentAcl, $"secret:{Id}", AclConfig.GetActions([typeof(SecretAclAction)]));
		}

		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to validate</param>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);
	}

	/// <summary>
	/// Format describing how to parse external secret values
	/// </summary>
	public enum ExternalSecretFormat
	{
		/// <summary>
		/// Secret is a plain text value which will be stored using the external secret key
		/// </summary>
		Text,

		/// <summary>
		/// Secret is a JSON formatted string containing key/value pairs
		/// </summary>
		Json,
	}

	/// <summary>
	/// Configuration for an external secret provider
	/// </summary>
	public class ExternalSecretConfig
	{
		/// <summary>
		/// The name to look up a secret provider config. A config is mutually exclusive to <see cref="Provider"/>
		/// and to be used when a secret requires additional options than just the name of the provider
		/// </summary>
		public string ProviderConfig { get; set; } = String.Empty;

		/// <summary>
		/// Name of the provider to use
		/// </summary>
		public string Provider { get; set; } = String.Empty;

		/// <summary>
		/// Format of the secret
		/// </summary>
		public ExternalSecretFormat Format { get; set; }

		/// <summary>
		/// Optional key indicating the parameter to set in the resulting data array. Required if <see cref="Format"/> is <see cref="ExternalSecretFormat.Text"/>.
		/// </summary>
		public string? Key { get; set; }

		/// <summary>
		/// Optional value indicating what to fetch from the provider
		/// </summary>
		public string Path { get; set; } = "default";
	}
}
