// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Secrets.Providers;

namespace HordeServer.Secrets
{
	/// <summary>
	/// Common settings object for different secret providers
	/// </summary>
	public class SecretProviderConfig
	{
		/// <summary>
		/// Name of the secret provider config
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Name of the provider
		/// </summary>
		public string Provider { get; set; } = String.Empty;

		/// <summary>
		/// Configuration for HCP Vault
		/// </summary>
		public HcpVaultConfig? HcpVault { get; set; }

		/// <summary>
		/// Configuration for Azure Key Vault
		/// </summary>
		public AzureKeyVaultConfig? AzureKeyVault { get; set; }
	}
}
