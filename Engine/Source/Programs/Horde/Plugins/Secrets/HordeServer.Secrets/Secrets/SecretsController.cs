// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Secrets;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Secrets
{
	/// <summary>
	/// Controller for the /api/v1/secrets endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class SecretsController : HordeControllerBase
	{
		readonly ISecretCollection _secretCollection;
		readonly IOptionsSnapshot<SecretsConfig> _secretsConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public SecretsController(ISecretCollection secretCollection, IOptionsSnapshot<SecretsConfig> secretsConfig)
		{
			_secretCollection = secretCollection;
			_secretsConfig = secretsConfig;
		}

		/// <summary>
		/// Query all the secrets available for the current user
		/// </summary>
		[HttpGet]
		[Route("/api/v1/secrets")]
		public ActionResult<GetSecretsResponse> GetSecrets()
		{
			List<SecretId> secretIds = new List<SecretId>();
			foreach (SecretConfig secret in _secretsConfig.Value.Secrets)
			{
				if (secret.Authorize(SecretAclAction.ViewSecret, User))
				{
					secretIds.Add(secret.Id);
				}
			}
			return new GetSecretsResponse(secretIds);
		}

		/// <summary>
		/// Retrieve information about a specific secret
		/// </summary>
		/// <param name="secretId">Id of the secret to retrieve</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested secret</returns>
		[HttpGet]
		[Route("/api/v1/secrets/{secretId}")]
		[ProducesResponseType(typeof(GetSecretResponse), 200)]
		public async Task<ActionResult<object>> GetSecretAsync(SecretId secretId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ISecret? secret = await _secretCollection.GetAsync(secretId, cancellationToken);
			if (secret == null)
			{
				return NotFound(secretId);
			}
			if (!_secretsConfig.Value.Authorize(secret.Id, SecretAclAction.ViewSecret, User))
			{
				return Forbid(SecretAclAction.ViewSecret, secretId);
			}

			return new GetSecretResponse(secret.Id, secret.Data.ToDictionary(x => x.Key, x => x.Value)).ApplyFilter(filter);
		}

		/// <summary>
		/// Converts the string representation of a secret and property to a concrete value
		/// The format of the string is 'horde:secret:secret-id.property-name'
		/// </summary>
		/// <param name="value">A string that contains a secret ID and property name</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested secret</returns>
		[HttpGet]
		[Route("/api/v1/secrets/resolve/{value}")]
		[ProducesResponseType(typeof(GetSecretPropertyResponse), 200)]
		public async Task<ActionResult<object>> ResolveSecretAsync(string value, CancellationToken cancellationToken = default)
		{
			ISecretProperty? secret = await _secretCollection.ResolveAsync(value, cancellationToken);
			if (secret == null)
			{
				return NotFound(value);
			}
			if (!_secretsConfig.Value.Authorize(secret.Id, SecretAclAction.ViewSecret, User))
			{
				return Forbid(SecretAclAction.ViewSecret, secret.Id);
			}

			return new GetSecretPropertyResponse(secret.Id, secret.Name, secret.Value);
		}
	}
}
