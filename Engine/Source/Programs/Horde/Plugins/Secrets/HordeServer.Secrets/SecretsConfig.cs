// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Secrets;
using HordeServer.Plugins;
using HordeServer.Secrets;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Configuration for the secrets system
	/// </summary>
	public class SecretsConfig : IPluginConfig
	{
		/// <summary>
		/// List of secrets
		/// </summary>
		public List<SecretConfig> Secrets { get; set; } = new List<SecretConfig>();

		private readonly Dictionary<SecretId, SecretConfig> _secretLookup = new Dictionary<SecretId, SecretConfig>();

		/// <summary>
		/// List of configurations for secret providers
		/// </summary>
		public List<SecretProviderConfig> ProviderConfigs { get; set; } = new();

		private readonly Dictionary<SecretId, Dictionary<string, SecretProviderConfig>> _providerConfigLookup = new();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			_secretLookup.Clear();
			foreach (SecretConfig secret in Secrets)
			{
				_secretLookup.Add(secret.Id, secret);
				secret.PostLoad(configOptions.ParentAcl);
			}

			Dictionary<string, SecretProviderConfig> providerConfigNameLookup = new();
			foreach (SecretProviderConfig secretProviderConfig in ProviderConfigs)
			{
				providerConfigNameLookup[secretProviderConfig.Name] = secretProviderConfig;
			}
			_providerConfigLookup.Clear();
			foreach (KeyValuePair<SecretId, SecretConfig> secret in _secretLookup)
			{
				foreach (ExternalSecretConfig externalConfig in secret.Value.Sources)
				{
					if (!String.IsNullOrEmpty(externalConfig.ProviderConfig))
					{
						if (!_providerConfigLookup.ContainsKey(secret.Key))
						{
							_providerConfigLookup[secret.Key] = new Dictionary<string, SecretProviderConfig>();
						}
						if (providerConfigNameLookup.TryGetValue(externalConfig.ProviderConfig, out SecretProviderConfig? secretProviderConfig))
						{
							_providerConfigLookup[secret.Key].TryAdd(externalConfig.ProviderConfig, secretProviderConfig);
						}
					}
				}
			}
		}

		/// <summary>
		/// Attempts to get a secret configuration from this object
		/// </summary>
		/// <param name="secretId">Secret id</param>
		/// <param name="config">Receives the secret configuration on success</param>
		/// <returns>True on success</returns>
		public bool TryGetSecret(SecretId secretId, [NotNullWhen(true)] out SecretConfig? config)
			=> _secretLookup.TryGetValue(secretId, out config);

		/// <summary>
		/// Attempts to get a secret provider configuration from this object
		/// </summary>
		/// <param name="secretId">Secret id</param>
		/// <param name="config">Receives the secret provider configurations on success</param>
		/// <returns>True on success</returns>
		public bool TryGetSecretProviderConfigs(SecretId secretId, [NotNullWhen(true)] out Dictionary<string, SecretProviderConfig>? config)
			=> _providerConfigLookup.TryGetValue(secretId, out config);

		/// <summary>
		/// Authorize access to a secret
		/// </summary>
		public bool Authorize(SecretId secretId, AclAction action, ClaimsPrincipal user)
			=> TryGetSecret(secretId, out SecretConfig? secretConfig) && secretConfig.Authorize(action, user);
	}
}
