// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.SimpleSystemsManagement;
using EpicGames.Horde.Secrets;
using HordeServer.Plugins;
using HordeServer.Secrets;
using HordeServer.Secrets.Providers;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the secrets plugin
	/// </summary>
	[Plugin("Secrets", GlobalConfigType = typeof(SecretsConfig), ServerConfigType = typeof(SecretsServerConfig))]
	public class SecretsPlugin : IPluginStartup
	{
		private readonly IServerInfo _serverInfo;
		private readonly SecretsServerConfig _staticConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public SecretsPlugin(IServerInfo serverInfo, SecretsServerConfig staticConfig)
		{
			_serverInfo = serverInfo;
			_staticConfig = staticConfig;
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{ }

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddHttpClient();
			services.AddSingleton<SecretCollectionInternal>();
			services.AddScoped<ISecretCollection, SecretCollection>();

			if (_staticConfig.WithAws)
			{
				services.AddDefaultAWSOptions(_serverInfo.Configuration.GetAWSOptions());
				services.AddAWSService<IAmazonSimpleSystemsManagement>();
				services.AddSingleton<ISecretProvider, AwsParameterStoreSecretProvider>();
			}

			services.AddSingleton<ISecretProvider, HcpVaultSecretProvider>();
			services.AddSingleton<ISecretProvider, AzureKeyVaultSecretProvider>();
		}
	}

	/// <summary>
	/// Helper methods for secrets config
	/// </summary>
	public static class SecretsPluginExtensions
	{
		/// <summary>
		/// Configures the secrets plugin
		/// </summary>
		public static void AddSecretsConfig(this IDictionary<PluginName, IPluginConfig> dictionary, SecretsConfig secretsConfig)
			=> dictionary[new PluginName("Secrets")] = secretsConfig;

		/// <summary>
		/// Gets configuration for the secrets plugin
		/// </summary>
		public static SecretsConfig GetSecretsConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (SecretsConfig)dictionary[new PluginName("Secrets")];
	}
}
