// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Static configuration for the secrets plugin
	/// </summary>
	public class SecretsServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Whether to enable Amazon Web Services (AWS) specific features
		/// </summary>
		public bool WithAws { get; set; }
	}
}
