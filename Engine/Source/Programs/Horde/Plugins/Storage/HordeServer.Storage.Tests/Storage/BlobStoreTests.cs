// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Storage
{
	[TestClass]
	public class BlobStoreTests : ServerTestSetup
	{
		static readonly BlobType s_blobType = new BlobType("{AFDF76A7-4DEE-5333-F5B5-37B8451251CA}", 1);

		public StorageService StorageService => ServiceProvider.GetRequiredService<StorageService>();

		public BlobStoreTests()
		{
			AddPlugin<StoragePlugin>();
		}

		async Task<IStorageNamespace> GetStorageNamespaceAsync()
		{
			StorageConfig storageConfig = new StorageConfig();
			storageConfig.Backends.Add(new BackendConfig { Id = new BackendId("default-backend"), Type = StorageBackendType.Memory });
			storageConfig.Namespaces.Add(new NamespaceConfig { Id = new NamespaceId("default"), Backend = new BackendId("default-backend"), GcDelayHrs = 0.0 });

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.Add(new PluginName("storage"), storageConfig);
			await SetConfigAsync(globalConfig);

			return StorageService.GetNamespace(new NamespaceId("default"));
		}

		static byte[] CreateTestData(int length, int seed)
		{
			byte[] data = new byte[length];
			new Random(seed).NextBytes(data);
			return data;
		}

		class Blob
		{
			public ReadOnlyMemory<byte> Data { get; set; }
			public List<HashedBlobRefValue> References { get; set; }

			public Blob() : this(ReadOnlyMemory<byte>.Empty, new List<HashedBlobRefValue>())
			{
			}

			public Blob(ReadOnlyMemory<byte> data, IEnumerable<HashedBlobRefValue> references)
			{
				Data = data;
				References = new List<HashedBlobRefValue>(references);
			}
		}

		static async ValueTask<Blob> ReadBlobAsync(IStorageNamespace store, HashedBlobRefValue locator)
		{
			using (BlobData blobData = await store.CreateBlobRef(locator.Hash, locator.Locator).ReadBlobDataAsync())
			{
				BlobReader reader = new BlobReader(blobData, null);
				ReadOnlyMemory<byte> data = reader.ReadVariableLengthBytes();

				List<IHashedBlobRef> handles = new List<IHashedBlobRef>();
				while (reader.RemainingMemory.Length > 0)
				{
					handles.Add(reader.ReadBlobRef<object>());
				}

				List<HashedBlobRefValue> locators = handles.ConvertAll(x => x.GetRefValue());
				return new Blob(data, locators);
			}
		}

		static async ValueTask<HashedBlobRefValue> WriteBlobAsync(IStorageNamespace store, Blob blob)
		{
			await using IBlobWriter writer = store.CreateBlobWriter();
			writer.WriteVariableLengthBytes(blob.Data.Span);

			foreach (HashedBlobRefValue reference in blob.References)
			{
				writer.WriteBlobRef(store.CreateBlobRef(reference));
			}

			IHashedBlobRef blobRef = await writer.CompleteAsync(s_blobType);
			await blobRef.FlushAsync();

			return new HashedBlobRefValue(blobRef.Hash, blobRef.GetLocator());
		}

		[TestMethod]
		public async Task LeafTestAsync()
		{
			IStorageNamespace store = await GetStorageNamespaceAsync();

			byte[] input = CreateTestData(256, 0);

			Blob inputBlob = new Blob(input, Array.Empty<HashedBlobRefValue>());
			HashedBlobRefValue locator = await WriteBlobAsync(store, inputBlob);
			Blob outputBlob = await ReadBlobAsync(store, locator);

			Assert.IsTrue(outputBlob.Data.Span.SequenceEqual(input));
			Assert.AreEqual(0, outputBlob.References.Count);
		}

		[TestMethod]
		public async Task ReferenceTestAsync()
		{
			IStorageNamespace store = await GetStorageNamespaceAsync();

			byte[] input1 = CreateTestData(256, 1);
			HashedBlobRefValue locator1 = await WriteBlobAsync(store, new Blob(input1, Array.Empty<HashedBlobRefValue>()));
			Blob blob1 = await ReadBlobAsync(store, locator1);
			Assert.IsTrue(blob1.Data.Span.SequenceEqual(input1));
			Assert.IsTrue(blob1.References.SequenceEqual(Array.Empty<HashedBlobRefValue>()));

			byte[] input2 = CreateTestData(256, 2);
			HashedBlobRefValue locator2 = await WriteBlobAsync(store, new Blob(input2, new HashedBlobRefValue[] { locator1 }));
			Blob blob2 = await ReadBlobAsync(store, locator2);
			Assert.IsTrue(blob2.Data.Span.SequenceEqual(input2));
			Assert.IsTrue(blob2.References.SequenceEqual(new[] { locator1 }));

			byte[] input3 = CreateTestData(256, 3);
			HashedBlobRefValue locator3 = await WriteBlobAsync(store, new Blob(input3, new HashedBlobRefValue[] { locator1, locator2, locator1 }));
			Blob blob3 = await ReadBlobAsync(store, locator3);
			Assert.IsTrue(blob3.Data.Span.SequenceEqual(input3));
			Assert.IsTrue(blob3.References.SequenceEqual(new[] { locator1, locator2, locator1 }));

			for (int idx = 0; idx < 2; idx++)
			{
				RefName refName = new RefName("hello");
				await store.AddRefAsync(refName, store.CreateBlobRef(locator3));
				IHashedBlobRef refTarget = await store.ReadRefAsync(refName);
				Assert.AreEqual(locator3.Locator, refTarget.GetLocator());
			}
		}

		[TestMethod]
		public async Task RefExpiryTestAsync()
		{
			IStorageNamespace store = await GetStorageNamespaceAsync();

			Blob blob1 = new Blob(new byte[] { 1, 2, 3 }, Array.Empty<HashedBlobRefValue>());
			HashedBlobRefValue target = await WriteBlobAsync(store, blob1);

			await store.AddRefAsync("test-ref-1", store.CreateBlobRef(target));
			await store.AddRefAsync("test-ref-2", store.CreateBlobRef(target), new RefOptions { Lifetime = TimeSpan.FromMinutes(30.0), Extend = true });
			await store.AddRefAsync("test-ref-3", store.CreateBlobRef(target), new RefOptions { Lifetime = TimeSpan.FromMinutes(30.0), Extend = false });

			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-1"));
			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-2"));
			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-3"));

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(25.0));

			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-1"));
			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-2"));
			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-3"));

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(25.0));

			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-1"));
			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-2"));
			Assert.AreEqual(default, await TryReadRefValueAsync(store, "test-ref-3"));

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(35.0));

			Assert.AreEqual(target, await TryReadRefValueAsync(store, "test-ref-1"));
			Assert.AreEqual(default, await TryReadRefValueAsync(store, "test-ref-2"));
			Assert.AreEqual(default, await TryReadRefValueAsync(store, "test-ref-3"));
		}

		static async Task<HashedBlobRefValue?> TryReadRefValueAsync(IStorageNamespace store, RefName name)
		{
			IHashedBlobRef? handle = await store.TryReadRefAsync(name);
			if (handle == null)
			{
				return default;
			}
			return handle.GetRefValue();
		}

		[TestMethod]
		public async Task AliasTestAsync()
		{
			IStorageNamespace store = await GetStorageNamespaceAsync();

			BlobType type = new BlobType(Guid.Empty, 0);

			IBlobRef blobRef;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				byte[] data = [1, 2, 3];
				writer.WriteVariableLengthBytes(data);
				blobRef = await writer.CompleteAsync(type);
			}

			await store.AddAliasAsync("hello", blobRef, rank: 4);
			await store.AddAliasAsync("world", blobRef, rank: 4);
			await store.AddAliasAsync("hello", blobRef, rank: 3);
			await store.AddAliasAsync("world", blobRef, rank: 3);

			BlobAlias[] aliases1 = await store.FindAliasesAsync("hello", 10);
			Assert.AreEqual(1, aliases1.Length);
			Assert.AreEqual(3, aliases1[0].Rank);

			BlobAlias[] aliases2 = await store.FindAliasesAsync("world", 10);
			Assert.AreEqual(1, aliases2.Length);
			Assert.AreEqual(3, aliases2[0].Rank);
		}

		[TestMethod]
		public async Task MultipleAliasTestsAsync()
		{
			IStorageNamespace store = await GetStorageNamespaceAsync();

			BlobType type = new BlobType(Guid.Empty, 0);

			List<IBlobRef> blobRefs = new List<IBlobRef>();
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				byte[] data = [1, 2, 3];
				for (int idx = 0; idx < 1000; idx++)
				{
					writer.WriteVariableLengthBytes(data);
					blobRefs.Add(await writer.CompleteAsync(type));
				}
			}

			await using (IStorageWriter writer = store.CreateWriter())
			{
				for (int idx = 0; idx < blobRefs.Count; idx++)
				{
					await writer.AddAliasAsync("test", blobRefs[idx]);
				}
			}
		}
	}
}

