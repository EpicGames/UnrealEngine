// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Storage;

[TestClass]
public class CollectionStatsServiceTests : ServerTestSetup
{
	private CollectionStatsService StatsService => ServiceProvider.GetRequiredService<CollectionStatsService>();

	public CollectionStatsServiceTests()
	{
		AddPlugin<StoragePlugin>();
	}

	[TestMethod]
	public async Task SnapshotTickCapturesStatsAsync()
	{
		CollectionStatsService service = StatsService;
		await service.StartAsync(CancellationToken.None);

		// Trigger the hourly snapshot ticker
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));

		CollectionStatsDocument? latest = await service.GetLatestAsync(CancellationToken.None);
		Assert.IsNotNull(latest);
		Assert.IsTrue(latest.Collections.Count >= 0);
	}

	[TestMethod]
	public async Task FindReturnsSnapshotsInTimeRangeAsync()
	{
		CollectionStatsService service = StatsService;
		await service.StartAsync(CancellationToken.None);

		DateTime beforeFirst = Clock.UtcNow;

		// Trigger two snapshot ticks
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));
		DateTime afterFirst = Clock.UtcNow;

		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));
		DateTime afterSecond = Clock.UtcNow;

		// Query all
		IReadOnlyList<CollectionStatsDocument> all = await service.FindAsync(beforeFirst, afterSecond, 100, CancellationToken.None);
		Assert.AreEqual(2, all.Count);

		// Query only first
		IReadOnlyList<CollectionStatsDocument> firstOnly = await service.FindAsync(beforeFirst, afterFirst, 100, CancellationToken.None);
		Assert.AreEqual(1, firstOnly.Count);
	}

	[TestMethod]
	public async Task CleanupRemovesOldSnapshotsAsync()
	{
		CollectionStatsService service = StatsService;
		await service.StartAsync(CancellationToken.None);

		DateTime baseTime = Clock.UtcNow;

		// Trigger a snapshot
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));

		CollectionStatsDocument? snapshot = await service.GetLatestAsync(CancellationToken.None);
		Assert.IsNotNull(snapshot);

		// Advance past 90-day retention + cleanup ticker interval (24 hours)
		await Clock.AdvanceAsync(TimeSpan.FromDays(91));

		// Old snapshot should be cleaned up
		IReadOnlyList<CollectionStatsDocument> old = await service.FindAsync(baseTime, baseTime.AddDays(1), 100, CancellationToken.None);
		Assert.AreEqual(0, old.Count);
	}

	[TestMethod]
	public async Task GetLatestReturnsNewestSnapshotAsync()
	{
		CollectionStatsService service = StatsService;
		await service.StartAsync(CancellationToken.None);

		// Trigger three snapshots
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));
		await Clock.AdvanceAsync(TimeSpan.FromHours(1.1));
		DateTime expectedTime = Clock.UtcNow;

		CollectionStatsDocument? latest = await service.GetLatestAsync(CancellationToken.None);
		Assert.IsNotNull(latest);

		// The latest should be close to the current clock time
		Assert.IsTrue(Math.Abs((latest.TimeUtc - expectedTime).TotalMinutes) < 10);
	}
}
