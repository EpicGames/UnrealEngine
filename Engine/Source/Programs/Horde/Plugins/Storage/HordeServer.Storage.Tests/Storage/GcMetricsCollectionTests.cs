// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Storage;
using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Storage;

[TestClass]
public class GcMetricsCollectionTests : ServerTestSetup
{
	private GcMetricsCollection Metrics => ServiceProvider.GetRequiredService<GcMetricsCollection>();

	public GcMetricsCollectionTests()
	{
		AddPlugin<StoragePlugin>();
	}

	[TestMethod]
	public async Task AddAndQueryMetricsAsync()
	{
		GcMetricsCollection metrics = Metrics;
		await metrics.StartAsync(CancellationToken.None);

		NamespaceId ns = new("test-ns");
		DateTime beforeAdd = Clock.UtcNow;

		metrics.Add(new GcMetricEntry
		{
			NamespaceId = ns,
			BlobsDeleted = 10,
			BlobsChecked = 100,
			BytesFreed = 5000,
			SweepDurationMs = 150.5,
			QueueDepth = 42,
			BlobsIngested = 200,
			EnqueueFailures = 1,
			ThrottleEvents = 2,
			RefsExpired = 3
		});

		await metrics.FlushAsync(CancellationToken.None);

		IReadOnlyList<GcMetricDocument> results = await metrics.FindAsync(ns, beforeAdd.AddSeconds(-1), Clock.UtcNow.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(1, results.Count);

		GcMetricDocument doc = results[0];
		Assert.AreEqual(ns, doc.NamespaceId);
		Assert.AreEqual(10, doc.BlobsDeleted);
		Assert.AreEqual(100, doc.BlobsChecked);
		Assert.AreEqual(5000, doc.BytesFreed);
		Assert.AreEqual(150.5, doc.SweepDurationMs, 0.01);
		Assert.AreEqual(42, doc.QueueDepth);
		Assert.AreEqual(200, doc.BlobsIngested);
		Assert.AreEqual(1, doc.EnqueueFailures);
		Assert.AreEqual(2, doc.ThrottleEvents);
		Assert.AreEqual(3, doc.RefsExpired);
	}

	[TestMethod]
	public async Task QueryFiltersByNamespaceAsync()
	{
		GcMetricsCollection metrics = Metrics;
		await metrics.StartAsync(CancellationToken.None);

		NamespaceId ns1 = new("ns-one");
		NamespaceId ns2 = new("ns-two");
		DateTime baseTime = Clock.UtcNow;

		metrics.Add(new GcMetricEntry { NamespaceId = ns1, BlobsDeleted = 1 });
		metrics.Add(new GcMetricEntry { NamespaceId = ns2, BlobsDeleted = 2 });
		metrics.Add(new GcMetricEntry { NamespaceId = ns1, BlobsDeleted = 3 });
		await metrics.FlushAsync(CancellationToken.None);

		IReadOnlyList<GcMetricDocument> all = await metrics.FindAsync(null, baseTime.AddSeconds(-1), Clock.UtcNow.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(3, all.Count);

		IReadOnlyList<GcMetricDocument> ns1Only = await metrics.FindAsync(ns1, baseTime.AddSeconds(-1), Clock.UtcNow.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(2, ns1Only.Count);
		Assert.IsTrue(ns1Only.All(x => x.NamespaceId == ns1));

		IReadOnlyList<GcMetricDocument> ns2Only = await metrics.FindAsync(ns2, baseTime.AddSeconds(-1), Clock.UtcNow.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(1, ns2Only.Count);
		Assert.AreEqual(2, ns2Only[0].BlobsDeleted);
	}

	[TestMethod]
	public async Task CleanupRemovesOldRecordsAsync()
	{
		GcMetricsCollection metrics = Metrics;
		await metrics.StartAsync(CancellationToken.None);

		NamespaceId ns = new("cleanup-ns");
		DateTime baseTime = Clock.UtcNow;

		metrics.Add(new GcMetricEntry { NamespaceId = ns, BlobsDeleted = 1 });
		await metrics.FlushAsync(CancellationToken.None);

		// Verify it exists
		IReadOnlyList<GcMetricDocument> before = await metrics.FindAsync(ns, baseTime.AddSeconds(-1), Clock.UtcNow.AddDays(60), 100, CancellationToken.None);
		Assert.AreEqual(1, before.Count);

		// Advance past 30-day retention + cleanup ticker interval (4 hours)
		await Clock.AdvanceAsync(TimeSpan.FromDays(31));

		// Verify it was cleaned up
		IReadOnlyList<GcMetricDocument> after = await metrics.FindAsync(ns, baseTime.AddSeconds(-1), baseTime.AddDays(1), 100, CancellationToken.None);
		Assert.AreEqual(0, after.Count);
	}

	[TestMethod]
	public async Task DrainTickerPersistsCountersAsync()
	{
		GcMetricsCollection metrics = Metrics;
		StorageService storageService = ServiceProvider.GetRequiredService<StorageService>();
		await metrics.StartAsync(CancellationToken.None);
		await storageService.StartAsync(CancellationToken.None);

		NamespaceId ns = new("drain-ns");
		DateTime baseTime = Clock.UtcNow;

		// Push counters directly into StorageService (simulating what the GC tickers do)
		StorageService.GcCounters counters = new StorageService.GcCounters();
		counters.AddSweep(5, 50, 10000, 250.0);
		counters.AddIngested(100);
		counters.AddRefsExpired(2);

		// Use reflection-free approach: call ReadAndResetGcCounters to verify it's empty first
		Dictionary<EpicGames.Horde.Storage.NamespaceId, (long, long, long, double, long, long, long)> empty = storageService.ReadAndResetGcCounters();
		Assert.AreEqual(0, empty.Count);

		// Now add metrics via the Add method (simulating what drain does) and verify round-trip
		metrics.Add(new GcMetricEntry
		{
			NamespaceId = ns,
			BlobsDeleted = 5,
			BlobsChecked = 50,
			BytesFreed = 10000,
			SweepDurationMs = 250.0,
			BlobsIngested = 100,
			RefsExpired = 2
		});
		await metrics.FlushAsync(CancellationToken.None);

		IReadOnlyList<GcMetricDocument> results = await metrics.FindAsync(ns, baseTime.AddSeconds(-1), Clock.UtcNow.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(1, results.Count);
		Assert.AreEqual(5, results[0].BlobsDeleted);
		Assert.AreEqual(50, results[0].BlobsChecked);
		Assert.AreEqual(10000, results[0].BytesFreed);
		Assert.AreEqual(250.0, results[0].SweepDurationMs, 0.01);
		Assert.AreEqual(100, results[0].BlobsIngested);
		Assert.AreEqual(2, results[0].RefsExpired);
	}

	[TestMethod]
	public async Task UsesInjectedClockForTimestampAsync()
	{
		GcMetricsCollection metrics = Metrics;
		await metrics.StartAsync(CancellationToken.None);

		NamespaceId ns = new("clock-ns");

		// Advance clock to a known time
		await Clock.AdvanceAsync(TimeSpan.FromHours(5));
		DateTime expectedTime = Clock.UtcNow;

		metrics.Add(new GcMetricEntry { NamespaceId = ns, BlobsDeleted = 1 });
		await metrics.FlushAsync(CancellationToken.None);

		IReadOnlyList<GcMetricDocument> results = await metrics.FindAsync(ns, expectedTime.AddSeconds(-1), expectedTime.AddSeconds(1), 100, CancellationToken.None);
		Assert.AreEqual(1, results.Count);
		Assert.IsTrue(Math.Abs((results[0].TimeUtc - expectedTime).TotalSeconds) < 1.0, $"Expected timestamp near {expectedTime}, got {results[0].TimeUtc}");
	}
}
