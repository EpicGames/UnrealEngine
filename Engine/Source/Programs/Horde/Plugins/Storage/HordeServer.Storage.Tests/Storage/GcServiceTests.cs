// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.ObjectStores;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Storage
{
	[TestClass]
	public sealed class GcServiceTests : ServerTestSetup
	{
		public StorageService StorageService => ServiceProvider.GetRequiredService<StorageService>();

		public GcServiceTests()
		{
			AddPlugin<StoragePlugin>();
		}

		[TestMethod]
		public async Task CreateBasicTreeAsync()
		{
			await StorageService.StartAsync(CancellationToken.None);

			StorageConfig storageConfig = new StorageConfig();
			storageConfig.Backends.Add(new BackendConfig { Id = new BackendId("default-backend"), Type = StorageBackendType.Memory });
			storageConfig.Namespaces.Add(new NamespaceConfig { Id = new NamespaceId("default"), Backend = new BackendId("default-backend"), GcDelayHrs = 0.0 });

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.Add(new PluginName("storage"), storageConfig);
			await SetConfigAsync(globalConfig);

			IStorageNamespace store = StorageService.GetNamespace(new NamespaceId("default"));

			Random random = new Random(0);
			IHashedBlobRef[] blobs = await CreateTestDataAsync(store, 30, 50, 30, 5, random);

			HashSet<IBlobRef> roots = new HashSet<IBlobRef>();
			for (int idx = 0; idx < 10; idx++)
			{
				int blobIdx = (int)(random.NextDouble() * blobs.Length);
				if (roots.Add(blobs[blobIdx]))
				{
					IHashedBlobRef handle = blobs[blobIdx];
					await store.AddRefAsync(new RefName($"ref-{idx}"), handle);
				}
			}

			HashSet<BlobLocator> nodes = await FindNodesAsync(store, roots);
			await Clock.AdvanceAsync(TimeSpan.FromDays(1.0));

			MemoryObjectStore backend = (MemoryObjectStore)ServiceProvider.GetRequiredService<IObjectStoreFactory>().CreateObjectStore(storageConfig.Backends[0]);

			ObjectKey[] remaining = backend.Blobs.Keys.ToArray();
			Assert.AreEqual(nodes.Count, remaining.Length);

			HashSet<ObjectKey> nodePaths = new HashSet<ObjectKey>(nodes.Select(x => StorageService.GetObjectKey(x.BaseLocator)));
			Assert.IsTrue(remaining.All(x => nodePaths.Contains(x)));
		}

		static async Task<HashSet<BlobLocator>> FindNodesAsync(IStorageNamespace store, IEnumerable<IBlobRef> roots)
		{
			HashSet<BlobLocator> nodes = new HashSet<BlobLocator>();
			await FindNodesAsync(store, roots, nodes);
			return nodes;
		}

		static async Task FindNodesAsync(IStorageNamespace store, IEnumerable<IBlobRef> roots, HashSet<BlobLocator> nodes)
		{
			foreach (IBlobRef root in roots)
			{
				BlobLocator locator = root.GetLocator();
				if (nodes.Add(locator))
				{
					using BlobData data = await root.ReadBlobDataAsync();
					await FindNodesAsync(store, data.Imports, nodes);
				}
			}
		}

		static async ValueTask<IHashedBlobRef[]> CreateTestDataAsync(IStorageNamespace store, int numRoots, int numInterior, int numLeaves, int avgChildren, Random random)
		{
			int firstRoot = 0;
			int firstInterior = firstRoot + numRoots;
			int firstLeaf = firstInterior + numInterior;
			int numNodes = firstLeaf + numLeaves;

			List<int>[] children = new List<int>[numNodes];
			for (int idx = 0; idx < numNodes; idx++)
			{
				children[idx] = new List<int>();
			}

			if (numRoots < numNodes)
			{
				double maxParents = ((numRoots + numInterior) * avgChildren) / (numInterior + numLeaves);
				for (int idx = numRoots; idx < numNodes; idx++)
				{
					int numParents = 1 + (int)(random.NextDouble() * maxParents);
					for (; numParents > 0; numParents--)
					{
						int parentIdx = Math.Min((int)(random.NextDouble() * Math.Min(idx, numRoots + numInterior)), idx - 1);
						children[parentIdx].Add(idx);
					}
				}
			}

			BlobType blobType = new BlobType(Guid.Parse("{AFDF76A7-4DEE-5333-F5B5-37B8451251CA}"), 0);

			IHashedBlobRef[] handles = new IHashedBlobRef[children.Length];
			for (int idx = numNodes - 1; idx >= 0; idx--)
			{
				IHashedBlobRef handle;
				await using (IBlobWriter writer = store.CreateBlobWriter("gctest"))
				{
					List<IHashedBlobRef> imports = children[idx].ConvertAll(x => handles[x]);
					foreach (IHashedBlobRef import in imports)
					{
						writer.WriteBlobRef(import);
					}
					handle = await writer.CompleteAsync<object>(blobType);
				}
				handles[idx] = handle;
			}

			return handles;
		}
	}
}
