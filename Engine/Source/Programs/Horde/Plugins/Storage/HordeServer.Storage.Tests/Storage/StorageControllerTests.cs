// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Tests.Storage
{
#if false
	[TestClass]
	public class StorageControllerTests : ControllerIntegrationTest
	{
		static readonly Guid s_guid = Guid.Parse("{8B9701C9-9677-4E3C-895F-2D654267C4CB}");

		[TestMethod]
		public async Task TestBlobImportsAsync()
		{
			ILogger logger = ServiceProvider.GetRequiredService<ILogger<StorageControllerTests>>();
			HttpStorageBackend backend = new HttpStorageBackend("api/v1/storage/default", () => CreateHttpClient(), () => CreateHttpClient(), logger);
			await using BundleCache cache = new BundleCache();
			using BundleStorageClient client = new BundleStorageClient(backend, cache, null, logger);

			IBlobRef blobRef1;
			IBlobRef blobRef2;
			await using (IBlobWriter writer = client.CreateBlobWriter())
			{
				writer.WriteInt32(1234);
				blobRef1 = await writer.CompleteAsync(new BlobType(s_guid, 0));
				await writer.FlushAsync();

				writer.WriteBlobRef(blobRef1);
				blobRef2 = await writer.CompleteAsync(new BlobType(s_guid, 0));
			}

			BlobLocator locator1 = blobRef1.GetLocator();
			BlobLocator locator2 = blobRef2.GetLocator();

			StorageService storageService = ServiceProvider.GetRequiredService<StorageService>();
			IMongoCollection<StorageService.BlobInfo> blobCollection = storageService.BlobCollection;

			StorageService.BlobInfo blob2 = await blobCollection.Find(x => x.Path == locator2.BaseLocator.ToString()).FirstAsync();
			Assert.IsNotNull(blob2.Imports);
			Assert.AreEqual(1, blob2.Imports.Count);
			Assert.AreEqual(blob2.Path, locator2.BaseLocator.ToString());

			StorageService.BlobInfo blob1 = await blobCollection.Find(x => x.Id == blob2.Imports![0]).FirstAsync();
			Assert.IsNotNull(blob1.Imports);
			Assert.AreEqual(0, blob1.Imports.Count);
			Assert.AreEqual(blob1.Path, locator1.BaseLocator.ToString());
		}
	}
#endif
}
