// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;
using MongoDB.Bson;

namespace HordeServer.Tests.Storage;
using BlobInfo = StorageService.BlobInfo;

[TestClass]
public class StorageServiceTests : ServerTestSetup
{
	private StorageService Service => ServiceProvider.GetRequiredService<StorageService>();
	private readonly NamespaceId _namespaceId = new("memory");
	
	public StorageServiceTests()
	{
		AddPlugin<StoragePlugin>();
	}
	
	[TestMethod]
	public async Task BlobCollectionTestAsync()
	{
		StorageService service = ServiceProvider.GetRequiredService<StorageService>();
		IStorageNamespace client = service.GetNamespace(new NamespaceId("memory"));
		
		BlobType type1 = new BlobType(Guid.Parse("{11C2D886-4164-3349-D1E9-6F943D2ED10B}"), 0);
		byte[] data1 = new byte[] { 1, 2, 3 };
		
		BlobType type2 = new BlobType(Guid.Parse("{6CB3A005-4787-26BA-3E79-D286CB7137D1}"), 0);
		byte[] data2 = new byte[] { 4, 5, 6 };
		
		IHashedBlobRef handle1a;
		IHashedBlobRef handle1b;
		IHashedBlobRef handle2;
		await using (IBlobWriter writer = client.CreateBlobWriter())
		{
			writer.WriteFixedLengthBytes(data1);
			writer.AddAlias("foo", 2);
			handle1a = await writer.CompleteAsync(type1);
			
			writer.WriteFixedLengthBytes(data1);
			writer.AddAlias("foo", 1);
			handle1b = await writer.CompleteAsync(type1);
			
			writer.WriteFixedLengthBytes(data2);
			writer.AddAlias("bar", 0);
			handle2 = await writer.CompleteAsync(type2);
		}
		
		BlobAlias[] aliases;
		
		aliases = await client.FindAliasesAsync("foo");
		Assert.AreEqual(2, aliases.Length);
		Assert.AreEqual(handle1a.GetLocator(), aliases[0].Target.GetLocator());
		Assert.AreEqual(handle1b.GetLocator(), aliases[1].Target.GetLocator());
		
		aliases = await client.FindAliasesAsync("bar");
		Assert.AreEqual(1, aliases.Length);
		Assert.AreEqual(handle2.GetLocator(), aliases[0].Target.GetLocator());
	}
	
	[TestMethod]
	public async Task ForwardDeclareAsync()
	{
		StorageService service = ServiceProvider.GetRequiredService<StorageService>();
		NamespaceId namespaceId = new NamespaceId("memory");
		
		BlobInfo? blobInfo = await service.FindBlobAsync(namespaceId, new BlobLocator("foo"));
		Assert.IsNull(blobInfo);
		
		blobInfo = await service.AddBlobAsync(namespaceId, new BlobLocator("foo"), [new BlobLocator("bar")]);
		Assert.AreEqual(1, blobInfo.Imports?.Count ?? 0);
		Assert.IsFalse(blobInfo.Shadow);
		
		BlobInfo? importInfo = await service.FindBlobAsync(namespaceId, new BlobLocator("bar"));
		Assert.IsNotNull(importInfo);
		Assert.IsTrue(importInfo.Shadow);
		
		BlobInfo importInfo2 = await service.AddBlobAsync(namespaceId, new BlobLocator("bar"), []);
		Assert.AreEqual(importInfo.Id, importInfo2.Id);
		Assert.IsFalse(blobInfo.Shadow);
	}
	
	[TestMethod]
	public async Task AddBlobAsync_DuplicateKey_RaceConditionAsync()
	{
		// Step 1: Create a blob that references another blob, creating a shadow
		BlobLocator mainBlob = new("main");
		BlobLocator referencedBlob = new("referenced");
		
		BlobInfo mainInfo = await Service.AddBlobAsync(_namespaceId, mainBlob, [referencedBlob]);
		Assert.AreEqual(1, mainInfo.Imports?.Count ?? 0);
		Assert.IsFalse(mainInfo.Shadow);
		
		// Verify the shadow blob was created
		BlobInfo? shadowInfo = await Service.FindBlobAsync(_namespaceId, referencedBlob);
		Assert.IsNotNull(shadowInfo);
		Assert.IsTrue(shadowInfo.Shadow);
		
		// Step 2: Convert the shadow blob to a real blob (simulating a normal upload)
		BlobInfo realBlobInfo = await Service.AddBlobAsync(_namespaceId, referencedBlob, []);
		Assert.AreEqual(shadowInfo.Id, realBlobInfo.Id);
		Assert.IsFalse(realBlobInfo.Shadow);
		
		// Step 3: Try to add the same blob again (this would cause the duplicate key error)
		// This simulates a race condition where multiple processes try to upload the same blob
		BlobInfo duplicateAttempt = await Service.AddBlobAsync(_namespaceId, referencedBlob, []);
		
		// Verify it returns the same blob without error
		Assert.AreEqual(realBlobInfo.Id, duplicateAttempt.Id);
		Assert.IsFalse(duplicateAttempt.Shadow);
	}
	
	[TestMethod]
	public async Task AddBlobAsync_MultipleConcurrentShadowConversionsAsync()
	{
		// Step 1: Create multiple blobs that all reference the same shadow blob
		BlobLocator sharedDependency = new("shared-dependency");
		
		for (int i = 0; i < 5; i++)
		{
			BlobLocator parentBlob = new ($"parent-{i}");
			await Service.AddBlobAsync(_namespaceId, parentBlob, [sharedDependency]);
		}
		
		// Verify shadow blob exists
		BlobInfo? shadowInfo = await Service.FindBlobAsync(_namespaceId, sharedDependency);
		Assert.IsNotNull(shadowInfo);
		Assert.IsTrue(shadowInfo.Shadow);
		
		// Step 2: Simulate multiple concurrent attempts to convert the shadow to a real blob
		// This would happen in a distributed system where multiple workers process the upload
		Task<BlobInfo>[] tasks = new Task<BlobInfo>[10];
		for (int i = 0; i < tasks.Length; i++)
		{
			tasks[i] = Service.AddBlobAsync(_namespaceId, sharedDependency, []);
		}
		
		// All tasks should complete without throwing duplicate key errors
		BlobInfo[] results = await Task.WhenAll(tasks);
		
		// Verify all results point to the same blob
		ObjectId expectedId = results[0].Id;
		foreach (BlobInfo result in results)
		{
			Assert.AreEqual(expectedId, result.Id);
			Assert.IsFalse(result.Shadow);
		}
		
		// Verify there's still only one blob in the database
		BlobInfo? finalBlob = await Service.FindBlobAsync(_namespaceId, sharedDependency);
		Assert.IsNotNull(finalBlob);
		Assert.AreEqual(expectedId, finalBlob.Id);
		Assert.IsFalse(finalBlob.Shadow);
	}
	
	[TestMethod]
	public async Task AddBlobAsync_UpdateExistingBlobWithNewImportsAsync()
	{
		BlobLocator targetBlob = new("target");
		BlobLocator import1 = new("import1");
		BlobLocator import2 = new("import2");
		
		// Step 1: Add blob with one import
		BlobInfo firstAdd = await Service.AddBlobAsync(_namespaceId, targetBlob, [import1]);
		Assert.AreEqual(1, firstAdd.Imports?.Count ?? 0);
		Assert.IsFalse(firstAdd.Shadow);
		
		// Step 2: Try to add the same blob with different imports
		// This tests whether the system handles updates to existing blobs properly
		BlobInfo secondAdd = await Service.AddBlobAsync(_namespaceId, targetBlob, [import1, import2]);
		
		// The blob should be updated with new imports
		Assert.AreEqual(firstAdd.Id, secondAdd.Id);
		Assert.AreEqual(2, secondAdd.Imports?.Count ?? 0);
		Assert.IsFalse(secondAdd.Shadow);
	}
}