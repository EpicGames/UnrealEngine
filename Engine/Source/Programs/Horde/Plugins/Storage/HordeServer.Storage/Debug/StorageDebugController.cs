// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Storage;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Debug
{
	/// <summary>
	/// Debug functionality for the storage system
	/// </summary>
	[ApiController]
	[Authorize]
	[DebugEndpoint]
	[Tags("Debug")]
	public class StorageDebugController : HordeControllerBase
	{
		readonly IServiceProvider _serviceProvider;
		readonly IOptionsSnapshot<StorageConfig> _storageConfig;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageDebugController(IServiceProvider serviceProvider, IOptionsSnapshot<StorageConfig> storageConfig, ILogger<StorageDebugController> logger)
		{
			_serviceProvider = serviceProvider;
			_storageConfig = storageConfig;
			_logger = logger;
		}

		/// <summary>
		/// Writes stats for the storage backend cache
		/// </summary>
		[HttpPost]
		[Route("/api/v1/debug/writecacherefstats")]
		public ActionResult WriteCacheRefStats()
		{
			if (!_storageConfig.Value.Acl.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			StorageBackendCache storageBackendCache = _serviceProvider.GetRequiredService<StorageBackendCache>();
			storageBackendCache.WriteRefStats(_logger);
			return Ok();
		}
	}
}
