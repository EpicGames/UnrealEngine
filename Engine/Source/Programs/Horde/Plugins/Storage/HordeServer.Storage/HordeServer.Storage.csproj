<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
  </PropertyGroup>

  <ItemGroup>
    <FrameworkReference Include="Microsoft.AspNetCore.App" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.S3" Version="3.7.304.1" />
    <PackageReference Include="AWSSDK.Extensions.NETCore.Setup" Version="3.7.300" />
    <PackageReference Include="Azure.Storage.Blobs" Version="12.19.1" />
    <PackageReference Include="Google.Cloud.Storage.V1" Version="4.10.0" />
    <PackageReference Include="MongoDB.Driver" Version="2.22.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
  </ItemGroup>

</Project>
