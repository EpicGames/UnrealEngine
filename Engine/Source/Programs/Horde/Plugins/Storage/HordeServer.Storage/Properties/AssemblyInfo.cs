// Copyright Epic Games, Inc. All Rights Reserved.

using System.Resources;
using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("HordeServer.Storage.Tests")]
[assembly: NeutralResourcesLanguage("en-US")]
