// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Serialization;

namespace HordeServer.Storage
{
	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<BackendId, BackendIdConverter>))]
	[StringIdConverter(typeof(BackendIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<BackendId, BackendIdConverter>))]
	public record struct BackendId(StringId Id)
	{
		/// <summary>
		/// Empty backend id
		/// </summary>
		public static BackendId Empty { get; } = default;

		/// <summary>
		/// Constructor
		/// </summary>
		public BackendId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class BackendIdConverter : StringIdConverter<BackendId>
	{
		/// <inheritdoc/>
		public override BackendId FromStringId(StringId id) => new BackendId(id);

		/// <inheritdoc/>
		public override StringId ToStringId(BackendId value) => value.Id;
	}
}
