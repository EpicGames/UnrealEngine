// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeServer.Server;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Storage
{
	/// <summary>
	/// Entry for a single collection's stats within a snapshot
	/// </summary>
	public class CollectionStatEntry
	{
		/// <summary>
		/// Name of the MongoDB collection
		/// </summary>
		[BsonElement("name")]
		public string CollectionName { get; set; } = string.Empty;

		/// <summary>
		/// Number of documents in the collection
		/// </summary>
		[BsonElement("cnt")]
		public long DocumentCount { get; set; }

		/// <summary>
		/// Total data size in bytes
		/// </summary>
		[BsonElement("dsz")]
		public long DataSizeBytes { get; set; }

		/// <summary>
		/// Total index size in bytes
		/// </summary>
		[BsonElement("isz")]
		public long IndexSizeBytes { get; set; }

		/// <summary>
		/// Total storage size in bytes
		/// </summary>
		[BsonElement("ssz")]
		public long StorageSizeBytes { get; set; }

		/// <summary>
		/// Average document size in bytes
		/// </summary>
		[BsonElement("asz")]
		public long AvgDocSizeBytes { get; set; }

		/// <summary>
		/// Ratio of storage size to data size indicating fragmentation
		/// </summary>
		[BsonElement("frag")]
		public double FragmentationRatio { get; set; }
	}

	/// <summary>
	/// MongoDB document for a collection stats snapshot
	/// </summary>
	public class CollectionStatsDocument
	{
		/// <summary>
		/// Unique identifier for this document
		/// </summary>
		public ObjectId Id { get; set; }

		/// <summary>
		/// Time this snapshot was captured
		/// </summary>
		[BsonElement("tm")]
		public DateTime TimeUtc { get; set; }

		/// <summary>
		/// Stats for each monitored collection
		/// </summary>
		[BsonElement("cols")]
		public List<CollectionStatEntry> Collections { get; set; } = new();
	}

	/// <summary>
	/// Periodically captures MongoDB collStats for key storage collections
	/// </summary>
	public sealed class CollectionStatsService : IHostedService, IAsyncDisposable
	{
		static TimeSpan RetainDays { get; } = TimeSpan.FromDays(90.0);

		static readonly string[] s_targetCollections = new[]
		{
			"Storage.Blobs",
			"Storage.Refs",
			"Storage.GcMetrics",
			"Storage.CollectionStats",
			"ArtifactsV2",
			"Jobs",
			"LogFiles",
			"LogEvents"
		};

		readonly IMongoCollection<CollectionStatsDocument> _collection;
		readonly IMongoService _mongoService;
		readonly IClock _clock;
		readonly ILogger _logger;

#pragma warning disable CA2213
		readonly ITicker _snapshotTicker;
		readonly ITicker _cleanupTicker;
#pragma warning restore CA2213

		/// <summary>
		/// Constructor
		/// </summary>
		public CollectionStatsService(IMongoService mongoService, IClock clock, ILogger<CollectionStatsService> logger)
		{
			_mongoService = mongoService;
			_clock = clock;
			_logger = logger;

			List<MongoIndex<CollectionStatsDocument>> indexes = new List<MongoIndex<CollectionStatsDocument>>();
			indexes.Add(MongoIndex.Create<CollectionStatsDocument>(x => x.Descending(x => x.TimeUtc)));
			_collection = mongoService.GetCollection<CollectionStatsDocument>("Storage.CollectionStats", indexes);

			_snapshotTicker = clock.AddSharedTicker<CollectionStatsService>(TimeSpan.FromHours(1.0), SnapshotAsync, logger);
			_cleanupTicker = clock.AddSharedTicker($"{nameof(CollectionStatsService)}:Cleanup", TimeSpan.FromHours(24.0), CleanupAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _snapshotTicker.StartAsync();
			await _cleanupTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cleanupTicker.StopAsync();
			await _snapshotTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _cleanupTicker.DisposeAsync();
			await _snapshotTicker.DisposeAsync();
		}

		/// <summary>
		/// Get the most recent collection stats snapshot
		/// </summary>
		public async Task<CollectionStatsDocument?> GetLatestAsync(CancellationToken cancellationToken)
		{
			return await _collection.Find(FilterDefinition<CollectionStatsDocument>.Empty).SortByDescending(x => x.TimeUtc).Limit(1).FirstOrDefaultAsync(cancellationToken);
		}

		/// <summary>
		/// Query historical collection stats snapshots
		/// </summary>
		public async Task<IReadOnlyList<CollectionStatsDocument>> FindAsync(DateTime minTime, DateTime maxTime, int maxResults, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<CollectionStatsDocument> builder = Builders<CollectionStatsDocument>.Filter;
			FilterDefinition<CollectionStatsDocument> filter = builder.Gte(x => x.TimeUtc, minTime) & builder.Lte(x => x.TimeUtc, maxTime);
			return await _collection.Find(filter).SortByDescending(x => x.TimeUtc).Limit(maxResults).ToListAsync(cancellationToken);
		}

		async ValueTask SnapshotAsync(CancellationToken cancellationToken)
		{
			List<CollectionStatEntry> entries = new List<CollectionStatEntry>();

			foreach (string collectionName in s_targetCollections)
			{
				try
				{
					BsonDocument command = new BsonDocument { { "collStats", collectionName } };
					BsonDocument result = await _mongoService.Database.RunCommandAsync<BsonDocument>(command, cancellationToken: cancellationToken);
					long count = result.GetValue("count", 0).ToInt64();
					long dataSize = result.GetValue("size", 0).ToInt64();
					long indexSize = result.GetValue("totalIndexSize", 0).ToInt64();
					long storageSize = result.GetValue("storageSize", 0).ToInt64();
					long avgObjSize = result.GetValue("avgObjSize", 0).ToInt64();
					double fragmentation = dataSize > 0 ? (double)storageSize / dataSize : 0.0;

					entries.Add(new CollectionStatEntry
					{
						CollectionName = collectionName,
						DocumentCount = count,
						DataSizeBytes = dataSize,
						IndexSizeBytes = indexSize,
						StorageSizeBytes = storageSize,
						AvgDocSizeBytes = avgObjSize,
						FragmentationRatio = fragmentation
					});
				}
				catch (MongoCommandException ex) when (ex.CodeName == "NamespaceNotFound")
				{
					_logger.LogDebug("Collection {Name} not found, skipping stats", collectionName);
				}
			}

			CollectionStatsDocument doc = new CollectionStatsDocument
			{
				TimeUtc = _clock.UtcNow,
				Collections = entries
			};
			await _collection.InsertOneAsync(doc, null, cancellationToken);
			_logger.LogInformation("Captured collection stats for {NumCollections} collections", entries.Count);
		}

		async ValueTask CleanupAsync(CancellationToken cancellationToken)
		{
			DateTime baseTime = _clock.UtcNow - RetainDays;
			DeleteResult result = await _collection.DeleteManyAsync(x => x.TimeUtc < baseTime, cancellationToken);
			_logger.LogInformation("Deleted {NumItems} collection stats records", result.DeletedCount);
		}
	}
}
