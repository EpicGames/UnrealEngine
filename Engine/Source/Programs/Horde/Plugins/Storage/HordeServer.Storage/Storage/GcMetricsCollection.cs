// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Storage
{
	/// <summary>
	/// Input record for adding GC metric entries
	/// </summary>
	public record GcMetricEntry
	{
		/// <summary>
		/// Namespace for this GC metric
		/// </summary>
		public NamespaceId NamespaceId { get; init; }

		/// <summary>
		/// Number of blobs deleted during this sweep
		/// </summary>
		public long BlobsDeleted { get; init; }

		/// <summary>
		/// Number of blobs checked during this sweep
		/// </summary>
		public long BlobsChecked { get; init; }

		/// <summary>
		/// Total bytes freed during this sweep
		/// </summary>
		public long BytesFreed { get; init; }

		/// <summary>
		/// Duration of the sweep in milliseconds
		/// </summary>
		public double SweepDurationMs { get; init; }

		/// <summary>
		/// Whether GC was paused due to queue pressure
		/// </summary>
		public bool WasPaused { get; init; }

		/// <summary>
		/// Current depth of the GC queue
		/// </summary>
		public long QueueDepth { get; init; }

		/// <summary>
		/// Number of blobs ingested since last tick
		/// </summary>
		public long BlobsIngested { get; init; }

		/// <summary>
		/// Number of enqueue failures since last tick
		/// </summary>
		public long EnqueueFailures { get; init; }

		/// <summary>
		/// Number of throttle events since last tick
		/// </summary>
		public long ThrottleEvents { get; init; }

		/// <summary>
		/// Number of refs that expired since last tick
		/// </summary>
		public long RefsExpired { get; init; }
	}

	/// <summary>
	/// MongoDB document for GC metric snapshots
	/// </summary>
	public class GcMetricDocument
	{
		/// <summary>
		/// Unique identifier for this document
		/// </summary>
		public ObjectId Id { get; set; }

		/// <summary>
		/// Time this metric was recorded
		/// </summary>
		[BsonElement("tm")]
		public DateTime TimeUtc { get; set; }

		/// <summary>
		/// Namespace for this GC metric
		/// </summary>
		[BsonElement("ns")]
		public NamespaceId NamespaceId { get; set; }

		/// <summary>
		/// Number of blobs deleted
		/// </summary>
		[BsonElement("del")]
		public long BlobsDeleted { get; set; }

		/// <summary>
		/// Number of blobs checked
		/// </summary>
		[BsonElement("chk")]
		public long BlobsChecked { get; set; }

		/// <summary>
		/// Total bytes freed
		/// </summary>
		[BsonElement("freed")]
		public long BytesFreed { get; set; }

		/// <summary>
		/// Duration of the sweep in milliseconds
		/// </summary>
		[BsonElement("sweep_ms")]
		public double SweepDurationMs { get; set; }

		/// <summary>
		/// Whether GC was paused due to queue pressure
		/// </summary>
		[BsonElement("paused")]
		public bool WasPaused { get; set; }

		/// <summary>
		/// Current depth of the GC queue
		/// </summary>
		[BsonElement("qdepth")]
		public long QueueDepth { get; set; }

		/// <summary>
		/// Number of blobs ingested
		/// </summary>
		[BsonElement("ing")]
		public long BlobsIngested { get; set; }

		/// <summary>
		/// Number of enqueue failures
		/// </summary>
		[BsonElement("enq_fail")]
		public long EnqueueFailures { get; set; }

		/// <summary>
		/// Number of throttle events
		/// </summary>
		[BsonElement("throttle")]
		public long ThrottleEvents { get; set; }

		/// <summary>
		/// Number of refs that expired
		/// </summary>
		[BsonElement("refs_exp")]
		public long RefsExpired { get; set; }
	}

	/// <summary>
	/// Stores per-tick GC metrics in MongoDB for dashboard trending
	/// </summary>
	public sealed class GcMetricsCollection : IHostedService, IAsyncDisposable
	{
		static TimeSpan RetainDays { get; } = TimeSpan.FromDays(30.0);

		readonly IMongoCollection<GcMetricDocument> _collection;
		readonly StorageService _storageService;
		readonly IOptionsMonitor<StorageConfig> _storageConfig;

#pragma warning disable CA2213
		readonly MongoBufferedWriter<GcMetricDocument> _writer;
		readonly ITicker _drainTicker;
		readonly ITicker _cleanupTicker;
#pragma warning restore CA2213

		readonly IClock _clock;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public GcMetricsCollection(StorageService storageService, IMongoService mongoService, IOptionsMonitor<StorageConfig> storageConfig, IClock clock, ILogger<GcMetricsCollection> logger)
		{
			_storageService = storageService;
			_storageConfig = storageConfig;
			_clock = clock;
			_logger = logger;

			List<MongoIndex<GcMetricDocument>> indexes = new List<MongoIndex<GcMetricDocument>>();
			indexes.Add(MongoIndex.Create<GcMetricDocument>(x => x.Ascending(x => x.NamespaceId).Descending(x => x.TimeUtc)));
			_collection = mongoService.GetCollection<GcMetricDocument>("Storage.GcMetrics", indexes);

			_writer = new MongoBufferedWriter<GcMetricDocument>(_collection, logger);
			_drainTicker = clock.AddTicker($"{nameof(GcMetricsCollection)}:Drain", TimeSpan.FromMinutes(1.0), DrainAsync, logger);
			_cleanupTicker = clock.AddSharedTicker<GcMetricsCollection>(TimeSpan.FromHours(4.0), CleanupAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _writer.StartAsync();
			await _drainTicker.StartAsync();
			await _cleanupTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cleanupTicker.StopAsync();
			await _drainTicker.StopAsync();
			await _writer.StopAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _cleanupTicker.DisposeAsync();
			await _drainTicker.DisposeAsync();
			await _writer.FlushAsync(CancellationToken.None);
			await _writer.DisposeAsync();
		}

		/// <summary>
		/// Record a GC metric entry (non-blocking buffered write)
		/// </summary>
		public void Add(GcMetricEntry entry)
		{
			GcMetricDocument doc = new GcMetricDocument
			{
				TimeUtc = _clock.UtcNow,
				NamespaceId = entry.NamespaceId,
				BlobsDeleted = entry.BlobsDeleted,
				BlobsChecked = entry.BlobsChecked,
				BytesFreed = entry.BytesFreed,
				SweepDurationMs = entry.SweepDurationMs,
				WasPaused = entry.WasPaused,
				QueueDepth = entry.QueueDepth,
				BlobsIngested = entry.BlobsIngested,
				EnqueueFailures = entry.EnqueueFailures,
				ThrottleEvents = entry.ThrottleEvents,
				RefsExpired = entry.RefsExpired
			};
			_writer.Write(doc);
		}

		/// <summary>
		/// Query GC metrics for dashboard display
		/// </summary>
		public async Task<IReadOnlyList<GcMetricDocument>> FindAsync(NamespaceId? ns, DateTime minTime, DateTime maxTime, int maxResults, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<GcMetricDocument> builder = Builders<GcMetricDocument>.Filter;
			FilterDefinition<GcMetricDocument> filter = builder.Gte(x => x.TimeUtc, minTime) & builder.Lte(x => x.TimeUtc, maxTime);
			if (ns != null)
			{
				filter &= builder.Eq(x => x.NamespaceId, ns.Value);
			}
			return await _collection.Find(filter).SortByDescending(x => x.TimeUtc).Limit(maxResults).ToListAsync(cancellationToken);
		}

		/// <summary>
		/// Flush buffered writes
		/// </summary>
		public ValueTask FlushAsync(CancellationToken cancellationToken) => _writer.FlushAsync(cancellationToken);

		async ValueTask<TimeSpan?> DrainAsync(CancellationToken cancellationToken)
		{
			Dictionary<NamespaceId, (long BlobsDeleted, long BlobsChecked, long BytesFreed, double SweepDurationMs, long BlobsIngested, long EnqueueFailures, long RefsExpired)> counters = _storageService.ReadAndResetGcCounters();
			if (counters.Count == 0)
			{
				return TimeSpan.FromMinutes(1.0);
			}

			(_, Dictionary<NamespaceId, long> queueDepths) = await _storageService.GetGcStatusAsync(cancellationToken);
			long gcQueueLimit = _storageConfig.CurrentValue.GcQueueLimit;

			foreach ((NamespaceId ns, (long blobsDeleted, long blobsChecked, long bytesFreed, double sweepDurationMs, long blobsIngested, long enqueueFailures, long refsExpired)) in counters)
			{
				long queueDepth = queueDepths.GetValueOrDefault(ns);
				Add(new GcMetricEntry
				{
					NamespaceId = ns,
					BlobsDeleted = blobsDeleted,
					BlobsChecked = blobsChecked,
					BytesFreed = bytesFreed,
					SweepDurationMs = sweepDurationMs,
					BlobsIngested = blobsIngested,
					EnqueueFailures = enqueueFailures,
					RefsExpired = refsExpired,
					QueueDepth = queueDepth,
					WasPaused = queueDepth >= gcQueueLimit
				});
			}

			await _writer.FlushAsync(cancellationToken);
			return TimeSpan.FromMinutes(1.0);
		}

		async ValueTask CleanupAsync(CancellationToken cancellationToken)
		{
			DateTime baseTime = _clock.UtcNow - RetainDays;
			DeleteResult result = await _collection.DeleteManyAsync(x => x.TimeUtc < baseTime, cancellationToken);
			_logger.LogInformation("Deleted {NumItems} GC metric records", result.DeletedCount);
		}
	}
}
