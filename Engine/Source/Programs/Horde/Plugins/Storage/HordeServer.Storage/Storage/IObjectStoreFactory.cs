// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Storage;

namespace HordeServer.Storage
{
	/// <summary>
	/// Service which resolves storage backend instances
	/// </summary>
	public interface IObjectStoreFactory
	{
		/// <summary>
		/// Creates a new storage backend with the given identifier
		/// </summary>
		/// <param name="backendConfig">Configuration object for the storage backend</param>
		/// <returns>New backend instance. This instance should be disposed when the client has finished with it.</returns>
		IObjectStore CreateObjectStore(BackendConfig backendConfig);
	}
}
