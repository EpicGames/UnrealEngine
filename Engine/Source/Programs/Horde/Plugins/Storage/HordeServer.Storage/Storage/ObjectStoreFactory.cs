// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.ObjectStores;
using HordeServer.Storage.ObjectStores;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace HordeServer.Storage
{
	/// <summary>
	/// Default implementation of <see cref="IObjectStoreFactory"/>
	/// </summary>
	class ObjectStoreFactory : IObjectStoreFactory
	{
		readonly IServiceProvider _serviceProvider;
		readonly IServerInfo _serverInfo;
		readonly object _lockObject = new object();
		IReadOnlyDictionary<IoHash, IObjectStore> _objectStores = new Dictionary<IoHash, IObjectStore>();
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectStoreFactory(IServiceProvider serviceProvider, IServerInfo serverInfo, ILogger<ObjectStoreFactory> logger)
		{
			_serviceProvider = serviceProvider;
			_serverInfo = serverInfo;
			_logger = logger;
		}

		/// <inheritdoc/>
		public IObjectStore CreateObjectStore(BackendConfig config)
		{
			// See if we've got an existing backend we can use
			IObjectStore? objectStore;
			if (!_objectStores.TryGetValue(config.Hash, out objectStore))
			{
				lock (_lockObject)
				{
					if (!_objectStores.TryGetValue(config.Hash, out objectStore))
					{
						objectStore = CreateObjectStoreInternal(config);

						Dictionary<IoHash, IObjectStore> newObjectStores = new Dictionary<IoHash, IObjectStore>(_objectStores);
						newObjectStores.Add(config.Hash, objectStore);
						_objectStores = newObjectStores;

						_logger.LogInformation("Created object store {Id}@{Hash}", config.Id, config.Hash);
					}
				}
			}

			return objectStore;
		}

		/// <summary>
		/// Creates a storage backend with the given configuration
		/// </summary>
		/// <param name="config">Configuration for the backend</param>
		/// <returns>New storage backend instance</returns>
		IObjectStore CreateObjectStoreInternal(BackendConfig config)
		{
			switch (config.Type ?? StorageBackendType.FileSystem)
			{
				case StorageBackendType.FileSystem:
					return _serviceProvider.GetRequiredService<FileObjectStoreFactory>().CreateStore(DirectoryReference.Combine(_serverInfo.DataDir, config.BaseDir ?? "Storage"));
				case StorageBackendType.Aws:
					return _serviceProvider.GetRequiredService<AwsObjectStoreFactory>().CreateStore(config);
				case StorageBackendType.Azure:
					return _serviceProvider.GetRequiredService<AzureObjectStoreFactory>().CreateStore(config);
				case StorageBackendType.Gcs:
					return _serviceProvider.GetRequiredService<GcsObjectStoreFactory>().CreateStore(config);
				case StorageBackendType.Memory:
					return new MemoryObjectStore();
				default:
					throw new NotImplementedException();
			}
		}
	}
}
