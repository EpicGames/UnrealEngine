// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Net;
using Amazon;
using Amazon.Extensions.NETCore.Setup;
using Amazon.Runtime;
using Amazon.S3;
using Amazon.S3.Model;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using HordeServer.Utilities;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Storage.ObjectStores
{
	/// <summary>
	/// Exception wrapper for S3 requests
	/// </summary>
	public sealed class AwsException : StorageException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message for the exception</param>
		/// <param name="innerException">Inner exception data</param>
		public AwsException(string message, Exception? innerException)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Credentials to use for AWS
	/// </summary>
	public enum AwsCredentialsType
	{
		/// <summary>
		/// Use default credentials from the AWS SDK
		/// </summary>
		Default,

		/// <summary>
		/// Read credentials from the <see cref="IAwsStorageOptions.AwsProfile"/> profile in the AWS config file
		/// </summary>
		Profile,

		/// <summary>
		/// Assume a particular role. Should specify ARN in <see cref="IAwsStorageOptions.AwsRole"/>
		/// </summary>
		AssumeRole,

		/// <summary>
		/// Assume a particular role using the current environment variables.
		/// </summary>
		AssumeRoleWebIdentity,
	}

	/// <summary>
	/// Options for AWS
	/// </summary>
	public interface IAwsStorageOptions
	{
		/// <summary>
		/// Type of credentials to use
		/// </summary>
		public AwsCredentialsType? AwsCredentials { get; }

		/// <summary>
		/// Name of the bucket to use
		/// </summary>
		public string? AwsBucketName { get; }

		/// <summary>
		/// Base path within the bucket 
		/// </summary>
		public string? AwsBucketPath { get; }

		/// <summary>
		/// ARN of a role to assume
		/// </summary>
		public string? AwsRole { get; }

		/// <summary>
		/// The AWS profile to read credentials form
		/// </summary>
		public string? AwsProfile { get; }

		/// <summary>
		/// Region to connect to
		/// </summary>
		public string? AwsRegion { get; }
	}

	/// <summary>
	/// Storage backend using AWS S3
	/// </summary>
	public sealed class AwsObjectStore : IObjectStore
	{
		/// <summary>
		/// S3 Client
		/// </summary>
		private readonly IAmazonS3 _client;

		/// <summary>
		/// Options for AWs
		/// </summary>
		private readonly IAwsStorageOptions _options;

		/// <summary>
		/// Semaphore for connecting to AWS
		/// </summary>
		private readonly SemaphoreSlim _semaphore;

		/// <summary>
		/// Prefix for objects in the bucket
		/// </summary>
		private readonly string _pathPrefix;

		/// <summary>
		/// Logger interface
		/// </summary>
		private readonly ILogger _logger;

		/// <inheritdoc/>
		public bool SupportsRedirects => true;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="client">Client instance</param>
		/// <param name="options">Storage options</param>
		/// <param name="semaphore">Semaphore for making requests</param>
		/// <param name="logger">Logger interface</param>
		public AwsObjectStore(IAmazonS3 client, IAwsStorageOptions options, SemaphoreSlim semaphore, ILogger<AwsObjectStore> logger)
		{
			_client = client;
			_options = options;
			_semaphore = semaphore;
			_logger = logger;

			_pathPrefix = (_options.AwsBucketPath ?? String.Empty).TrimEnd('/');
			if (_pathPrefix.Length > 0)
			{
				_pathPrefix += '/';
			}
		}

		class WrappedResponseStream : Stream
		{
			readonly IDisposable _semaphore;
			readonly TelemetrySpan _semaphoreSpan;
			readonly GetObjectResponse _response;
			readonly Stream _responseStream;

			public WrappedResponseStream(IDisposable semaphore, TelemetrySpan semaphoreSpan, GetObjectResponse response)
			{
				_semaphore = semaphore;
				_semaphoreSpan = semaphoreSpan;
				_response = response;
				_responseStream = response.ResponseStream;
			}

			public override bool CanRead => true;
			public override bool CanSeek => false;
			public override bool CanWrite => false;
			public override long Length => _response.ContentLength;
			public override long Position { get => _responseStream.Position; set => throw new NotSupportedException(); }

			public override void Flush() => _responseStream.Flush();

			public override int Read(byte[] buffer, int offset, int count) => _responseStream.Read(buffer, offset, count);
			public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => _responseStream.ReadAsync(buffer, cancellationToken);
			public override Task<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _responseStream.ReadAsync(buffer, offset, count, cancellationToken);

			public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();
			public override void SetLength(long value) => throw new NotSupportedException();
			public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();

			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				if (disposing)
				{
					_semaphore.Dispose();
					_semaphoreSpan.Dispose();
					_response.Dispose();
					_responseStream.Dispose();
				}
			}

			public override async ValueTask DisposeAsync()
			{
				await base.DisposeAsync();

				_semaphore.Dispose();
				_response.Dispose();
				await _responseStream.DisposeAsync();
			}
		}

		string GetFullPath(ObjectKey key) => $"{_pathPrefix}{key}";

		/// <inheritdoc/>
		public Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken)
		{
			string range;
			if (length == null)
			{
				range = $"bytes={offset}-";
			}
			else if (length.Value == 0)
			{
				throw new ArgumentException("Cannot read empty stream from AWS backend", nameof(length));
			}
			else
			{
				range = $"bytes={offset}-{offset + length.Value - 1}";
			}
			return OpenAsync(key, new ByteRange(range), cancellationToken);
		}

		async Task<Stream> OpenAsync(ObjectKey key, ByteRange? byteRange, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(OpenAsync)}");
			span.SetAttribute("path", key.ToString());

			string fullPath = GetFullPath(key);

			IDisposable? semaLock = null;
			TelemetrySpan? semaphoreSpan = null;
			GetObjectResponse? response = null;
			try
			{
				semaLock = await _semaphore.WaitDisposableAsync(cancellationToken);

				semaphoreSpan = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(OpenAsync)}.Semaphore");
				semaphoreSpan.SetAttribute("path", key.ToString());

				GetObjectRequest newGetRequest = new GetObjectRequest();
				newGetRequest.BucketName = _options.AwsBucketName;
				newGetRequest.Key = fullPath;
				newGetRequest.ByteRange = byteRange;

				try
				{
					response = await _client.GetObjectAsync(newGetRequest, cancellationToken);
				}
				catch (Exception ex) when (ex is not OperationCanceledException)
				{
					_logger.LogInformation(ex, "S3 read of {BucketId} {Key} failed: {Message}", newGetRequest.BucketName, newGetRequest.Key, ex.Message);

					// Temp hack for files losing '.blob' extension
					const string BlobExtension = ".blob";
					if (fullPath.EndsWith(BlobExtension, StringComparison.OrdinalIgnoreCase))
					{
						try
						{
							newGetRequest.Key = fullPath.Substring(0, fullPath.Length - 5);
							_logger.LogInformation("Attempting S3 read of {BucketId} {Key}...", newGetRequest.BucketName, newGetRequest.Key);
							response = await _client.GetObjectAsync(newGetRequest, cancellationToken);
							return new WrappedResponseStream(semaLock, semaphoreSpan, response);
						}
						catch (Exception ex2)
						{
							_logger.LogInformation(ex2, "Alternate S3 read (no extension) of {BucketId} {Key} failed: {Message}", newGetRequest.BucketName, newGetRequest.Key, ex2.Message);
						}
					}

					// Temp hack for case changes with sanitized object keys
					string newFullPath = GetFullPath(ObjectKey.Sanitize(key.Path));
					if (!String.Equals(fullPath, newFullPath, StringComparison.Ordinal))
					{
						try
						{
							newGetRequest.Key = newFullPath;
							_logger.LogInformation("Attempting S3 read of {BucketId} {Key}...", newGetRequest.BucketName, newGetRequest.Key);
							response = await _client.GetObjectAsync(newGetRequest, cancellationToken);
							return new WrappedResponseStream(semaLock, semaphoreSpan, response);
						}
						catch (Exception ex2)
						{
							_logger.LogInformation(ex2, "Alternate S3 read (sanitized path) of {BucketId} {Key} failed: {Message}", newGetRequest.BucketName, newGetRequest.Key, ex2.Message);
						}
					}

					// Temp hack for case changes with sanitized object keys AND no extension
					if (newFullPath.EndsWith(BlobExtension, StringComparison.OrdinalIgnoreCase))
					{
						try
						{
							newGetRequest.Key = newFullPath.Substring(0, newFullPath.Length - 5);
							_logger.LogInformation("Attempting S3 read of {BucketId} {Key}...", newGetRequest.BucketName, newGetRequest.Key);
							response = await _client.GetObjectAsync(newGetRequest, cancellationToken);
							return new WrappedResponseStream(semaLock, semaphoreSpan, response);
						}
						catch (Exception ex2)
						{
							_logger.LogInformation(ex2, "Alternate S3 read (sanitized path, no extension) of {BucketId} {Key} failed: {Message}", newGetRequest.BucketName, newGetRequest.Key, ex2.Message);
						}
					}

					throw;
				}

				return new WrappedResponseStream(semaLock, semaphoreSpan, response);
			}
			catch (Exception ex)
			{
				semaLock?.Dispose();
				response?.Dispose();
				semaphoreSpan?.Dispose();

				if (ex is OperationCanceledException)
				{
					throw;
				}

				_logger.LogWarning(ex, "Unable to read {Path} from S3", fullPath);

				if (ex is AmazonS3Exception s3ex && s3ex.StatusCode == HttpStatusCode.NotFound)
				{
					throw new ObjectNotFoundException(key, $"Object {key} not found in bucket {_options.AwsBucketName}", ex);
				}
				else
				{
					throw new StorageException($"Unable to read {fullPath} from {_options.AwsBucketName}: {ex.Message}", ex);
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken)
		{
			using Stream stream = await OpenAsync(key, offset, length, cancellationToken);
			return ReadOnlyMemoryOwner.Create(await stream.ReadAllBytesAsync(cancellationToken));
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> new ValueTask<Uri?>(GetPresignedUrl(key, HttpVerb.GET));

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> new ValueTask<Uri?>(GetPresignedUrl(key, HttpVerb.PUT));

		/// <summary>
		/// Helper method to generate a presigned URL for a request
		/// </summary>
		Uri? GetPresignedUrl(ObjectKey key, HttpVerb verb)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(GetPresignedUrl)}");
			span.SetAttribute("path", key.ToString());

			string fullPath = GetFullPath(key);

			try
			{
				GetPreSignedUrlRequest newGetRequest = new GetPreSignedUrlRequest();
				newGetRequest.BucketName = _options.AwsBucketName;
				newGetRequest.Key = fullPath;
				newGetRequest.Verb = verb;
				newGetRequest.Expires = DateTime.UtcNow.AddHours(3.0);
				newGetRequest.ResponseHeaderOverrides.CacheControl = "private, max-age=2592000, immutable"; // 30 days

				string url = _client.GetPreSignedURL(newGetRequest);
				_logger.LogDebug("Creating presigned URL for {Verb} to {Path}", verb, fullPath);
				return new Uri(url);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to get presigned url for {Path} from S3", fullPath);
				return null;
			}
		}

		/// <inheritdoc/>
		public async Task WriteAsync(ObjectKey key, Stream inputStream, CancellationToken cancellationToken = default)
		{
			TimeSpan[] retryTimes =
			{
				TimeSpan.FromSeconds(1.0),
				TimeSpan.FromSeconds(5.0),
				TimeSpan.FromSeconds(10.0),
			};

			string fullPath = GetFullPath(key);
			for (int attempt = 0; ; attempt++)
			{
				try
				{
					using IDisposable semaLock = await _semaphore.WaitDisposableAsync(cancellationToken);
					await WriteInternalAsync(fullPath, inputStream, cancellationToken);
					_logger.LogDebug("Written data to {Path}", fullPath);
					break;
				}
				catch (Exception ex) when (ex is not OperationCanceledException)
				{
					_logger.LogError(ex, "Unable to write data to {Path} ({Attempt}/{AttemptCount})", fullPath, attempt + 1, retryTimes.Length + 1);
					if (attempt >= retryTimes.Length)
					{
						throw new AwsException($"Unable to write to bucket {_options.AwsBucketName} path {fullPath}", ex);
					}
				}

				await Task.Delay(retryTimes[attempt], cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task WriteInternalAsync(string fullPath, Stream stream, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(WriteInternalAsync)}");
			span.SetAttribute("path", fullPath);

			const int MinPartSize = 5 * 1024 * 1024;

			long streamLen = stream.Length;
			if (streamLen < MinPartSize)
			{
				// Read the data into memory. Errors with hash value not matching if we don't do this (?)
				byte[] buffer = new byte[streamLen];
				await ReadExactLengthAsync(stream, buffer, (int)streamLen, cancellationToken);

				// Upload it to S3
				using (MemoryStream inputStream = new MemoryStream(buffer))
				{
					PutObjectRequest uploadRequest = new PutObjectRequest();
					uploadRequest.BucketName = _options.AwsBucketName;
					uploadRequest.Key = fullPath;
					uploadRequest.InputStream = inputStream;
					uploadRequest.Metadata.Add("bytes-written", streamLen.ToString(CultureInfo.InvariantCulture));
					await _client.PutObjectAsync(uploadRequest, cancellationToken);
				}
			}
			else
			{
				// Initiate a multi-part upload
				InitiateMultipartUploadRequest initiateRequest = new InitiateMultipartUploadRequest();
				initiateRequest.BucketName = _options.AwsBucketName;
				initiateRequest.Key = fullPath;

				InitiateMultipartUploadResponse initiateResponse = await _client.InitiateMultipartUploadAsync(initiateRequest, cancellationToken);
				try
				{
					// Buffer for reading the data
					byte[] buffer = new byte[MinPartSize];

					// Upload all the parts
					List<PartETag> partTags = new List<PartETag>();
					for (long streamPos = 0; streamPos < streamLen;)
					{
						// Read the next chunk of data into the buffer
						int bufferLen = (int)Math.Min((long)MinPartSize, streamLen - streamPos);
						await ReadExactLengthAsync(stream, buffer, bufferLen, cancellationToken);
						streamPos += bufferLen;

						// Upload the part
						using (MemoryStream inputStream = new MemoryStream(buffer, 0, bufferLen))
						{
							UploadPartRequest partRequest = new UploadPartRequest();
							partRequest.BucketName = _options.AwsBucketName;
							partRequest.Key = fullPath;
							partRequest.UploadId = initiateResponse.UploadId;
							partRequest.InputStream = inputStream;
							partRequest.PartSize = bufferLen;
							partRequest.PartNumber = partTags.Count + 1;
							partRequest.IsLastPart = (streamPos == streamLen);

							UploadPartResponse partResponse = await _client.UploadPartAsync(partRequest, cancellationToken);
							partTags.Add(new PartETag(partResponse.PartNumber, partResponse.ETag));
						}
					}

					// Mark the upload as complete
					CompleteMultipartUploadRequest completeRequest = new CompleteMultipartUploadRequest();
					completeRequest.BucketName = _options.AwsBucketName;
					completeRequest.Key = fullPath;
					completeRequest.UploadId = initiateResponse.UploadId;
					completeRequest.PartETags = partTags;
					await _client.CompleteMultipartUploadAsync(completeRequest, cancellationToken);
				}
				catch
				{
					// Abort the upload
					AbortMultipartUploadRequest abortRequest = new AbortMultipartUploadRequest();
					abortRequest.BucketName = _options.AwsBucketName;
					abortRequest.Key = fullPath;
					abortRequest.UploadId = initiateResponse.UploadId;
					await _client.AbortMultipartUploadAsync(abortRequest, cancellationToken);

					throw;
				}
			}
		}

		/// <summary>
		/// Reads data of an exact length into a stream
		/// </summary>
		/// <param name="stream">The stream to read from</param>
		/// <param name="buffer">The buffer to read into</param>
		/// <param name="length">Length of the data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		static async Task ReadExactLengthAsync(System.IO.Stream stream, byte[] buffer, int length, CancellationToken cancellationToken)
		{
			int bufferPos = 0;
			while (bufferPos < length)
			{
				int bytesRead = await stream.ReadAsync(buffer, bufferPos, length - bufferPos, cancellationToken);
				if (bytesRead == 0)
				{
					throw new InvalidOperationException("Unexpected end of stream");
				}
				bufferPos += bytesRead;
			}
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(DeleteAsync)}");
			span.SetAttribute("path", key.ToString());

			DeleteObjectRequest newDeleteRequest = new DeleteObjectRequest();
			newDeleteRequest.BucketName = _options.AwsBucketName;
			newDeleteRequest.Key = GetFullPath(key);
			await _client.DeleteObjectAsync(newDeleteRequest, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(ExistsAsync)}");
			span.SetAttribute("path", key.ToString());

			try
			{
				GetObjectMetadataRequest request = new GetObjectMetadataRequest();
				request.BucketName = _options.AwsBucketName;
				request.Key = GetFullPath(key);
				await _client.GetObjectMetadataAsync(request, cancellationToken);
				return true;
			}
			catch (AmazonS3Exception ex) when (ex.StatusCode == System.Net.HttpStatusCode.NotFound)
			{
				return false;
			}
		}

		/// <inheritdoc/>
		public async Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsObjectStore)}.{nameof(ExistsAsync)}");
			span.SetAttribute("path", key.ToString());

			try
			{
				GetObjectMetadataRequest request = new GetObjectMetadataRequest();
				request.BucketName = _options.AwsBucketName;
				request.Key = GetFullPath(key);

				GetObjectMetadataResponse response = await _client.GetObjectMetadataAsync(request, cancellationToken);
				return response.ContentLength;
			}
			catch (AmazonS3Exception ex) when (ex.StatusCode == System.Net.HttpStatusCode.NotFound)
			{
				return -1;
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats) { }
	}

	/// <summary>
	/// Factory for constructing <see cref="AwsObjectStore"/> instances
	/// </summary>
	public sealed class AwsObjectStoreFactory : IDisposable
	{
		readonly IConfiguration _configuration;
		readonly SemaphoreSlim _semaphore;
		readonly ILogger<AwsObjectStore> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AwsObjectStoreFactory(IConfiguration configuration, ILogger<AwsObjectStore> logger)
		{
			_configuration = configuration;
			_semaphore = new SemaphoreSlim(16);
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_semaphore.Dispose();
		}

		/// <summary>
		/// Create a new object store with the given configuration
		/// </summary>
		/// <param name="options">Configuration for the store</param>
		public AwsObjectStore CreateStore(IAwsStorageOptions options)
		{
			AWSOptions awsOptions = GetAwsOptions(_configuration, options);

			IAmazonS3 client = awsOptions.CreateServiceClient<IAmazonS3>();
			_logger.LogInformation("Created AWS storage backend for bucket {BucketName} using credentials {Credentials} {CredentialsStr}", options.AwsBucketName, awsOptions.Credentials.GetType(), awsOptions.Credentials.ToString());

			return new AwsObjectStore(client, options, _semaphore, _logger);
		}

		/// <summary>
		/// Gets the AWS options 
		/// </summary>
		/// <param name="configuration">Global configuration object</param>
		/// <param name="options">AWS storage options</param>
		/// <returns></returns>
		static AWSOptions GetAwsOptions(IConfiguration configuration, IAwsStorageOptions options)
		{
			AWSOptions awsOptions = configuration.GetAWSOptions();
			if (options.AwsRegion != null)
			{
				awsOptions.Region = RegionEndpoint.GetBySystemName(options.AwsRegion);
			}

			switch (options.AwsCredentials ?? AwsCredentialsType.Default)
			{
				case AwsCredentialsType.Default:
					// Using the fallback credentials from the AWS SDK, it will pick up credentials through a number of default mechanisms.
					awsOptions.Credentials = FallbackCredentialsFactory.GetCredentials();
					break;
				case AwsCredentialsType.Profile:
					if (options.AwsProfile == null)
					{
						throw new AwsException($"Missing {nameof(IAwsStorageOptions.AwsProfile)} setting for configuring {nameof(AwsObjectStore)}", null);
					}

					(string accessKey, string secretAccessKey, string secretToken) = AwsHelper.ReadAwsCredentials(options.AwsProfile);
					awsOptions.Credentials = new Amazon.SecurityToken.Model.Credentials(accessKey, secretAccessKey, secretToken, DateTime.Now + TimeSpan.FromHours(12));
					break;
				case AwsCredentialsType.AssumeRole:
					if (options.AwsRole == null)
					{
						throw new AwsException($"Missing {nameof(IAwsStorageOptions.AwsRole)} setting for configuring {nameof(AwsObjectStore)}", null);
					}
					awsOptions.Credentials = new AssumeRoleAWSCredentials(FallbackCredentialsFactory.GetCredentials(), options.AwsRole, "Horde");
					break;
				case AwsCredentialsType.AssumeRoleWebIdentity:
					awsOptions.Credentials = AssumeRoleWithWebIdentityCredentials.FromEnvironmentVariables();
					break;
			}
			return awsOptions;
		}
	}
}
