// Copyright Epic Games, Inc. All Rights Reserved.

using Azure;
using Azure.Storage.Blobs;
using Azure.Storage.Blobs.Models;
using Azure.Storage.Sas;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Storage.ObjectStores
{
	/// <summary>
	/// Exception wrapper for S3 requests
	/// </summary>
	public sealed class AzureException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message for the exception</param>
		/// <param name="innerException">Inner exception data</param>
		public AzureException(string? message, Exception? innerException)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Options for Azure
	/// </summary>
	public interface IAzureStorageOptions
	{
		/// <summary>
		/// Connection string for Azure
		/// </summary>
		public string? AzureConnectionString { get; }

		/// <summary>
		/// Name of the container
		/// </summary>
		public string? AzureContainerName { get; }
	}

	/// <summary>
	/// Storage backend using Azure
	/// </summary>
	public sealed class AzureObjectStore : IObjectStore
	{
		private readonly ILogger _logger;
		private readonly Tracer _tracer;
		private readonly BlobContainerClient _blobContainer;

		/// <inheritdoc/>
		public bool SupportsRedirects => true;

		/// <summary>
		/// Constructor
		/// </summary>
		public AzureObjectStore(IAzureStorageOptions options, ILogger<AzureObjectStore> logger, Tracer tracer)
		{
			if (options.AzureConnectionString == null)
			{
				throw new AzureException($"Missing {nameof(IAzureStorageOptions.AzureConnectionString)} setting for Azure storage backend", null);
			}
			if (options.AzureContainerName == null)
			{
				throw new AzureException($"Missing {nameof(IAzureStorageOptions.AzureContainerName)} setting for Azure storage backend", null);
			}

			_logger = logger;
			_tracer = tracer;
			_blobContainer = new BlobContainerClient(options.AzureConnectionString, options.AzureContainerName);
		}

		/// <inheritdoc/>
		public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			try
			{
				BlobClient blob = _blobContainer.GetBlobClient(key.ToString());
				Response<BlobDownloadInfo> blobInfo = await blob.DownloadAsync(cancellationToken);
				return blobInfo.Value.Content;
			}
			catch (RequestFailedException ex)
			{
				throw new AzureException($"Unable to read {key} from Azure", ex);
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			using Stream stream = await OpenAsync(key, offset, length, cancellationToken);
			return ReadOnlyMemoryOwner.Create(await stream.ReadAllBytesAsync(cancellationToken));
		}

		/// <inheritdoc/>
		public async Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
		{
			await _blobContainer.CreateIfNotExistsAsync(cancellationToken: cancellationToken);

			_logger.LogDebug("Fetching blob reference with name {ObjectName}", key);
			try
			{
				await _blobContainer.GetBlobClient(key.ToString()).UploadAsync(stream, cancellationToken);
			}
			catch (Exception ex)
			{
				throw new AzureException("Unable to upload blob to Azure", ex);
			}
		}

		/// <inheritdoc/>
		public async Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				return false;
			}

			BlobClient blob = _blobContainer.GetBlobClient(key.ToString());
			return await blob.ExistsAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				return -1;
			}

			BlobClient blob = _blobContainer.GetBlobClient(key.ToString());

			try
			{
				BlobProperties properties = await blob.GetPropertiesAsync(cancellationToken: cancellationToken);
				return properties.ContentLength;
			}
			catch(RequestFailedException)
			{
				return -1;
			}
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} does not exist");
			}

			await _blobContainer.DeleteBlobAsync(key.ToString(), cancellationToken: cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} does not exist");
			}

			return TryGetPresignedUrl(key, BlobSasPermissions.Read);
		}

		/// <inheritdoc/>
		public async ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} does not exist");
			}

			return TryGetPresignedUrl(key, BlobSasPermissions.Write);
		}

		/// <summary>
		/// Helper method to generate a presigned URL for a request
		/// </summary>
		Uri? TryGetPresignedUrl(ObjectKey key, BlobSasPermissions permissions)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan("azure.BuildPresignedUrl").SetAttribute("Path", key.ToString());

			try
			{
				BlobClient blob = _blobContainer.GetBlobClient(key.ToString());
				return blob.GenerateSasUri(permissions, DateTimeOffset.Now.AddHours(1.0));
			}
			catch (RequestFailedException e)
			{
				if (e.Status == 404)
				{
					return null;
				}

				throw;
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
		}
	}

	/// <summary>
	/// Factory for constructing <see cref="AzureObjectStore"/> instances
	/// </summary>
	public sealed class AzureObjectStoreFactory
	{
		readonly Tracer _tracer;
		readonly ILogger<AzureObjectStore> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AzureObjectStoreFactory(Tracer tracer, ILogger<AzureObjectStore> logger)
		{
			_tracer = tracer;
			_logger = logger;
		}

		/// <summary>
		/// Create a new object store with the given configuration
		/// </summary>
		/// <param name="options">Configuration for the store</param>
		public AzureObjectStore CreateStore(IAzureStorageOptions options)
		{
			return new AzureObjectStore(options, _logger, _tracer);
		}
	}
}
