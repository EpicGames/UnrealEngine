// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;

namespace HordeServer.Storage.Storage.ObjectStores
{
	/// <summary>
	/// Object store which allows falling back to reading from a secondary object store if a key is not found in the primary store. Useful for migrations.
	/// </summary>
	class ChainedObjectStore : IObjectStore
	{
		readonly IObjectStore _primary;
		readonly IObjectStore _secondary;

		/// <inheritdoc/>
		public bool SupportsRedirects => _primary.SupportsRedirects;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="primary">The primary object store to read from, and the store to use for all writes</param>
		/// <param name="secondary">The secondary object store</param>
		public ChainedObjectStore(IObjectStore primary, IObjectStore secondary)
		{
			_primary = primary;
			_secondary = secondary;
		}

		/// <inheritdoc/>
		public Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> _primary.DeleteAsync(key, cancellationToken);

		/// <inheritdoc/>
		public async Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> await _primary.ExistsAsync(key, cancellationToken) || await _secondary.ExistsAsync(key, cancellationToken);

		/// <inheritdoc/>
		public Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> _primary.GetSizeAsync(key, cancellationToken);

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
			_primary.GetStats(stats);
			_secondary.GetStats(stats);
		}

		/// <inheritdoc/>
		public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			if (await _primary.ExistsAsync(key, cancellationToken) || !await _secondary.ExistsAsync(key, cancellationToken))
			{
				return await _primary.OpenAsync(key, offset, length, cancellationToken);
			}
			else
			{
				return await _secondary.OpenAsync(key, offset, length, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			if (await _primary.ExistsAsync(key, cancellationToken) || !await _secondary.ExistsAsync(key, cancellationToken))
			{
				return await _primary.ReadAsync(key, offset, length, cancellationToken);
			}
			else
			{
				return await _secondary.ReadAsync(key, offset, length, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			if (await _primary.ExistsAsync(key, cancellationToken) || !await _secondary.ExistsAsync(key, cancellationToken))
			{
				return await _primary.TryGetReadRedirectAsync(key, cancellationToken);
			}
			else
			{
				return await _secondary.TryGetReadRedirectAsync(key, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> _primary.TryGetWriteRedirectAsync(key, cancellationToken);

		/// <inheritdoc/>
		public Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
			=> _primary.WriteAsync(key, stream, cancellationToken);
	}
}
