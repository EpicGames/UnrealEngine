// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http.Headers;
using System.Net.Mime;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Google;
using Google.Apis.Auth.OAuth2;
using Google.Apis.Storage.v1;
using Google.Cloud.Storage.V1;
using HordeServer.Utilities;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;
using Object = Google.Apis.Storage.v1.Data.Object;

namespace HordeServer.Storage.ObjectStores
{
	/// <summary>
	/// Exception wrapper for GCS requests
	/// </summary>
	public sealed class GcsException : StorageException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message for the exception</param>
		/// <param name="innerException">Inner exception data</param>
		public GcsException(string message, Exception? innerException)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Options for GCS
	/// </summary>
	public interface IGcsStorageOptions
	{
		/// <summary>
		/// Name of the GCS bucket to use
		/// </summary>
		public string? GcsBucketName { get; }

		/// <summary>
		/// Base path within the bucket 
		/// </summary>
		public string? GcsBucketPath { get; }
	}

	/// <summary>
	/// Storage backend using GCP GCS
	/// </summary>
	public sealed class GcsObjectStore : IObjectStore, IDisposable
	{
		/// <summary>
		/// GCS Client
		/// </summary>
		private readonly StorageClient _client;

		/// <summary>
		/// Options for GCS
		/// </summary>
		private readonly IGcsStorageOptions _options;

		/// <summary>
		/// Semaphore for connecting to GCS
		/// </summary>
		private readonly SemaphoreSlim _semaphore;

		/// <summary>
		/// Prefix for objects in the bucket
		/// </summary>
		private readonly string _pathPrefix;

		/// <summary>
		/// Logger interface
		/// </summary>
		private readonly ILogger _logger;

		/// <inheritdoc/>
		public bool SupportsRedirects => true;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="options">Storage options</param>
		/// <param name="semaphore">Semaphore</param>
		/// <param name="logger">Logger interface</param>
		public GcsObjectStore(IGcsStorageOptions options, SemaphoreSlim semaphore, ILogger<GcsObjectStore> logger)
		{
			_client = StorageClient.Create();
			_options = options;
			_semaphore = semaphore;
			_logger = logger;

			_pathPrefix = (_options.GcsBucketPath ?? String.Empty).TrimEnd('/');
			if (_pathPrefix.Length > 0)
			{
				_pathPrefix += '/';
			}

			logger.LogInformation("Created GCS storage backend for bucket {BucketName}", options.GcsBucketName);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_client.Dispose();
			_semaphore.Dispose();
		}

		class WrappedResponseStream(IDisposable semaphore, TelemetrySpan semaphoreSpan, MemoryStream stream, Google.Apis.Storage.v1.Data.Object obj)
			: Stream
		{
			readonly Stream _responseStream = stream;

			public override bool CanRead => true;
			public override bool CanSeek => false;
			public override bool CanWrite => false;
			public override long Length
			{
				get
				{
					if (obj.Size != null)
					{
						return (long)obj.Size.Value + Int64.MinValue;
					}

					return -1;
				}
			}

			public override long Position { get => _responseStream.Position; set => throw new NotSupportedException(); }

			public override void Flush() => _responseStream.Flush();

			public override int Read(byte[] buffer, int offset, int count) => _responseStream.Read(buffer, offset, count);
			public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => _responseStream.ReadAsync(buffer, cancellationToken);
			public override Task<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _responseStream.ReadAsync(buffer, offset, count, cancellationToken);

			public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();
			public override void SetLength(long value) => throw new NotSupportedException();
			public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();

			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				if (disposing)
				{
					semaphore.Dispose();
					semaphoreSpan.Dispose();
					_responseStream.Dispose();
				}
			}

			public override async ValueTask DisposeAsync()
			{
				await base.DisposeAsync();

				semaphore.Dispose();
				await _responseStream.DisposeAsync();
			}
		}

		string GetFullPath(ObjectKey key) => $"{_pathPrefix}{key}";

		/// <inheritdoc />
		public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			RangeHeaderValue rangeHeader;
			if (length == null)
			{
				rangeHeader = new RangeHeaderValue(offset, null);
			}
			else if (length.Value == 0)
			{
				throw new ArgumentException("Cannot read empty stream from GCS backend", nameof(length));
			}
			else
			{
				rangeHeader = new RangeHeaderValue(offset, offset + length.Value - 1);
			}

			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(OpenAsync)}");
			string fullPath = GetFullPath(key);
			span.SetAttribute("path", fullPath);
			span.SetAttribute("range", rangeHeader.ToString());
			
			IDisposable? semaLock = null;
			TelemetrySpan? semaphoreSpan = null;
			MemoryStream stream = new MemoryStream();
			try
			{
				semaLock = await _semaphore.WaitDisposableAsync(cancellationToken);

				semaphoreSpan = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(OpenAsync)}.Semaphore");
				semaphoreSpan.SetAttribute("path", fullPath);

				// GCS doesn't give us size for objects when we download them, so we need to fetch the metadata first
				Object obj =
					await _client.GetObjectAsync(_options.GcsBucketName, fullPath, cancellationToken: cancellationToken);
				DownloadObjectOptions downloadOptions = new();
				downloadOptions.Range = rangeHeader;

				await _client.DownloadObjectAsync(obj, stream, downloadOptions, cancellationToken);
				
				// Rewind stream before passing it on
				stream.Position = 0;
				
				return new WrappedResponseStream(semaLock, semaphoreSpan, stream, obj);
			}
			catch (Exception ex)
			{
				semaLock?.Dispose();
				semaphoreSpan?.Dispose();

				if (ex is OperationCanceledException)
				{
					throw;
				}

				_logger.LogWarning(ex, "Unable to read {Path} from GCS", fullPath);
				throw new StorageException($"Unable to read {fullPath} from {_options.GcsBucketName}", ex);
			}					}

		/// <inheritdoc />
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			await using Stream stream = await OpenAsync(key, offset, length, cancellationToken);
			return ReadOnlyMemoryOwner.Create(await stream.ReadAllBytesAsync(cancellationToken));
		}

		/// <inheritdoc />
		public async Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
		{
			TimeSpan[] retryTimes =
			{
				TimeSpan.FromSeconds(1.0),
				TimeSpan.FromSeconds(5.0),
				TimeSpan.FromSeconds(10.0),
			};

			string fullPath = GetFullPath(key);
			for (int attempt = 0; ; attempt++)
			{
				try
				{
					using IDisposable semaLock = await _semaphore.WaitDisposableAsync(cancellationToken);
					ObjectsResource.InsertMediaUpload uploader = _client.CreateObjectUploader(_options.GcsBucketName, fullPath, MediaTypeNames.Application.Octet, stream);
					await uploader.UploadAsync(cancellationToken);
					_logger.LogDebug("Written data to {Path}", fullPath);
					break;
				}
				catch (Exception ex) when (ex is not OperationCanceledException)
				{
					_logger.LogError(ex, "Unable to write data to {Path} ({Attempt}/{AttemptCount})", fullPath, attempt + 1, retryTimes.Length + 1);
					if (attempt >= retryTimes.Length)
					{
						throw new GcsException($"Unable to write to bucket {_options.GcsBucketName} path {fullPath}", ex);
					}
				}

				await Task.Delay(retryTimes[attempt], cancellationToken);
			}		
		}

		/// <inheritdoc />
		public async Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			string fullPath = GetFullPath(key);
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(ExistsAsync)}");
			span.SetAttribute("path", fullPath);

			try
			{
				await _client.GetObjectAsync(_options.GcsBucketName, fullPath, cancellationToken: cancellationToken);
				return true;
			}
			catch (GoogleApiException ex) when (ex.HttpStatusCode == System.Net.HttpStatusCode.NotFound)
			{
				return false;
			}
		}

		/// <inheritdoc />
		public async Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			string fullPath = GetFullPath(key);
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(GetSizeAsync)}");
			span.SetAttribute("path", fullPath);

			try
			{
				Object? obj = await _client.GetObjectAsync(_options.GcsBucketName, fullPath, cancellationToken: cancellationToken);
				if (obj.Size != null)
				{
					return (long)obj.Size.Value;
				}
				throw new GcsException("GCS did not return size of object", null);
			}
			catch (GoogleApiException ex) when (ex.HttpStatusCode == System.Net.HttpStatusCode.NotFound)
			{
				return -1;
			}
		}

		/// <inheritdoc />
		public async Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			string fullPath = GetFullPath(key);
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(DeleteAsync)}");
			span.SetAttribute("path", fullPath);

			await _client.DeleteObjectAsync(_options.GcsBucketName, fullPath , cancellationToken: cancellationToken);
		}

		/// <inheritdoc />
		public async ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			return await GetPresignedUrlAsync(key, HttpMethod.Get, cancellationToken);
		}

		/// <inheritdoc />
		public async ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			return await GetPresignedUrlAsync(key, HttpMethod.Put, cancellationToken);
		}
		/// <inheritdoc/>

		/// <summary>
		/// Helper method to generate a presigned URL for a request
		/// </summary>
		async Task<Uri?> GetPresignedUrlAsync(ObjectKey key, HttpMethod method, CancellationToken cancellationToken = default)
		{
			string fullPath = GetFullPath(key);

			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(GcsObjectStore)}.{nameof(GetPresignedUrlAsync)}");
			span.SetAttribute("path", fullPath);

			try
			{
				UrlSigner urlSigner =
					UrlSigner.FromCredential(await GoogleCredential.GetApplicationDefaultAsync(cancellationToken: cancellationToken));
				string url = await urlSigner.SignAsync(_options.GcsBucketName, fullPath, TimeSpan.FromHours(3), method,
					cancellationToken: cancellationToken);
				return new Uri(url);
			}
			catch (HttpRequestException ex)
			{
				_logger.LogWarning(ex, "Unable to get pre-signed url for {Bucket}:{Path} from GCS. {StatusCode}: {Message}", 
					_options.GcsBucketName, fullPath, ex.StatusCode, ex.Message);
				return null;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to get pre-signed url for {Bucket}:{Path} from GCS", _options.GcsBucketName, fullPath);
				return null;
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats) { }
	}
	
	/// <summary>
	/// Factory for constructing <see cref="GcsObjectStore"/> instances
	/// </summary>
	public sealed class GcsObjectStoreFactory : IDisposable
	{
		readonly SemaphoreSlim _semaphore;
		readonly ILogger<GcsObjectStore> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public GcsObjectStoreFactory(IConfiguration _, ILogger<GcsObjectStore> logger)
		{
			_semaphore = new SemaphoreSlim(16);
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_semaphore.Dispose();
		}

		/// <summary>
		/// Create a new object store with the given configuration
		/// </summary>
		/// <param name="options">Configuration for the store</param>
		public GcsObjectStore CreateStore(IGcsStorageOptions options)
		{
			_logger.LogInformation("Created Gcs storage backend for bucket {BucketName}", options.GcsBucketName);

			return new GcsObjectStore(options, _semaphore, _logger);
		}
	}
}