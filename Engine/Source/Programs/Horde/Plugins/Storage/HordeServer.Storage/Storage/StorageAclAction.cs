// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Storage
{
	/// <summary>
	/// ACL actions which apply to storage
	/// </summary>
	public static class StorageAclAction
	{
		/// <summary>
		/// Ability to read blobs from the storage service
		/// </summary>
		public static AclAction ReadBlobs { get; } = new AclAction("ReadBlobs");

		/// <summary>
		/// Ability to write blobs to the storage service
		/// </summary>
		public static AclAction WriteBlobs { get; } = new AclAction("WriteBlobs");

		/// <summary>
		/// Ability to read aliases from the storage service
		/// </summary>
		public static AclAction ReadAliases { get; } = new AclAction("ReadAliases");

		/// <summary>
		/// Ability to write aliases to the storage service
		/// </summary>
		public static AclAction WriteAliases { get; } = new AclAction("WriteAliases");

		/// <summary>
		/// Ability to write aliases to the storage service
		/// </summary>
		public static AclAction DeleteAliases { get; } = new AclAction("DeleteAliases");

		/// <summary>
		/// Ability to read refs from the storage service
		/// </summary>
		public static AclAction ReadRefs { get; } = new AclAction("ReadRefs");

		/// <summary>
		/// Ability to write refs to the storage service
		/// </summary>
		public static AclAction WriteRefs { get; } = new AclAction("WriteRefs");

		/// <summary>
		/// Ability to delete refs
		/// </summary>
		public static AclAction DeleteRefs { get; } = new AclAction("DeleteRefs");
	}
}
