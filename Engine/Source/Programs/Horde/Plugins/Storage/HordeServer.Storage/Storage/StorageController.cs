// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Net.Http.Headers;
using System.Security.Claims;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Serialization;
using HordeServer.Acls;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Storage
{
	/// <summary>
	/// Controller for the /api/v1/storage endpoint
	/// </summary>
	[Authorize]
	[ApiController]
	[Route("[controller]")]
	public class StorageController : HordeControllerBase
	{
		readonly StorageService _storageService;
		readonly IOptionsSnapshot<StorageConfig> _storageConfig;
		readonly IOptions<StorageServerConfig> _staticStorageConfig;
		readonly IAclService _aclService;
		readonly GcMetricsCollection _gcMetrics;
		readonly CollectionStatsService _collectionStats;

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageController(StorageService storageService, IOptionsSnapshot<StorageConfig> storageConfig, IOptions<StorageServerConfig> staticStorageConfig, IAclService aclService, GcMetricsCollection gcMetrics, CollectionStatsService collectionStats)
		{
			_storageService = storageService;
			_storageConfig = storageConfig;
			_staticStorageConfig = staticStorageConfig;
			_aclService = aclService;
			_gcMetrics = gcMetrics;
			_collectionStats = collectionStats;
		}

		bool Authorize(NamespaceId namespaceId, AclAction action)
		{
			return _storageConfig.Value.TryGetNamespace(namespaceId, out NamespaceConfig? namespaceConfig) && namespaceConfig.Authorize(action, User);
		}

		bool AuthorizePaths(NamespaceId namespaceId, AclAction action, IEnumerable<string> paths)
		{
			if (!paths.Any())
			{
				return true;
			}
			if (Authorize(namespaceId, action))
			{
				return true;
			}
			if (paths.All(x => HasPathClaim(User, HordeClaimTypes.WriteNamespace, namespaceId, x)))
			{
				return true;
			}
			return false;
		}

		/// <summary>
		/// Update metadata in the storage service
		/// </summary>
		/// <param name="namespaceId">Namespace to write to</param>
		/// <param name="request">Request for the alias  to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPost]
		[Route("/api/v1/storage/{namespaceId}")]
		public async Task<ActionResult> UpdateNamespaceAsync(NamespaceId namespaceId, [FromBody] UpdateMetadataRequest request, CancellationToken cancellationToken)
		{
			IStorageBackend? backend = _storageService.TryCreateBackend(namespaceId);
			if (backend == null)
			{
				return NotFound(namespaceId);
			}

			// Authorize access to the target paths
			if (!AuthorizePaths(namespaceId, StorageAclAction.WriteAliases, request.AddAliases.Select(x => x.Target.ToString())))
			{
				return Forbid(StorageAclAction.WriteAliases, namespaceId);
			}
			if (!AuthorizePaths(namespaceId, StorageAclAction.DeleteAliases, request.RemoveAliases.Select(x => x.Target.ToString())))
			{
				return Forbid(StorageAclAction.DeleteAliases, namespaceId);
			}
			if (!AuthorizePaths(namespaceId, StorageAclAction.WriteRefs, request.AddRefs.Select(x => x.RefName.ToString())))
			{
				return Forbid(StorageAclAction.WriteRefs, namespaceId);
			}
			if (!AuthorizePaths(namespaceId, StorageAclAction.DeleteRefs, request.RemoveRefs.Select(x => x.RefName.ToString())))
			{
				return Forbid(StorageAclAction.DeleteRefs, namespaceId);
			}

			// Execute the requests
			await backend.UpdateMetadataAsync(request, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Uploads data to the storage service. 
		/// </summary>
		/// <param name="namespaceId">Namespace to fetch from</param>
		/// <param name="request">Information about the blob to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPost]
		[Route("/api/v1/storage/{namespaceId}/blobs")]
		public async Task<ActionResult<WriteBlobResponse>> WriteBlobAsync(NamespaceId namespaceId, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			IStorageBackend? storageBackend = _storageService.TryCreateBackend(namespaceId);
			if (storageBackend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.WriteBlobs) && !HasPathClaim(User, HordeClaimTypes.WriteNamespace, namespaceId, request.Prefix ?? String.Empty))
			{
				return Forbid(StorageAclAction.WriteBlobs, namespaceId);
			}

			return await WriteBlobAsync(storageBackend, null, request, cancellationToken);
		}

		/// <summary>
		/// Uploads data to the storage service using a client-determined path.
		/// </summary>
		/// <param name="namespaceId">Namespace to fetch from</param>
		/// <param name="request">Information about the blob to write</param>
		/// <param name="locator">Location for the blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPut]
		[Route("/api/v1/storage/{namespaceId}/blobs/{*locator}")]
		public async Task<ActionResult<WriteBlobResponse>> WriteBlobAsync(NamespaceId namespaceId, BlobLocator locator, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			IStorageBackend? storageBackend = _storageService.TryCreateBackend(namespaceId);
			if (storageBackend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.WriteBlobs) && !HasPathClaim(User, HordeClaimTypes.WriteNamespace, namespaceId, GetPathFromLocator(locator)))
			{
				return Forbid(StorageAclAction.WriteBlobs, namespaceId);
			}

			return await WriteBlobAsync(storageBackend, locator, request, cancellationToken);
		}

		static string GetPathFromLocator(BlobLocator locator)
		{
			int lastIdx = locator.Path.LastIndexOf('/');
			if (lastIdx == -1)
			{
				return String.Empty;
			}
			else
			{
				return locator.Path.Substring(0, lastIdx).ToString();
			}
		}

		/// <summary>
		/// Writes a blob to storage. Exposed as a public utility method to allow other routes with their own authentication methods to wrap their own authentication/redirection.
		/// </summary>
		/// <param name="storageBackend">The backend to write to</param>
		/// <param name="locator">Locator for the blob to write</param>
		/// <param name="request">Information about the blob to write</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Information about the written blob, or redirect information</returns>
		public static async Task<ActionResult<WriteBlobResponse>> WriteBlobAsync(IStorageBackend storageBackend, BlobLocator? locator, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			IReadOnlyCollection<BlobLocator> imports = request.Imports ?? (IReadOnlyCollection<BlobLocator>)Array.Empty<BlobLocator>();
			if (request.File == null)
			{
				if (locator == null)
				{
					(BlobLocator Locator, Uri UploadUrl)? result = await storageBackend.TryGetBlobWriteRedirectAsync(imports, request.Prefix ?? String.Empty, cancellationToken);
					if (result == null)
					{
						return new WriteBlobResponse { SupportsRedirects = false };
					}
					else
					{
						return new WriteBlobResponse { Blob = result.Value.Locator.Path.ToString(), UploadUrl = result.Value.UploadUrl };
					}
				}
				else
				{
					Uri? uploadUrl = await storageBackend.TryGetBlobWriteRedirectAsync(locator.Value, imports, cancellationToken);
					if (uploadUrl == null)
					{
						return new WriteBlobResponse { SupportsRedirects = false };
					}
					else
					{
						return new WriteBlobResponse { Blob = locator.Value.Path.ToString(), UploadUrl = uploadUrl };
					}
				}
			}
			else
			{
				using Stream stream = request.File.OpenReadStream();
				if (locator == null)
				{
					locator = await storageBackend.WriteBlobAsync(stream, imports, request.Prefix, cancellationToken);
				}
				else
				{
					await storageBackend.WriteBlobAsync(locator.Value, stream, imports, cancellationToken);
				}
				return new WriteBlobResponse { Blob = locator.ToString()!, SupportsRedirects = storageBackend.SupportsRedirects };
			}
		}

		/// <summary>
		/// Retrieves data from the storage service. 
		/// </summary>
		/// <param name="namespaceId">Namespace to fetch from</param>
		/// <param name="locator">Bundle to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Route("/api/v1/storage/{namespaceId}/blobs/{*locator}")]
		public async Task<ActionResult> ReadBlobAsync(NamespaceId namespaceId, BlobLocator locator, CancellationToken cancellationToken = default)
		{
			IStorageBackend? backend = _storageService.TryCreateBackend(namespaceId);
			if (backend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.ReadBlobs) && !HasPathClaim(User, HordeClaimTypes.ReadNamespace, namespaceId, locator.Path.ToString()))
			{
				return Forbid(StorageAclAction.ReadBlobs, namespaceId);
			}

			return await ReadBlobInternalAsync(backend, locator, Request.Headers, cancellationToken);
		}

		/// <summary>
		/// Reads a blob from storage, without performing namespace access checks.
		/// </summary>
		public static async Task<ActionResult> ReadBlobInternalAsync(IStorageBackend storageBackend, BlobLocator locator, IHeaderDictionary headers, CancellationToken cancellationToken)
		{
			Uri? redirectUrl = await storageBackend.TryGetBlobReadRedirectAsync(locator, cancellationToken);
			if (redirectUrl != null)
			{
				return new RedirectResult(redirectUrl.ToString());
			}

			// Parse the range header
			int offset = 0;
			int? length = null;

			if (headers.Range.Count > 0)
			{
				if (headers.Range.Count > 1)
				{
					return new BadRequestObjectResult(LogEvent.Create(LogLevel.Error, "Unsupported range header; only one range is allowed"));
				}

				string? value = headers.Range[0];
				if (value == null)
				{
					return new BadRequestObjectResult(LogEvent.Create(LogLevel.Error, "Unsupported range header; only one range is allowed"));
				}

				Match match = Regex.Match(value, @"^\s*bytes\s*=\s*(\d*)-(\d*)$");
				if (!match.Success)
				{
					return new BadRequestObjectResult(LogEvent.Create(LogLevel.Error, "Unsupported range header syntax; cannot parse {Value}", value));
				}

				if (match.Groups[1].Length > 0 && !Int32.TryParse(match.Groups[1].Value, out offset))
				{
					return new BadRequestObjectResult(LogEvent.Create(LogLevel.Error, "Unable to parse start for range: {Value}", value));
				}
				if (match.Groups[2].Length > 0)
				{
					int end;
					if (Int32.TryParse(match.Groups[2].Value, out end) && end > offset)
					{
						length = (end + 1) - offset;
					}
					else
					{
						return new BadRequestObjectResult(LogEvent.Create(LogLevel.Error, "Unable to parse end for range: {Value}", value));
					}
				}
			}

#pragma warning disable CA2000 // Dispose objects before losing scope
			Stream stream = await storageBackend.OpenBlobAsync(locator, offset, length, cancellationToken);
			return new FileStreamResult(stream, "application/octet-stream");
#pragma warning restore CA2000 // Dispose objects before losing scope
		}

		/// <summary>
		/// Retrieves data from the storage service. 
		/// </summary>
		/// <param name="namespaceId">Namespace to fetch from</param>
		/// <param name="alias">Alias of the node to find</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Route("/api/v1/storage/{namespaceId}/nodes")]
		public async Task<ActionResult<FindNodesResponse>> FindNodesAsync(NamespaceId namespaceId, [FromQuery] string alias, [FromQuery] int? maxResults = null, CancellationToken cancellationToken = default)
		{
			IStorageBackend? backend = _storageService.TryCreateBackend(namespaceId);
			if (backend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.ReadBlobs))
			{
				return Forbid(StorageAclAction.ReadBlobs, namespaceId);
			}

			BlobAliasLocator[] aliases = await backend.FindAliasesAsync(alias, maxResults, cancellationToken);

			FindNodesResponse response = new FindNodesResponse();
			response.Nodes.AddRange(aliases.Select(x => new FindNodeResponse(x.Target, x.Rank, x.Data.ToArray())));

			if (response.Nodes.Count == 0)
			{
				return NotFound();
			}

			return response;
		}

		/// <summary>
		/// Deletes a ref from the storage service.
		/// </summary>
		/// <param name="namespaceId">Namespace to write to</param>
		/// <param name="refName">Name of the ref</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpDelete]
		[Route("/api/v1/storage/{namespaceId}/refs/{*refName}")]
		public async Task<ActionResult> WriteRefAsync(NamespaceId namespaceId, RefName refName, CancellationToken cancellationToken)
		{
			IStorageBackend? backend = _storageService.TryCreateBackend(namespaceId);
			if (backend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.WriteRefs) && !HasPathClaim(User, HordeClaimTypes.WriteNamespace, namespaceId, refName.ToString()))
			{
				return Forbid(StorageAclAction.WriteRefs, namespaceId);
			}

			RemoveRefRequest removeRef = new RemoveRefRequest { RefName = refName };
			await backend.UpdateMetadataAsync(new UpdateMetadataRequest { RemoveRefs = [removeRef] }, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Writes a ref to the storage service.
		/// </summary>
		/// <param name="namespaceId">Namespace to write to</param>
		/// <param name="refName">Name of the ref</param>
		/// <param name="request">Request for the ref to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPut]
		[Route("/api/v1/storage/{namespaceId}/refs/{*refName}")]
		public async Task<ActionResult> WriteRefAsync(NamespaceId namespaceId, RefName refName, [FromBody] WriteRefRequest request, CancellationToken cancellationToken)
		{
			IStorageBackend? backend = _storageService.TryCreateBackend(namespaceId);
			if (backend == null)
			{
				return NotFound(namespaceId);
			}
			if (!Authorize(namespaceId, StorageAclAction.WriteRefs) && !HasPathClaim(User, HordeClaimTypes.WriteNamespace, namespaceId, refName.ToString()))
			{
				return Forbid(StorageAclAction.WriteRefs, namespaceId);
			}

#pragma warning disable CS0618 // Type or member is obsolete
			if (request.Blob != null && request.ExportIdx != null)
			{
				request.Target = new BlobLocator($"{request.Blob.Value}#{request.ExportIdx.Value}");
			}
#pragma warning restore CS0618 // Type or member is obsolete

			AddRefRequest addRef = new AddRefRequest { RefName = refName, Hash = request.Hash, Target = request.Target, Data = request.Data, Options = request.Options };
			await backend.UpdateMetadataAsync(new UpdateMetadataRequest { AddRefs = [addRef] }, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Retrieves a ref from the storage service. 
		/// </summary>
		/// <param name="namespaceId"></param>
		/// <param name="refName"></param>
		/// <param name="cancellationToken"></param>
		[HttpGet]
		[Route("/api/v1/storage/{namespaceId}/refs/{*refName}")]
		public async Task<ActionResult<ReadRefResponse>> ReadRefAsync(NamespaceId namespaceId, RefName refName, CancellationToken cancellationToken)
		{
			NamespaceConfig? namespaceConfig;
			if (!_storageConfig.Value.TryGetNamespace(namespaceId, out namespaceConfig))
			{
				return NotFound(namespaceId);
			}
			if (!namespaceConfig.Authorize(StorageAclAction.ReadRefs, User) && !HasPathClaim(User, HordeClaimTypes.ReadNamespace, namespaceId, refName.ToString()))
			{
				return Forbid(StorageAclAction.ReadRefs, namespaceId);
			}

			return await ReadRefInternalAsync(_storageService, $"/api/v1/storage/{namespaceId}", namespaceId, refName, Request.Headers, cancellationToken);
		}

		/// <summary>
		/// Reads a ref from storage, without performing namespace access checks.
		/// </summary>
		public static async Task<ActionResult<ReadRefResponse>> ReadRefInternalAsync(IStorageClient storageClient, string basePath, NamespaceId namespaceId, RefName refName, IHeaderDictionary headers, CancellationToken cancellationToken)
		{
			IStorageNamespace storageNamespace = storageClient.GetNamespace(namespaceId);

			RefCacheTime cacheTime = new RefCacheTime();
			foreach (string? entry in headers.CacheControl)
			{
				if (entry != null && CacheControlHeaderValue.TryParse(entry, out CacheControlHeaderValue? value) && value?.MaxAge != null)
				{
					cacheTime = new RefCacheTime(value.MaxAge.Value);
				}
			}

			IHashedBlobRef? target = await storageNamespace.TryReadRefAsync(refName, cacheTime, cancellationToken: cancellationToken);
			if (target == null)
			{
				return new NotFoundResult();
			}

			return new ReadRefResponse { Hash = target.Hash, Target = target.GetLocator(), Link = GetNodeLink(namespaceId, target), BasePath = basePath };
		}

		/// <summary>
		/// Checks whether the user has an explicit claim to read or write to a path within a namespace
		/// </summary>
		/// <param name="user">User to query</param>
		/// <param name="claimType">The claim name</param>
		/// <param name="namespaceId">Namespace id to check for</param>
		/// <param name="entity">Path to the entity to query</param>
		/// <returns>True if the user is authorized for access to the given path</returns>
		static bool HasPathClaim(ClaimsPrincipal user, string claimType, NamespaceId namespaceId, string entity)
		{
			foreach (Claim claim in user.Claims)
			{
				if (claim.Type.Equals(claimType, StringComparison.Ordinal))
				{
					int colonIdx = claim.Value.IndexOf(':', StringComparison.Ordinal);
					if (colonIdx == -1)
					{
						if (namespaceId.Text.Equals(claim.Value))
						{
							return true;
						}
					}
					else
					{
						if (namespaceId.Text.Equals(claim.Value.AsMemory(0, colonIdx)) && HasPathPrefix(entity, claim.Value.AsSpan(colonIdx + 1)))
						{
							return true;
						}
					}
				}
			}
			return false;
		}

		static bool HasPathPrefix(string name, ReadOnlySpan<char> prefix)
		{
			if (name.Length > prefix.Length)
			{
				return name[prefix.Length] == '/' && name.AsSpan(0, prefix.Length).SequenceEqual(prefix);
			}
			else if (name.Length == prefix.Length)
			{
				return name.AsSpan().SequenceEqual(prefix);
			}
			else
			{
				return false;
			}
		}

		static readonly IReadOnlyDictionary<Guid, Type> s_blobGuidToType = GetBlobGuidTypeMap();

		static Dictionary<Guid, Type> GetBlobGuidTypeMap()
		{
			Dictionary<Guid, Type> guidTypeMap = new Dictionary<Guid, Type>();
			guidTypeMap.Add(CbNode.BlobTypeGuid, typeof(CbNode));
			guidTypeMap.Add(LeafChunkedDataNode.BlobTypeGuid, typeof(LeafChunkedDataNode));
			guidTypeMap.Add(InteriorChunkedDataNode.BlobTypeGuid, typeof(InteriorChunkedDataNode));
			guidTypeMap.Add(CommitNode.BlobTypeGuid, typeof(CommitNode));
			guidTypeMap.Add(DdcRefNode.BlobTypeGuid, typeof(DdcRefNode));
			guidTypeMap.Add(DirectoryNode.BlobTypeGuid, typeof(DirectoryNode));
			guidTypeMap.Add(RedirectNode.BlobTypeGuid, typeof(RedirectNode));
			return guidTypeMap;
		}

		/// <summary>
		/// Gets information about a particular bundle in storage
		/// </summary>
		/// <param name="namespaceId">Namespace containing the blob</param>
		/// <param name="locator">Blob identifier</param>
		/// <param name="pkt">Packet string</param>
		/// <param name="exp">Export index</param>
		/// <param name="data">Whether to download the blob data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/storage/{namespaceId}/nodes/{*locator}")]
		public async Task<ActionResult<object>> GetNodeAsync(NamespaceId namespaceId, BlobLocator locator, [FromQuery] string? pkt = null, [FromQuery] string? exp = null, [FromQuery] bool data = false, CancellationToken cancellationToken = default)
		{
			NamespaceConfig? namespaceConfig;
			if (!_storageConfig.Value.TryGetNamespace(namespaceId, out namespaceConfig))
			{
				return NotFound(namespaceId);
			}
			if (!namespaceConfig.Authorize(StorageAclAction.ReadBlobs, User))
			{
				return Forbid(StorageAclAction.ReadBlobs, namespaceId);
			}

			List<string> fragments = new List<string>();
			if (pkt != null)
			{
				fragments.Add($"pkt={pkt}");
			}
			if (exp != null)
			{
				fragments.Add($"exp={exp}");
			}
			if (fragments.Count > 0)
			{
				locator = new BlobLocator(locator, String.Join("&", fragments));
			}

			IStorageNamespace storageNamespace = _storageService.GetNamespace(namespaceId);

			object content;

			using BlobData blobData = await storageNamespace.CreateBlobRef(locator).ReadBlobDataAsync(cancellationToken);
			if (data)
			{
				ReadOnlyMemoryStream stream = new ReadOnlyMemoryStream(blobData.Data.ToArray());
				return new FileStreamResult(stream, "application/octet-stream");
			}

			if (blobData.Type.Guid == DirectoryNode.BlobTypeGuid)
			{
				DirectoryNode directoryNode = BlobSerializer.Deserialize<DirectoryNode>(blobData);

				List<object> directories = new List<object>();
				foreach ((string name, DirectoryEntry entry) in directoryNode.NameToDirectory)
				{
					directories.Add(new { name = name.ToString(), length = entry.Length, target = GetNodeHandleLink(namespaceId, entry.Handle) });
				}

				List<object> files = new List<object>();
				foreach ((string name, FileEntry entry) in directoryNode.NameToFile)
				{
					files.Add(new { name = name.ToString(), length = entry.Length, flags = entry.Flags, hash = entry.Hash, target = GetNodeHandleLink(namespaceId, entry.Target) });
				}

				content = new { directoryNode.Length, directories, files };
			}
			else if (blobData.Type.Guid == InteriorChunkedDataNode.BlobTypeGuid)
			{
				InteriorChunkedDataNode interiorNode = BlobSerializer.Deserialize<InteriorChunkedDataNode>(blobData);

				List<object> children = new List<object>();
				foreach (ChunkedDataNodeRef nodeRef in interiorNode.Children)
				{
					children.Add(new { nodeRef.Type, nodeRef.Length, hash = nodeRef.Hash, link = GetNodeLink(namespaceId, nodeRef) });
				}

				content = new { children };
			}
			else if (blobData.Type.Guid == CommitNode.BlobTypeGuid)
			{
				CommitNode commitNode = BlobSerializer.Deserialize<CommitNode>(blobData);

				Dictionary<Guid, object>? metadata = null;
				if (commitNode.Metadata.Count > 0)
				{
					metadata = new Dictionary<Guid, object>();
					foreach ((Guid blobGuid, IHashedBlobRef handle) in commitNode.Metadata)
					{
						metadata.Add(blobGuid, GetNodeHandleLink(namespaceId, handle));
					}
				}

				content = new { commitNode.Number, parent = GetNodeHandleLink(namespaceId, commitNode.Parent), commitNode.Author, commitNode.AuthorId, commitNode.Committer, commitNode.CommitterId, commitNode.Message, commitNode.Time, contents = GetNodeObject(namespaceId, commitNode.Contents), metadata };
			}
			else if (blobData.Type.Guid == CbNode.BlobTypeGuid)
			{
				CbNode cbNode = BlobSerializer.Deserialize<CbNode>(blobData);
				content = GetCbNodeObject(namespaceId, cbNode.Object.AsField(), cbNode.Imports.GetEnumerator()) ?? new object();
			}
			else
			{
				IEnumerable<string>? references = null;
				if (blobData.Imports.Count > 0)
				{
					references = blobData.Imports.Select(x => GetNodeLink(namespaceId, x));
				}
				content = new { length = blobData.Data.Length, references };
			}

			string? typeName = null;
			if (s_blobGuidToType.TryGetValue(blobData.Type.Guid, out Type? type))
			{
				typeName = type.Name;
			}

			return new { type = typeName, guid = blobData.Type.Guid, data = $"{GetNodeLink(namespaceId, locator)}&data=true", content = content };
		}

		static object? GetCbNodeObject(NamespaceId namespaceId, CbField field, IEnumerator<IBlobRef> imports)
		{
			if (field.IsAttachment())
			{
				object? link = GetNodeLink(namespaceId, imports.Current);
				imports.MoveNext();
				return link;
			}
			else if (field.IsObject())
			{
				Dictionary<string, object?> fields = new Dictionary<string, object?>();

				CbObject obj = field.AsObject();
				foreach (CbField member in obj)
				{
					fields[member.Name.ToString()] = GetCbNodeObject(namespaceId, member, imports);
				}

				return fields;
			}
			else if (field.IsArray())
			{
				List<object?> elements = new List<object?>();

				CbArray arr = field.AsArray();
				foreach (CbField member in arr)
				{
					elements.Add(GetCbNodeObject(namespaceId, member, imports));
				}

				return elements;
			}
			else
			{
				return field.Value;
			}
		}

		[return: NotNullIfNotNull("nodeRef")]
		static object? GetNodeObject(NamespaceId namespaceId, DirectoryNodeRef? nodeRef) => (nodeRef == null) ? null : new { nodeRef.Length, nodeRef.Handle.Hash, link = GetNodeLink(namespaceId, nodeRef.Handle.GetLocator()) };

		[return: NotNullIfNotNull("handle")]
		static object? GetNodeHandleLink(NamespaceId namespaceId, IHashedBlobRef? handle) => (handle == null) ? null : new { handle.Hash, link = GetNodeLink(namespaceId, handle.GetLocator()) };

		static string GetNodeLink(NamespaceId namespaceId, IBlobRef handle) => GetNodeLink(namespaceId, handle.GetLocator());

		static string GetNodeLink(NamespaceId namespaceId, BlobLocator locator) => $"/api/v1/storage/{namespaceId}/nodes/{locator.BaseLocator}?{locator.Fragment}";

		/// <summary>
		/// Get consolidated storage dashboard data (admin-only)
		/// </summary>
		[HttpGet]
		[Route("/api/v1/storage/dashboard")]
		public async Task<ActionResult<GetStorageDashboardResponse>> GetDashboardAsync(
			[FromQuery] DateTime? minTime,
			[FromQuery] DateTime? maxTime,
			[FromQuery] int trendCount = 500,
			CancellationToken cancellationToken = default)
		{
			if (!_aclService.Authorize(AclScopeName.Root, AdminAclAction.AdminRead, User))
			{
				return Forbid(AdminAclAction.AdminRead);
			}

			trendCount = Math.Clamp(trendCount, 1, 2000);

			DateTime effectiveMaxTime = maxTime ?? DateTime.UtcNow;
			DateTime effectiveMinTime = minTime ?? effectiveMaxTime.AddHours(-24);

			Task<(List<(NamespaceId Id, DateTime LastTime)> Namespaces, Dictionary<NamespaceId, long> QueueDepths)> gcStatusTask = _storageService.GetGcStatusAsync(cancellationToken);
			Task<IReadOnlyList<GcMetricDocument>> gcMetricsTask = _gcMetrics.FindAsync(null, effectiveMinTime, effectiveMaxTime, trendCount, cancellationToken);
			Task<IReadOnlyList<CollectionStatsDocument>> collStatsTask = _collectionStats.FindAsync(effectiveMinTime, effectiveMaxTime, trendCount, cancellationToken);

			await Task.WhenAll(gcStatusTask, gcMetricsTask, collStatsTask);

			StorageConfig config = _storageConfig.Value;
			StorageServerConfig serverConfig = _staticStorageConfig.Value;

			return new GetStorageDashboardResponse
			{
				Status = BuildStatusResponse(config, serverConfig, await gcStatusTask),
				GcMetrics = BuildGcMetricsResponse(await gcMetricsTask),
				CollectionStats = BuildCollectionStatsResponse(await collStatsTask)
			};
		}

		static StorageDashboardStatus BuildStatusResponse(StorageConfig config, StorageServerConfig serverConfig, (List<(NamespaceId Id, DateTime LastTime)> Namespaces, Dictionary<NamespaceId, long> QueueDepths) gcStatus)
		{
			Dictionary<NamespaceId, DateTime> lastGcTimes = gcStatus.Namespaces.ToDictionary(x => x.Id, x => x.LastTime);

			StorageDashboardStatus status = new StorageDashboardStatus
			{
				EnableGc = config.EnableGc,
				EnableGcVerification = config.EnableGcVerification,
				GcWorkerCount = config.GcWorkerCount,
				GcQueueLimit = config.GcQueueLimit,
				EnableReliableGcEnqueue = config.EnableReliableGcEnqueue,
				EnablePerNamespaceQueueIsolation = config.EnablePerNamespaceQueueIsolation,
				GcBlobIngestionBatchSize = config.GcBlobIngestionBatchSize,
				GcBatchSize = config.GcBatchSize,
				BundleCacheDir = serverConfig.BundleCacheDir,
				BundleCacheSize = serverConfig.BundleCacheSize
			};

			foreach (NamespaceConfig nsConfig in config.Namespaces)
			{
				long queueDepth = gcStatus.QueueDepths.GetValueOrDefault(nsConfig.Id);
				status.Namespaces.Add(new StorageNamespaceStatus
				{
					Id = nsConfig.Id.ToString(),
					Backend = nsConfig.Backend.ToString(),
					Prefix = nsConfig.Prefix,
					GcFrequencyHrs = nsConfig.GcFrequencyHrs,
					GcDelayHrs = nsConfig.GcDelayHrs,
					GcQueueDepth = queueDepth,
					IsPaused = queueDepth >= config.GcQueueLimit,
					LastGcTime = lastGcTimes.TryGetValue(nsConfig.Id, out DateTime lastTime) ? lastTime : null,
					EnableAliases = nsConfig.EnableAliases
				});
			}

			foreach (BackendConfig backendConfig in config.Backends)
			{
				status.Backends.Add(new StorageBackendInfo
				{
					Id = backendConfig.Id.ToString(),
					Type = backendConfig.Type?.ToString() ?? "Unknown",
					BucketName = backendConfig.AwsBucketName,
					BucketPath = backendConfig.AwsBucketPath,
					Region = backendConfig.AwsRegion,
					ContainerName = backendConfig.AzureContainerName,
					GcsBucketName = backendConfig.GcsBucketName,
					GcsBucketPath = backendConfig.GcsBucketPath,
					BaseDir = backendConfig.BaseDir,
					SecondaryBackend = backendConfig.Secondary.IsEmpty ? null : backendConfig.Secondary.ToString()
				});
			}

			return status;
		}

		static StorageDashboardGcMetrics BuildGcMetricsResponse(IReadOnlyList<GcMetricDocument> docs)
		{
			StorageDashboardGcMetrics response = new StorageDashboardGcMetrics();
			foreach (GcMetricDocument doc in docs)
			{
				response.Entries.Add(new StorageGcMetricEntry
				{
					Time = doc.TimeUtc,
					NamespaceId = doc.NamespaceId.ToString(),
					BlobsDeleted = doc.BlobsDeleted,
					BlobsChecked = doc.BlobsChecked,
					BytesFreed = doc.BytesFreed,
					BlobsIngested = doc.BlobsIngested,
					QueueDepth = doc.QueueDepth,
					SweepDurationMs = doc.SweepDurationMs,
					WasPaused = doc.WasPaused,
					EnqueueFailures = doc.EnqueueFailures,
					ThrottleEvents = doc.ThrottleEvents,
					RefsExpired = doc.RefsExpired
				});
			}
			return response;
		}

		static StorageDashboardCollectionStats BuildCollectionStatsResponse(IReadOnlyList<CollectionStatsDocument> docs)
		{
			StorageDashboardCollectionStats response = new StorageDashboardCollectionStats();
			foreach (CollectionStatsDocument doc in docs)
			{
				CollectionStatsSnapshotResponse snapshot = new CollectionStatsSnapshotResponse { Time = doc.TimeUtc };
				foreach (CollectionStatEntry entry in doc.Collections)
				{
					snapshot.Collections.Add(new CollectionStatResponse
					{
						Name = entry.CollectionName,
						DocumentCount = entry.DocumentCount,
						DataSizeBytes = entry.DataSizeBytes,
						IndexSizeBytes = entry.IndexSizeBytes,
						StorageSizeBytes = entry.StorageSizeBytes,
						AvgDocSizeBytes = entry.AvgDocSizeBytes,
						FragmentationRatio = entry.FragmentationRatio
					});
				}
				response.Entries.Add(snapshot);
			}
			return response;
		}
	}

	/// <summary>
	/// Request to upload a blob
	/// </summary>
	public class WriteBlobRequest
	{
		/// <summary>
		/// Set to indicate that the blob does not have any imports. Used to distinguish from an "unknown" set of imports.
		/// </summary>
		[FromForm(Name = "leaf")]
		public bool Leaf
		{
			get => Imports != null && Imports.Count == 0;
			set => Imports = (value ? new List<BlobLocator>() : Imports);
		}

		/// <summary>
		/// Imported blobs
		/// </summary>
		[FromForm(Name = "import")]
		public List<BlobLocator>? Imports { get; set; }

		/// <summary>
		/// Prefix for the uploaded file
		/// </summary>
		[FromForm(Name = "prefix")]
		public string? Prefix { get; set; }

		/// <summary>
		/// Data to be uploaded. May be null, in which case the server may return a separate url.
		/// </summary>
		[FromForm(Name = "file")]
		public IFormFile? File { get; set; }
	}

	/// <summary>
	/// Consolidated storage dashboard response
	/// </summary>
	public class GetStorageDashboardResponse
	{
		/// <summary>Live status, config, namespaces, backends</summary>
		public StorageDashboardStatus Status { get; set; } = new();

		/// <summary>GC metric history</summary>
		public StorageDashboardGcMetrics GcMetrics { get; set; } = new();

		/// <summary>MongoDB collection stats history</summary>
		public StorageDashboardCollectionStats CollectionStats { get; set; } = new();
	}

	/// <summary>
	/// Live GC configuration and namespace status
	/// </summary>
	public class StorageDashboardStatus
	{
		/// <summary>Primary GC switch</summary>
		public bool EnableGc { get; set; }

		/// <summary>Verification mode (log-only, no deletes)</summary>
		public bool EnableGcVerification { get; set; }

		/// <summary>Concurrent reachability check workers</summary>
		public int GcWorkerCount { get; set; }

		/// <summary>Max Redis queue entries before pausing ingestion</summary>
		public long GcQueueLimit { get; set; }

		/// <summary>Awaited Redis enqueue vs fire-and-forget</summary>
		public bool EnableReliableGcEnqueue { get; set; }

		/// <summary>Per-namespace queue limits vs global</summary>
		public bool EnablePerNamespaceQueueIsolation { get; set; }

		/// <summary>Blobs scanned per ingestion tick</summary>
		public int GcBlobIngestionBatchSize { get; set; }

		/// <summary>Blob IDs per $in reachability query</summary>
		public int GcBatchSize { get; set; }

		/// <summary>Local bundle cache directory</summary>
		public string? BundleCacheDir { get; set; }

		/// <summary>Max bundle cache size</summary>
		public string BundleCacheSize { get; set; } = string.Empty;

		/// <summary>Per-namespace status</summary>
		public List<StorageNamespaceStatus> Namespaces { get; set; } = new();

		/// <summary>Backend configurations</summary>
		public List<StorageBackendInfo> Backends { get; set; } = new();
	}

	/// <summary>
	/// Per-namespace status information
	/// </summary>
	public class StorageNamespaceStatus
	{
		/// <summary>Namespace ID</summary>
		public string Id { get; set; } = string.Empty;

		/// <summary>Backend ID</summary>
		public string Backend { get; set; } = string.Empty;

		/// <summary>Storage key prefix</summary>
		public string Prefix { get; set; } = string.Empty;

		/// <summary>Hours between GC sweeps</summary>
		public double GcFrequencyHrs { get; set; }

		/// <summary>Grace period before deleting orphaned blobs (hours)</summary>
		public double GcDelayHrs { get; set; }

		/// <summary>Current Redis GC queue depth</summary>
		public long GcQueueDepth { get; set; }

		/// <summary>Whether ingestion is paused (queue at limit)</summary>
		public bool IsPaused { get; set; }

		/// <summary>Time of last GC sweep</summary>
		public DateTime? LastGcTime { get; set; }

		/// <summary>Whether alias queries are enabled</summary>
		public bool EnableAliases { get; set; }
	}

	/// <summary>
	/// Storage backend configuration info
	/// </summary>
	public class StorageBackendInfo
	{
		/// <summary>Backend ID</summary>
		public string Id { get; set; } = string.Empty;

		/// <summary>Backend type (Aws, Azure, Gcs, FileSystem, Memory)</summary>
		public string Type { get; set; } = string.Empty;

		/// <summary>S3 bucket name</summary>
		public string? BucketName { get; set; }

		/// <summary>S3 bucket path</summary>
		public string? BucketPath { get; set; }

		/// <summary>AWS region</summary>
		public string? Region { get; set; }

		/// <summary>Azure container name</summary>
		public string? ContainerName { get; set; }

		/// <summary>GCS bucket name</summary>
		public string? GcsBucketName { get; set; }

		/// <summary>GCS bucket path</summary>
		public string? GcsBucketPath { get; set; }

		/// <summary>Filesystem base directory</summary>
		public string? BaseDir { get; set; }

		/// <summary>Secondary backend for migration</summary>
		public string? SecondaryBackend { get; set; }
	}

	/// <summary>
	/// GC metric entries container
	/// </summary>
	public class StorageDashboardGcMetrics
	{
		/// <summary>Metric entries</summary>
		public List<StorageGcMetricEntry> Entries { get; set; } = new();
	}

	/// <summary>
	/// Single GC metric entry (per-tick snapshot)
	/// </summary>
	public class StorageGcMetricEntry
	{
		/// <summary>Timestamp</summary>
		public DateTime Time { get; set; }

		/// <summary>Namespace ID</summary>
		public string NamespaceId { get; set; } = string.Empty;

		/// <summary>Blobs deleted in this tick</summary>
		public long BlobsDeleted { get; set; }

		/// <summary>Blobs checked for reachability</summary>
		public long BlobsChecked { get; set; }

		/// <summary>Bytes freed by deletion</summary>
		public long BytesFreed { get; set; }

		/// <summary>Blobs ingested into GC queue</summary>
		public long BlobsIngested { get; set; }

		/// <summary>Queue depth at recording time</summary>
		public long QueueDepth { get; set; }

		/// <summary>Sweep duration in milliseconds</summary>
		public double SweepDurationMs { get; set; }

		/// <summary>Whether namespace was paused</summary>
		public bool WasPaused { get; set; }

		/// <summary>Redis enqueue failures</summary>
		public long EnqueueFailures { get; set; }

		/// <summary>Queue backpressure events</summary>
		public long ThrottleEvents { get; set; }

		/// <summary>Refs expired and blobs queued for GC</summary>
		public long RefsExpired { get; set; }
	}

	/// <summary>
	/// Collection stats entries container
	/// </summary>
	public class StorageDashboardCollectionStats
	{
		/// <summary>Snapshot entries</summary>
		public List<CollectionStatsSnapshotResponse> Entries { get; set; } = new();
	}

	/// <summary>
	/// A single collection stats snapshot in time
	/// </summary>
	public class CollectionStatsSnapshotResponse
	{
		/// <summary>Snapshot timestamp</summary>
		public DateTime Time { get; set; }

		/// <summary>Per-collection stats</summary>
		public List<CollectionStatResponse> Collections { get; set; } = new();
	}

	/// <summary>
	/// Stats for a single MongoDB collection
	/// </summary>
	public class CollectionStatResponse
	{
		/// <summary>Collection name</summary>
		public string Name { get; set; } = string.Empty;

		/// <summary>Number of documents</summary>
		public long DocumentCount { get; set; }

		/// <summary>Data size in bytes</summary>
		public long DataSizeBytes { get; set; }

		/// <summary>Index size in bytes</summary>
		public long IndexSizeBytes { get; set; }

		/// <summary>Storage size in bytes</summary>
		public long StorageSizeBytes { get; set; }

		/// <summary>Average document size in bytes</summary>
		public long AvgDocSizeBytes { get; set; }

		/// <summary>Fragmentation ratio (storageSize / dataSize)</summary>
		public double FragmentationRatio { get; set; }
	}
}
