// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Collections.Concurrent;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.Metrics;
using System.Linq.Expressions;
using System.Threading.Channels;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.ObjectStores;
using EpicGames.Redis;
using EpicGames.Redis.Utility;
using HordeServer.Server;
using HordeServer.Storage.Storage.ObjectStores;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using MongoDB.Driver.Linq;
using OpenTelemetry.Trace;
using StackExchange.Redis;

namespace HordeServer.Storage
{
	/// <summary>
	/// Interface for the storage service
	/// </summary>
	public interface IStorageService : IStorageClient
	{
	}

	/// <summary>
	/// Functionality related to the storage service
	/// </summary>
	public sealed class StorageService : IHostedService, IStorageService, IAsyncDisposable
	{
		class RefCount
		{
			int _value;

			public RefCount() => _value = 1;
			public void AddRef() => Interlocked.Increment(ref _value);
			public int Release() => Interlocked.Decrement(ref _value);
		}

		sealed class StorageBackendImpl : IStorageBackend
		{
			readonly StorageService _outer;
			readonly IObjectStore _store;
			readonly NamespaceConfig _config;

			public NamespaceConfig Config => _config;
			public NamespaceId NamespaceId => _config.Id;

			/// <inheritdoc/>
			public bool SupportsRedirects { get; }

			public StorageBackendImpl(StorageService outer, NamespaceConfig config, IObjectStore store)
			{
				_outer = outer;
				_store = store;
				_config = config;

				SupportsRedirects = store.SupportsRedirects && !config.EnableAliases;
			}

			#region Blobs

			public async Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
			{
				await _outer.CheckBlobExistsAsync(NamespaceId, locator, cancellationToken);
				return await _store.OpenAsync(GetObjectKey(locator), offset, length, cancellationToken);
			}

			/// <inheritdoc/>
			public async Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
			{
				await _outer.CheckBlobExistsAsync(NamespaceId, locator, cancellationToken);
				return await _store.ReadAsync(GetObjectKey(locator), offset, length, cancellationToken);
			}

			/// <inheritdoc/>
			public async Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator> imports, string? basePath = null, CancellationToken cancellationToken = default)
			{
				BlobLocator locator = StorageHelpers.CreateUniqueLocator(basePath);
				await WriteBlobAsync(locator, stream, imports, cancellationToken);
				return locator;
			}

			/// <inheritdoc/>
			public async Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
			{
				await _store.WriteAsync(GetObjectKey(locator), stream, cancellationToken);
				await _outer.AddBlobAsync(NamespaceId, locator, imports, null, cancellationToken);
			}

			/// <inheritdoc/>
			public async ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
			{
				await _outer.CheckBlobExistsAsync(NamespaceId, locator, cancellationToken);
				return await _store.TryGetReadRedirectAsync(GetObjectKey(locator), cancellationToken);
			}

			/// <inheritdoc/>
			public async ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
			{
				if (!_store.SupportsRedirects)
				{
					return null;
				}

				Uri? url = await _store.TryGetWriteRedirectAsync(GetObjectKey(locator), cancellationToken);
				if (url == null)
				{
					return null;
				}

				await _outer.AddBlobAsync(NamespaceId, locator, imports, null, cancellationToken);
				return url;
			}

			/// <inheritdoc/>
			public async ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default)
			{
				if (!_store.SupportsRedirects)
				{
					return null;
				}

				BlobLocator locator = StorageHelpers.CreateUniqueLocator(prefix);

				Uri? url = await TryGetBlobWriteRedirectAsync(locator, imports, cancellationToken);
				if (url == null)
				{
					return null;
				}

				return (locator, url);
			}

			#endregion

			#region Aliases

			/// <inheritdoc/>
			public async Task<BlobAliasLocator[]> FindAliasesAsync(string alias, int? maxResults, CancellationToken cancellationToken = default)
			{
				List<(BlobLocator, AliasInfo)> aliases = await _outer.FindAliasesAsync(NamespaceId, alias, cancellationToken);
				if (maxResults != null && maxResults.Value < aliases.Count)
				{
					aliases.RemoveRange(maxResults.Value, aliases.Count - maxResults.Value);
				}
				return aliases.Select(x => new BlobAliasLocator(x.Item1, x.Item2.Rank, x.Item2.Data)).ToArray();
			}

			#endregion

			#region Refs

			/// <inheritdoc/>
			public async Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime, CancellationToken cancellationToken)
			{
				RefInfo? refInfo = await _outer.TryReadRefAsync(NamespaceId, name, cacheTime, cancellationToken);
				if (refInfo == null)
				{
					return null;
				}
				return new HashedBlobRefValue(refInfo.Hash, refInfo.Target);
			}

			/// <inheritdoc/>
			public Task WriteRefAsync(RefName name, HashedBlobRefValue value, RefOptions? options = null, CancellationToken cancellationToken = default)
				=> _outer.AddRefAsync(NamespaceId, name, value, options, cancellationToken);

			/// <inheritdoc/>
			public Task<bool> DeleteRefAsync(RefName name, CancellationToken cancellationToken = default)
				=> _outer.RemoveRefAsync(NamespaceId, name, cancellationToken);

			#endregion

			/// <inheritdoc/>
			public async Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
			{
				if (request.AddAliases.Count > 0)
				{
					await _outer.AddAliasesAsync(NamespaceId, request.AddAliases, cancellationToken);
				}
				if (request.RemoveAliases.Count > 0)
				{
					await _outer.RemoveAliasesAsync(NamespaceId, request.RemoveAliases, cancellationToken);
				}

				List<Func<CancellationToken, Task>> actions = new List<Func<CancellationToken, Task>>();
				foreach (AddRefRequest addRef in request.AddRefs)
				{
					actions.Add(ctx => _outer.AddRefAsync(NamespaceId, addRef.RefName, new HashedBlobRefValue(addRef.Hash, addRef.Target), addRef.Options, ctx));
				}
				foreach (RemoveRefRequest removeRef in request.RemoveRefs)
				{
					actions.Add(ctx => _outer.RemoveRefAsync(NamespaceId, removeRef.RefName, ctx));
				}

				await Parallel.ForEachAsync(actions, cancellationToken, (action, cancellationToken)
					=> WrapAsync(() => action(cancellationToken)));
			}

			async ValueTask WrapAsync(Func<Task> func)
			{
				try
				{
					await func();
				}
				catch (Exception ex)
				{
					_outer._logger.LogError(ex, "Error during metadata update: {Message}", ex.Message);
					throw;
				}
			}

			/// <inheritdoc/>
			public void GetStats(StorageStats stats) => _store.GetStats(stats);
		}

		class State
		{
			public StorageConfig Config { get; }
			public Dictionary<NamespaceId, NamespaceInfo> Namespaces { get; } = new Dictionary<NamespaceId, NamespaceInfo>();

			public State(StorageConfig config)
			{
				Config = config;
			}

			public IStorageBackend? TryCreateBackend(NamespaceId namespaceId)
			{
				NamespaceInfo? namespaceInfo;
				if (!Namespaces.TryGetValue(namespaceId, out namespaceInfo))
				{
					return null;
				}
				return namespaceInfo.Backend;
			}
		}

		class NamespaceInfo
		{
			public NamespaceId Id => Config.Id;
			public NamespaceConfig Config { get; }
			public IObjectStore Store { get; }
			public StorageBackendImpl Backend { get; }

			public NamespaceInfo(NamespaceConfig config, IObjectStore store, StorageBackendImpl backend)
			{
				Config = config;
				Store = store;
				Backend = backend;
			}
		}

		internal class AliasInfo
		{
			[BsonElement("nam")]
			public string Name { get; set; }

			[BsonElement("frg")]
			public string Fragment { get; set; }

			[BsonElement("rnk"), BsonIgnoreIfDefault]
			public int Rank { get; set; }

			[BsonElement("dat"), BsonIgnoreIfNull]
			public byte[]? Data { get; set; }

			[BsonConstructor]
			public AliasInfo()
			{
				Name = String.Empty;
				Fragment = String.Empty;
			}

			public AliasInfo(string name, string fragment, byte[]? data, int rank)
			{
				Name = name;
				Fragment = fragment;
				Rank = rank;
				Data = (data == null || data.Length == 0) ? null : data;
			}
		}

		const int CurrentGcVersion = 2;

		internal class BlobInfo
		{
			public ObjectId Id { get; set; }

			[BsonElement("ns")]
			public NamespaceId NamespaceId { get; set; }

			[BsonElement("blob")]
			public string Path { get; set; }

			[BsonElement("imp"), BsonIgnoreIfNull]
			public List<ObjectId>? Imports { get; set; }

			[BsonElement("ali"), BsonIgnoreIfNull]
			public List<AliasInfo>? Aliases { get; set; }

			[BsonElement("shd"), BsonIgnoreIfDefault]
			public bool Shadow { get; set; } // Blob has not been added yet, but is referenced by something else

			[BsonElement("del"), BsonIgnoreIfDefault]
			public int GcVersion { get; set; }

			[BsonElement("len"), BsonIgnoreIfDefault]
			public long Length { get; set; }

			[BsonElement("idx"), BsonIgnoreIfDefault]
			public int? UpdateIndex { get; set; }

			[BsonIgnore]
			public BlobLocator Locator => new BlobLocator(Path);

			public BlobInfo()
			{
				Path = String.Empty;
			}

			public BlobInfo(ObjectId id, NamespaceId namespaceId, BlobLocator locator)
			{
				Id = id;
				NamespaceId = namespaceId;
				Path = locator.ToString();
			}
		}

		internal class RefInfo : ISupportInitialize
		{
			[BsonIgnoreIfDefault]
			public ObjectId Id { get; set; }

			[BsonElement("ns")]
			public NamespaceId NamespaceId { get; set; }

			[BsonElement("name")]
			public RefName Name { get; set; }

			[BsonElement("hash")]
			public IoHash Hash { get; set; }

			[BsonElement("tgt")]
			public BlobLocator Target { get; set; }

			[BsonElement("binf")]
			public ObjectId TargetBlobId { get; set; }

			[BsonElement("xa"), BsonIgnoreIfDefault]
			public DateTime? ExpiresAtUtc { get; set; }

			[BsonElement("xt"), BsonIgnoreIfDefault]
			public TimeSpan? Lifetime { get; set; }

#pragma warning disable IDE0051 // Remove unused private members
			[BsonExtraElements]
			BsonDocument? ExtraElements { get; set; }
#pragma warning restore IDE0051 // Remove unused private members

			[BsonConstructor]
			public RefInfo()
			{
				Name = RefName.Empty;
			}

			public RefInfo(NamespaceId namespaceId, RefName name, IoHash hash, BlobLocator target, ObjectId targetBlobId)
			{
				NamespaceId = namespaceId;
				Name = name;
				Hash = hash;
				Target = target;
				TargetBlobId = targetBlobId;
			}

			public bool HasExpired(DateTime utcNow) => ExpiresAtUtc.HasValue && utcNow >= ExpiresAtUtc.Value;

			public bool RequiresTouch(DateTime utcNow) => ExpiresAtUtc.HasValue && Lifetime.HasValue && utcNow >= ExpiresAtUtc.Value - new TimeSpan(Lifetime.Value.Ticks / 4);

			void ISupportInitialize.BeginInit() { }

			void ISupportInitialize.EndInit()
			{
				if (ExtraElements != null)
				{
					if (ExtraElements.TryGetValue("blob", out BsonValue blob) && ExtraElements.TryGetValue("idx", out BsonValue idx))
					{
						Target = new BlobLocator($"{blob.AsString}#{idx.AsInt32}");
					}
					ExtraElements = null;
				}
			}
		}

		[SingletonDocument("gc-state")]
		class GcState : SingletonBase
		{
			public ObjectId LastImportBlobInfoId { get; set; }
			public List<GcNamespaceState> Namespaces { get; set; } = new List<GcNamespaceState>();
			public bool Reset { get; set; }

			public void DoReset()
			{
				LastImportBlobInfoId = ObjectId.Empty;
				Reset = false;
			}

			public GcNamespaceState FindOrAddNamespace(NamespaceId namespaceId)
			{
				GcNamespaceState? namespaceState = Namespaces.FirstOrDefault(x => x.Id == namespaceId);
				if (namespaceState == null)
				{
					namespaceState = new GcNamespaceState { Id = namespaceId, LastTime = DateTime.UtcNow };
					Namespaces.Add(namespaceState);
				}
				return namespaceState;
			}
		}

		[SingletonDocument("length-scan-state")]
		class LengthScanState : SingletonBase
		{
			public ObjectId LastImportBlobInfoId { get; set; }
			public bool Reset { get; set; }

			public void DoReset()
			{
				LastImportBlobInfoId = ObjectId.Empty;
				Reset = false;
			}
		}

		class GcNamespaceState
		{
			public NamespaceId Id { get; set; }
			public DateTime LastTime { get; set; }
		}

		readonly IRedisService _redisService;
		readonly IClock _clock;
		readonly BundleCache _bundleCache;
		readonly IMemoryCache _memoryCache;
		readonly IObjectStoreFactory _objectStoreFactory;
		readonly IOptionsMonitor<StorageConfig> _storageConfig;
		readonly IOptions<StorageServerConfig> _staticStorageConfig;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		readonly Counter<long> _gcBlobsDeletedCounter;
		readonly Counter<long> _gcBlobsIngestedCounter;
		readonly Counter<long> _gcEnqueueFailuresCounter;
		readonly Counter<long> _gcTickThrottleEventsCounter;
		readonly Histogram<double> _gcReachabilityCheckDuration;

		readonly ConcurrentDictionary<NamespaceId, GcCounters> _gcCounters = new();

		internal class GcCounters
		{
			long _blobsDeleted;
			long _blobsChecked;
			long _bytesFreed;
			long _sweepDurationUs;
			long _blobsIngested;
			long _enqueueFailures;
			long _refsExpired;

			public void AddSweep(long deleted, long checked_, long bytesFreed, double sweepMs)
			{
				Interlocked.Add(ref _blobsDeleted, deleted);
				Interlocked.Add(ref _blobsChecked, checked_);
				Interlocked.Add(ref _bytesFreed, bytesFreed);
				Interlocked.Add(ref _sweepDurationUs, (long)(sweepMs * 1000.0));
			}

			public void AddIngested(long count) => Interlocked.Add(ref _blobsIngested, count);
			public void AddEnqueueFailures(long count) => Interlocked.Add(ref _enqueueFailures, count);
			public void AddRefsExpired(long count) => Interlocked.Add(ref _refsExpired, count);

			public (long BlobsDeleted, long BlobsChecked, long BytesFreed, double SweepDurationMs, long BlobsIngested, long EnqueueFailures, long RefsExpired) ReadAndReset()
			{
				return (
					Interlocked.Exchange(ref _blobsDeleted, 0),
					Interlocked.Exchange(ref _blobsChecked, 0),
					Interlocked.Exchange(ref _bytesFreed, 0),
					Interlocked.Exchange(ref _sweepDurationUs, 0) / 1000.0,
					Interlocked.Exchange(ref _blobsIngested, 0),
					Interlocked.Exchange(ref _enqueueFailures, 0),
					Interlocked.Exchange(ref _refsExpired, 0)
				);
			}
		}

		GcCounters GetOrAddCounters(NamespaceId ns) => _gcCounters.GetOrAdd(ns, _ => new GcCounters());

		readonly IMongoCollection<BlobInfo> _blobCollection;
		readonly IMongoCollection<RefInfo> _refCollection;

		readonly ITicker _blobTicker;
		readonly ITicker _refTicker;

		readonly SingletonDocument<GcState> _gcState;
		readonly ITicker _gcTicker;

		readonly SingletonDocument<LengthScanState> _lengthScanState;
		readonly ITicker _lengthScanTicker;

		readonly object _lockObject = new object();

		string? _lastConfigRevision;
		State? _lastState;

		internal IMongoCollection<BlobInfo> BlobCollection => _blobCollection;

		static readonly FieldDefinition<BlobInfo, string> s_blobAliasField
			= new StringFieldDefinition<BlobInfo, string>($"{GetFieldName<BlobInfo>(x => x.Aliases)}.{GetFieldName<AliasInfo>(x => x.Name)}");

		class ObjectIdRedisConverter : IRedisConverter<ObjectId>
		{
			public ObjectId FromRedisValue(RedisValue value)
				=> value.IsNullOrEmpty ? ObjectId.Empty : new ObjectId((byte[]?)value);

			public RedisValue ToRedisValue(ObjectId value)
				=> value.ToByteArray();
		}

		static StorageService()
		{
			RedisSerializer.RegisterConverter<ObjectId, ObjectIdRedisConverter>();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageService(IMongoService mongoService, IRedisService redisService, IClock clock, BundleCache bundleCache, IMemoryCache memoryCache, IObjectStoreFactory objectStoreFactory, IOptionsMonitor<StorageConfig> storageConfig, IOptions<StorageServerConfig> staticStorageConfig, Tracer tracer, Meter meter, ILogger<StorageService> logger)
		{
			_redisService = redisService;
			_clock = clock;
			_bundleCache = bundleCache;
			_memoryCache = memoryCache;
			_objectStoreFactory = objectStoreFactory;
			_storageConfig = storageConfig;
			_staticStorageConfig = staticStorageConfig;
			_tracer = tracer;
			_logger = logger;

			_gcBlobsDeletedCounter = meter.CreateCounter<long>("horde.gc.blobs.deleted");
			_gcBlobsIngestedCounter = meter.CreateCounter<long>("horde.gc.blobs.ingested");
			_gcEnqueueFailuresCounter = meter.CreateCounter<long>("horde.gc.enqueue.failures");
			_gcTickThrottleEventsCounter = meter.CreateCounter<long>("horde.gc.tick.throttle.events");
			_gcReachabilityCheckDuration = meter.CreateHistogram<double>("horde.gc.reachability.check.duration", unit: "ms");

			List<MongoIndex<BlobInfo>> blobIndexes = new List<MongoIndex<BlobInfo>>();
			blobIndexes.Add(keys => keys.Ascending(x => x.Imports));
			blobIndexes.Add(keys => keys.Ascending(x => x.NamespaceId).Ascending(s_blobAliasField));
			blobIndexes.Add(keys => keys.Ascending(x => x.NamespaceId).Ascending(x => x.Path), unique: true);
			_blobCollection = mongoService.GetCollection<BlobInfo>("Storage.Blobs", blobIndexes);

			List<MongoIndex<RefInfo>> refIndexes = new List<MongoIndex<RefInfo>>();
			refIndexes.Add(keys => keys.Ascending(x => x.NamespaceId).Ascending(x => x.Name), unique: true);
			refIndexes.Add(keys => keys.Ascending(x => x.TargetBlobId));
			refIndexes.Add(keys => keys.Descending(x => x.ExpiresAtUtc), sparse: true);
			_refCollection = mongoService.GetCollection<RefInfo>("Storage.Refs", refIndexes);

			_blobTicker = clock.AddSharedTicker("Storage:Blobs", TimeSpan.FromMinutes(5.0), TickBlobsAsync, _logger);
			_refTicker = clock.AddSharedTicker("Storage:Refs", TimeSpan.FromMinutes(5.0), TickRefsAsync, _logger);

			_gcState = new SingletonDocument<GcState>(mongoService);
			_gcTicker = clock.AddTicker("Storage:GC", TimeSpan.FromMinutes(5.0), TickGcAsync, logger);

			_lengthScanState = new SingletonDocument<LengthScanState>(mongoService);
			_lengthScanTicker = clock.AddSharedTicker("Storage:LengthScan", TimeSpan.FromMinutes(5.0), TickLengthsAsync, logger);
		}

		static string GetFieldName<TClass>(Expression<Func<TClass, object?>> expr)
		{
			ExpressionFieldDefinition<TClass> collection = new ExpressionFieldDefinition<TClass>(expr);
			RenderedFieldDefinition fd = collection.Render(BsonSerializer.LookupSerializer<TClass>(), BsonSerializer.SerializerRegistry);
			return fd.FieldName;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _blobTicker.DisposeAsync();
			await _refTicker.DisposeAsync();
			await _gcTicker.DisposeAsync();
			await _lengthScanTicker.DisposeAsync();
		}

		internal static ObjectKey GetObjectKey(BlobLocator locator) => new ObjectKey($"{locator.Path}.blob");

		class StorageClient : IStorageClient
		{
			readonly StorageService _storageService;
			readonly StorageConfig _storageConfig;

			public StorageClient(StorageService storageService, StorageConfig storageConfig)
			{
				_storageService = storageService;
				_storageConfig = storageConfig;
			}

			public IStorageNamespace? TryGetNamespace(NamespaceId namespaceId)
				=> _storageService.TryCreateClient(_storageConfig, namespaceId);
		}

		/// <summary>
		/// Creates a new storage namespace factory using the current global config value
		/// </summary>
		public IStorageClient CreateStorageClient(StorageConfig storageConfig)
			=> new StorageClient(this, storageConfig);

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _blobTicker.StartAsync();
			await _refTicker.StartAsync();
			await _gcTicker.StartAsync();
			await _lengthScanTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _lengthScanTicker.StopAsync();
			await _gcTicker.StopAsync();
			await _refTicker.StopAsync();
			await _blobTicker.StopAsync();
		}

		/// <inheritdoc/>
		public IStorageBackend CreateBackend(NamespaceId namespaceId)
		{
			return TryCreateBackend(namespaceId) ?? throw new StorageException($"Namespace '{namespaceId}' not found");
		}

		/// <inheritdoc/>
		public IStorageBackend? TryCreateBackend(NamespaceId namespaceId)
			=> TryCreateBackend(_storageConfig.CurrentValue, namespaceId);

		/// <inheritdoc/>
		public IStorageBackend? TryCreateBackend(StorageConfig storageConfig, NamespaceId namespaceId)
		{
			State snapshot = CreateState(storageConfig);
			return snapshot.TryCreateBackend(namespaceId);
		}

		/// <inheritdoc/>
		public IStorageNamespace? TryGetNamespace(NamespaceId namespaceId)
			=> TryCreateClient(namespaceId, null);

		/// <inheritdoc/>
		public IStorageNamespace? TryCreateClient(NamespaceId namespaceId, BundleOptions? bundleOptions = null)
			=> TryCreateClient(_storageConfig.CurrentValue, namespaceId, bundleOptions);

		/// <inheritdoc/>
		public IStorageNamespace? TryCreateClient(StorageConfig storageConfig, NamespaceId namespaceId, BundleOptions? bundleOptions = null)
		{
#pragma warning disable CA2000 // Call dispose on backend; will be disposed by BundleStorageNamespace
			IStorageBackend? backend = TryCreateBackend(storageConfig, namespaceId);
#pragma warning restore CA2000
			if (backend == null)
			{
				return null;
			}
			else
			{
				return new BundleStorageNamespace(backend, _bundleCache, bundleOptions, _logger);
			}
		}

		#region Config

		State CreateState(StorageConfig storageConfig)
		{
			lock (_lockObject)
			{
				if (_lastState == null || !String.Equals(_lastConfigRevision, storageConfig.Revision, StringComparison.Ordinal))
				{
					_logger.LogDebug("Updating storage providers for config {Revision}", storageConfig.Revision);

					State nextState = new State(storageConfig);

					foreach (NamespaceConfig namespaceConfig in storageConfig.Namespaces)
					{
						NamespaceId namespaceId = namespaceConfig.Id;
						try
						{
							IObjectStore objectStore = CreateObjectStore(storageConfig, namespaceConfig.Backend);
							if (!String.IsNullOrEmpty(namespaceConfig.Prefix))
							{
								objectStore = new PrefixedObjectStore(namespaceConfig.Prefix, objectStore);
							}

							StorageBackendImpl backend = new StorageBackendImpl(this, namespaceConfig, objectStore);

							NamespaceInfo namespaceInfo = new NamespaceInfo(namespaceConfig, objectStore, backend);
							nextState.Namespaces.Add(namespaceId, namespaceInfo);
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Unable to create storage backend for {NamespaceId}: ", namespaceId);
						}
					}

					_lastState = nextState;
					_lastConfigRevision = storageConfig.Revision;
				}
				return _lastState;
			}
		}

		IObjectStore CreateObjectStore(StorageConfig storageConfig, BackendId backendId)
		{
			BackendConfig? backendConfig = _staticStorageConfig.Value.Backends.FirstOrDefault(x => x.Id == backendId);
			if (backendConfig == null && !storageConfig.TryGetBackend(backendId, out backendConfig))
			{
				throw new StorageException($"Missing or invalid backend identifier '{backendId}');");
			}

			IObjectStore objectStore = _objectStoreFactory.CreateObjectStore(backendConfig);
			if (!backendConfig.Secondary.IsEmpty)
			{
				IObjectStore secondaryObjectStore = CreateObjectStore(storageConfig, backendConfig.Secondary);
				objectStore = new ChainedObjectStore(objectStore, secondaryObjectStore);
			}

			return objectStore;
		}

		#endregion

		#region Blobs

		internal async Task<BlobInfo> AddBlobAsync(NamespaceId namespaceId, BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, List<AliasInfo>? exports = null, CancellationToken cancellationToken = default)
		{
			List<ObjectId> importIds = await FindOrAddShadowBlobsAsync(namespaceId, imports, cancellationToken);

			FilterDefinition<BlobInfo> insertFilter =
				Builders<BlobInfo>.Filter.Eq(x => x.NamespaceId, namespaceId) &
				Builders<BlobInfo>.Filter.Eq(x => x.Path, locator.ToString());
			
			UpdateDefinition<BlobInfo> insertUpdate = Builders<BlobInfo>.Update
				.Unset(x => x.Shadow)
				.SetOnInsert(x => x.NamespaceId, namespaceId)  // Set these only on insert
				.SetOnInsert(x => x.Path, locator.ToString());

			if (importIds.Count > 0)
			{
				insertUpdate = insertUpdate.Set(x => x.Imports, importIds);
			}
			if (exports is { Count: > 0 })
			{
				insertUpdate = insertUpdate.Set(x => x.Aliases, exports);
			}
			
			FindOneAndUpdateOptions<BlobInfo,BlobInfo> options = new () { IsUpsert = true, ReturnDocument = ReturnDocument.After };
			BlobInfo blobInfo = await _blobCollection.FindOneAndUpdateAsync(insertFilter, insertUpdate, options, cancellationToken);
			_logger.LogDebug("Created blob {BlobId} at {Path} ({NumImports} imports)", blobInfo.Id, blobInfo.Path, imports.Count);
			return blobInfo;
		}

		async Task<List<ObjectId>> FindOrAddShadowBlobsAsync(NamespaceId namespaceId, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StorageService)}.{nameof(FindOrAddShadowBlobsAsync)}");

			List<ObjectId> importIds = new List<ObjectId>(imports.Count);
			if(imports.Count > 0)
			{
				// Find all the unique import paths
				HashSet<string> remaining = new HashSet<string>(imports.Count, StringComparer.Ordinal);
				foreach (BlobLocator import in imports)
				{
					remaining.Add(import.ToString());
				}

				// Get the corresponding import ids
				Dictionary<BlobLocator, ObjectId> locatorToId = new Dictionary<BlobLocator, ObjectId>(imports.Count);
				for (; ;)
				{
					// Find all the existing blobs, in batches of 100
					foreach (IReadOnlyList<string> batch in remaining.Batch(100))
					{
						FilterDefinition<BlobInfo> filter =
							Builders<BlobInfo>.Filter.Eq(x => x.NamespaceId, namespaceId) &
							Builders<BlobInfo>.Filter.In(x => x.Path, batch);

						ProjectionDefinition<BlobInfo> projection =
							Builders<BlobInfo>.Projection.Include(x => x.Id).Include(x => x.Path);

						await foreach (BlobInfo blobInfo in _blobCollection.Find(filter).Project<BlobInfo>(projection).ToAsyncEnumerable(cancellationToken))
						{
							remaining.Remove(blobInfo.Path);
							locatorToId.Add(new BlobLocator(blobInfo.Path), blobInfo.Id);
						}
					}

					// Break out if there's nothing to upsert
					if (remaining.Count == 0)
					{
						break;
					}

					// If there's anything left, try to insert shadow blobs for them. We'll do a separate query to figure out which succeeded.
					List<WriteModel<BlobInfo>> writes = new List<WriteModel<BlobInfo>>();
					foreach (string import in remaining)
					{
						FilterDefinition<BlobInfo> filter =
							Builders<BlobInfo>.Filter.Eq(x => x.NamespaceId, namespaceId) &
							Builders<BlobInfo>.Filter.Eq(x => x.Path, import);

						UpdateDefinition<BlobInfo> update =
							Builders<BlobInfo>.Update.SetOnInsert(x => x.Shadow, true);

						writes.Add(new UpdateOneModel<BlobInfo>(filter, update) { IsUpsert = true });
					}
					try
					{
						await _blobCollection.BulkWriteAsync(writes, new BulkWriteOptions { IsOrdered = false }, cancellationToken);
					}
					catch (MongoBulkWriteException ex)
					{
						foreach (WriteError writeError in ex.WriteErrors)
						{
							if (writeError.Category != ServerErrorCategory.DuplicateKey)
							{
								throw;
							}
						}
					}
				}

				// Find all the matching import ids in order
				foreach (BlobLocator import in imports)
				{
					importIds.Add(locatorToId[import]);
				}
			}
			return importIds;
		}

		internal async Task<BlobInfo?> GetBlobAsync(ObjectId id, CancellationToken cancellationToken = default)
		{
			return await _blobCollection.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
		}

		internal async Task<BlobInfo?> FindBlobAsync(NamespaceId namespaceId, BlobLocator locator, CancellationToken cancellationToken = default)
		{
			return await _blobCollection.Find(x => x.NamespaceId == namespaceId && x.Path == locator.ToString()).FirstOrDefaultAsync(cancellationToken);
		}

		async ValueTask CheckBlobExistsAsync(NamespaceId namespaceId, BlobLocator locator, CancellationToken cancellationToken)
		{
			if (_storageConfig.CurrentValue.EnableGcVerification)
			{
				try
				{
					string path = locator.Path.ToString();

					BlobInfo? blobInfo = await _blobCollection.Find(x => x.NamespaceId == namespaceId && x.Path == path && x.GcVersion >= CurrentGcVersion).FirstOrDefaultAsync(cancellationToken);
					if (blobInfo != null)
					{
						_logger.LogWarning("Blob {BlobId} ({Locator}) accessed after being garbage collected", blobInfo.Id, locator);
					}
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Exception checking if blob {NamespaceId}:{Locator} exists", namespaceId, locator);
				}
			}
		}

		async Task<bool> IsBlobReferencedAsync(ObjectId blobInfoId, CancellationToken cancellationToken = default)
		{
			FilterDefinition<BlobInfo> blobFilter = Builders<BlobInfo>.Filter.AnyEq(x => x.Imports, blobInfoId);
			if (await _blobCollection.Find(blobFilter).Limit(1).CountDocumentsAsync(cancellationToken) > 0)
			{
				return true;
			}

			FilterDefinition<RefInfo> refFilter = Builders<RefInfo>.Filter.Eq(x => x.TargetBlobId, blobInfoId);
			if (await _refCollection.Find(refFilter).Limit(1).CountDocumentsAsync(cancellationToken) > 0)
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Finds blobs at least 30 minutes old and computes import metadata for them. Done with a delay to allow write redirects.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		internal async ValueTask TickBlobsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StorageService)}.{nameof(TickBlobsAsync)}");

			StorageConfig storageConfig = _storageConfig.CurrentValue;
			GcState gcState = await _gcState.GetAsync(cancellationToken);

			DateTime ingestTimeUtc = _clock.UtcNow - TimeSpan.FromHours(12.0);

			// Get the current state of the storage system
			State state = CreateState(storageConfig);

			// Drain deferred re-scan sets from previous ticks (always, regardless of isolation flag).
			// This ensures deferred blobs are not stranded if the flag is toggled off.
			await DrainDeferredSetsAsync(gcState, storageConfig, cancellationToken);

			long ingestedCount = 0;
			Dictionary<NamespaceId, long> namespaceCounts = new Dictionary<NamespaceId, long>();
			Dictionary<NamespaceId, long> namespaceEnqueueFailures = new Dictionary<NamespaceId, long>();

			// Compute missing import info, by searching for blobs with an ObjectId timestamp after the last import compute cycle
			ObjectId latestInfoId = ObjectId.GenerateNewId(ingestTimeUtc);
			for (; ; )
			{
				// Check for the reset flag being set
				if (gcState.Reset)
				{
					_logger.LogInformation("Resetting scan for new blobs...");
					gcState = await _gcState.UpdateAsync(x => x.DoReset(), cancellationToken);
				}

				// Fetch the next batch of blobs
				List<BlobInfo> current = await _blobCollection.Find(x => x.Id > gcState.LastImportBlobInfoId && x.Id < latestInfoId).SortBy(x => x.Id).Limit(storageConfig.GcBlobIngestionBatchSize).ToListAsync(cancellationToken);
				if (current.Count == 0)
				{
					break;
				}

				// Check which namespaces have backed-up GC queues
				HashSet<NamespaceId>? pausedNamespaces = await GetPausedNamespacesAsync(storageConfig, current, cancellationToken);

				// DEFAULT PATH (isolation disabled): preserve current behavior exactly.
				// If ANY namespace is backed up, pause the entire scan.
				if (!storageConfig.EnablePerNamespaceQueueIsolation)
				{
					if (pausedNamespaces != null)
					{
						int delaySecs = 1;
						while (pausedNamespaces != null)
						{
							await Task.Delay(TimeSpan.FromSeconds(delaySecs), cancellationToken);
							delaySecs = Math.Min(delaySecs * 2, 128);
							pausedNamespaces = await GetPausedNamespacesAsync(storageConfig, current, cancellationToken);
						}
					}
				}

				// Add check records for each new blob (with optional per-namespace isolation)
				_logger.LogDebug("Adding {NumBlobs} blobs for GC consideration ({FirstId} to {LastId})", current.Count, current[0].Id, current[^1].Id);

				// Collect entries for batched Redis write when reliable enqueue is enabled
				Dictionary<NamespaceId, List<SortedSetEntry<ObjectId>>>? batchEntries = storageConfig.EnableReliableGcEnqueue
					? new Dictionary<NamespaceId, List<SortedSetEntry<ObjectId>>>()
					: null;

				double score = GetGcTimestamp();
				foreach (BlobInfo blobInfo in current)
				{
					if (blobInfo.Shadow)
					{
						_logger.LogWarning("Referenced blob {NamespaceId} {Path} was never uploaded.", blobInfo.NamespaceId, blobInfo.Path);
					}

					// Per-namespace isolation: defer blobs from backed-up namespaces
					if (storageConfig.EnablePerNamespaceQueueIsolation && pausedNamespaces != null && pausedNamespaces.Contains(blobInfo.NamespaceId))
					{
						await _redisService.GetDatabase().SetAddAsync(GetDeferredScanSet(blobInfo.NamespaceId), blobInfo.Id.ToByteArray());
						continue;
					}

					if (blobInfo.Id < latestInfoId)
					{
						if (batchEntries != null)
						{
							if (!batchEntries.TryGetValue(blobInfo.NamespaceId, out List<SortedSetEntry<ObjectId>>? entries))
							{
								entries = new List<SortedSetEntry<ObjectId>>();
								batchEntries[blobInfo.NamespaceId] = entries;
							}
							entries.Add(new SortedSetEntry<ObjectId>(blobInfo.Id, score));
						}
						else
						{
							_ = _redisService.GetDatabase().SortedSetAddAsync(GetGcCheckSet(blobInfo.NamespaceId), blobInfo.Id, score, flags: CommandFlags.FireAndForget);
						}
						namespaceCounts[blobInfo.NamespaceId] = namespaceCounts.GetValueOrDefault(blobInfo.NamespaceId) + 1;
					}
				}

				// Flush batched entries (reliable enqueue path)
				if (batchEntries != null)
				{
					foreach ((NamespaceId nsId, List<SortedSetEntry<ObjectId>> entries) in batchEntries)
					{
						try
						{
							await _redisService.GetDatabase().SortedSetAddAsync(GetGcCheckSet(nsId), entries.ToArray());
						}
						catch (Exception ex)
						{
							_gcEnqueueFailuresCounter.Add(1, new KeyValuePair<string, object?>("namespace", nsId.ToString()));
							namespaceEnqueueFailures[nsId] = namespaceEnqueueFailures.GetValueOrDefault(nsId) + 1;
							_logger.LogError(ex, "Failed to enqueue {Count} blobs for GC in namespace {NamespaceId}", entries.Count, nsId);
							throw;
						}
					}
				}

				// Update the last imported blob id
				gcState = await _gcState.UpdateAsync(s => s.LastImportBlobInfoId = current[^1].Id, cancellationToken);
				ingestedCount += current.Count;
			}

			_gcBlobsIngestedCounter.Add(ingestedCount);
			_logger.LogInformation("Added {NumBlobs} blobs for GC (upper time: {Time})", ingestedCount, gcState.LastImportBlobInfoId.CreationTime);

			foreach ((NamespaceId nsId, long count) in namespaceCounts)
			{
				GcCounters counters = GetOrAddCounters(nsId);
				counters.AddIngested(count);
				counters.AddEnqueueFailures(namespaceEnqueueFailures.GetValueOrDefault(nsId));
			}
		}

		async Task DrainDeferredSetsAsync(GcState gcState, StorageConfig storageConfig, CancellationToken cancellationToken)
		{
			double score = GetGcTimestamp();

			foreach (GcNamespaceState nsState in gcState.Namespaces)
			{
				// Check if this namespace is still backed up
				long length = await _redisService.GetDatabase().SortedSetLengthAsync(GetGcCheckSet(nsState.Id)).WaitAsync(cancellationToken);
				if (length > storageConfig.GcQueueLimit)
				{
					continue; // Still backed up — leave for next tick
				}

				RedisKey deferredKey = GetDeferredScanSet(nsState.Id);
				RedisValue[] deferred = await _redisService.GetDatabase().SetPopAsync(deferredKey, count: 500);
				if (deferred.Length > 0)
				{
					_logger.LogInformation("Draining {Count} deferred blobs for namespace {NamespaceId}", deferred.Length, nsState.Id);
					SortedSetEntry<ObjectId>[] entries = deferred
						.Select(v => new SortedSetEntry<ObjectId>(new ObjectId((byte[])v!), score))
						.ToArray();
					await _redisService.GetDatabase().SortedSetAddAsync(GetGcCheckSet(nsState.Id), entries);
				}
			}
		}

		async Task<HashSet<NamespaceId>?> GetPausedNamespacesAsync(StorageConfig storageConfig, IEnumerable<BlobInfo> blobs, CancellationToken cancellationToken)
		{
			HashSet<NamespaceId>? pausedNamespaces = null;
			foreach (NamespaceId namespaceId in blobs.Select(x => x.NamespaceId).Distinct())
			{
				long length = await _redisService.GetDatabase().SortedSetLengthAsync(GetGcCheckSet(namespaceId)).WaitAsync(cancellationToken);
				if (length > storageConfig.GcQueueLimit)
				{
					_logger.LogInformation("Length of GC queue for namespace {NamespaceId} is {Count}. Pausing addition of new items.", namespaceId, length);
					_gcTickThrottleEventsCounter.Add(1, new KeyValuePair<string, object?>("namespace", namespaceId.ToString()));
					pausedNamespaces ??= new HashSet<NamespaceId>();
					pausedNamespaces.Add(namespaceId);
				}
			}
			return pausedNamespaces;
		}

		#endregion

		#region Nodes

		async Task AddAliasesAsync(NamespaceId namespaceId, IEnumerable<AddAliasRequest> addAliases, CancellationToken cancellationToken)
		{
			foreach (IGrouping<BlobLocator, AddAliasRequest> group in addAliases.GroupBy(x => x.Target.BaseLocator))
			{
				string path = group.Key.ToString();
				for (; ; )
				{
					// Get the existing blob definition
					BlobInfo? blobInfo = await _blobCollection.Find(x => x.NamespaceId == namespaceId && x.Path == path).FirstOrDefaultAsync(cancellationToken);
					if (blobInfo == null)
					{
						break;
					}

					// Create all the new alises
					List<AliasInfo> aliases = new List<AliasInfo>();
					foreach (AddAliasRequest addAlias in group)
					{
						string fragment = addAlias.Target.Fragment.ToString();
						aliases.Add(new AliasInfo(addAlias.Name, fragment, addAlias.Data, addAlias.Rank));
					}

					// Add all the current aliases that haven't been replaced
					if (blobInfo.Aliases != null)
					{
						HashSet<(string Name, string Fragment)> excludeAliases = aliases.Select(x => (x.Name, x.Fragment)).ToHashSet();
						foreach (AliasInfo aliasInfo in blobInfo.Aliases)
						{
							if (!excludeAliases.Contains((aliasInfo.Name, aliasInfo.Fragment)))
							{
								aliases.Add(aliasInfo);
							}
						}
					}

					// Update the blob info
					FilterDefinition<BlobInfo> filter = Builders<BlobInfo>.Filter.Eq(x => x.Id, blobInfo.Id);
					if (blobInfo.UpdateIndex == null)
					{
						filter &= Builders<BlobInfo>.Filter.Exists(x => x.UpdateIndex, false);
					}
					else
					{
						filter &= Builders<BlobInfo>.Filter.Eq(x => x.UpdateIndex, blobInfo.UpdateIndex);
					}

					UpdateDefinition<BlobInfo> update = Builders<BlobInfo>.Update.Set(x => x.Aliases, aliases).Set(x => x.UpdateIndex, (blobInfo.UpdateIndex ?? 0) + 1);

					UpdateResult result = await _blobCollection.UpdateOneAsync(filter, update, cancellationToken: cancellationToken);
					if (result.MatchedCount > 0)
					{
						break;
					}
				}

				foreach (AddAliasRequest addAlias in group)
				{
					_logger.LogDebug("Added alias {Name} to {Target}", addAlias.Name, addAlias.Target);
				}
			}
		}

		async Task RemoveAliasesAsync(NamespaceId namespaceId, IEnumerable<RemoveAliasRequest> removeAliases, CancellationToken cancellationToken)
		{
			foreach (IGrouping<BlobLocator, RemoveAliasRequest> group in removeAliases.GroupBy(x => x.Target.BaseLocator))
			{
				string path = group.Key.ToString();
				for (; ; )
				{
					// Get the existing blob definition
					BlobInfo? blobInfo = await _blobCollection.Find(x => x.NamespaceId == namespaceId && x.Path == path).FirstOrDefaultAsync(cancellationToken);
					if (blobInfo == null || blobInfo.Aliases == null)
					{
						break;
					}

					// Build a set of aliases to remove
					HashSet<(string Name, string Fragment)> excludeAliases = new();
					foreach (RemoveAliasRequest removeAlias in group)
					{
						string fragment = removeAlias.Target.Fragment.ToString();
						excludeAliases.Add((removeAlias.Name, fragment));
					}

					// Remove them from the blob
					List<AliasInfo> aliases = blobInfo.Aliases.Where(x => !excludeAliases.Contains((x.Name, x.Fragment))).ToList();

					// Update the blob info
					FilterDefinition<BlobInfo> filter = Builders<BlobInfo>.Filter.Eq(x => x.Id, blobInfo.Id);
					if (blobInfo.UpdateIndex == null)
					{
						filter &= Builders<BlobInfo>.Filter.Exists(x => x.UpdateIndex, false);
					}
					else
					{
						filter &= Builders<BlobInfo>.Filter.Eq(x => x.UpdateIndex, blobInfo.UpdateIndex);
					}

					UpdateDefinition<BlobInfo> update = Builders<BlobInfo>.Update.Set(x => x.UpdateIndex, (blobInfo.UpdateIndex ?? 0) + 1);
					if (aliases.Count == 0)
					{
						update = update.Unset(x => x.Aliases);
					}
					else
					{
						update = update.Set(x => x.Aliases, aliases);
					}

					UpdateResult result = await _blobCollection.UpdateOneAsync(filter, update, cancellationToken: cancellationToken);
					if (result.MatchedCount > 0)
					{
						break;
					}
				}

				foreach (RemoveAliasRequest removeAlias in group)
				{
					_logger.LogDebug("Removed alias {Name} from {Target}", removeAlias.Name, removeAlias.Target);
				}
			}
		}

		/// <summary>
		/// Finds nodes with the given type and hash
		/// </summary>
		/// <param name="namespaceId">Namespace to search</param>
		/// <param name="name">Alias for the node</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sequence of thandles</returns>
		async Task<List<(BlobLocator, AliasInfo)>> FindAliasesAsync(NamespaceId namespaceId, string name, CancellationToken cancellationToken = default)
		{
			FilterDefinition<BlobInfo> filter =
				Builders<BlobInfo>.Filter.Eq(x => x.NamespaceId, namespaceId) & Builders<BlobInfo>.Filter.Eq(s_blobAliasField, name);

			List<(BlobLocator, AliasInfo)> results = new List<(BlobLocator, AliasInfo)>();
			await foreach (BlobInfo blobInfo in _blobCollection.Find(filter).ToAsyncEnumerable(cancellationToken))
			{
				if (blobInfo.Aliases != null)
				{
					foreach (AliasInfo aliasInfo in blobInfo.Aliases)
					{
						if (String.Equals(aliasInfo.Name, name, StringComparison.Ordinal))
						{
							BlobLocator locator = new BlobLocator(blobInfo.Locator, aliasInfo.Fragment);
							results.Add((locator, aliasInfo));
						}
					}
				}
			}
			return results.OrderByDescending(x => x.Item2.Rank).ToList();
		}

		#endregion

		#region Refs

		record RefCacheKey(NamespaceId NamespaceId, RefName Name);
		record RefCacheValue(RefInfo? Value, DateTime Time);

		/// <summary>
		/// Adds a ref value to the cache
		/// </summary>
		/// <param name="namespaceId">Namespace containing the ref</param>
		/// <param name="name">Name of the ref</param>
		/// <param name="value">New target for the ref</param>
		/// <returns>The cached value</returns>
		RefCacheValue AddRefToCache(NamespaceId namespaceId, RefName name, RefInfo? value)
		{
			RefCacheValue cacheValue = new RefCacheValue(value, DateTime.UtcNow);
			using (ICacheEntry newEntry = _memoryCache.CreateEntry(new RefCacheKey(namespaceId, name)))
			{
				newEntry.Value = cacheValue;
				newEntry.SetSize(name.Text.Length);
				newEntry.SetSlidingExpiration(TimeSpan.FromMinutes(5.0));
			}
			return cacheValue;
		}

		/// <summary>
		/// Expires any refs that are no longer valid
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async ValueTask TickRefsAsync(CancellationToken cancellationToken)
		{
			DateTime utcNow = DateTime.UtcNow;

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StorageService)}.{nameof(TickRefsAsync)}");

			GcState gcState = await _gcState.GetAsync(cancellationToken);
			Dictionary<NamespaceId, long> namespaceRefCounts = new Dictionary<NamespaceId, long>();

			FilterDefinition<RefInfo> queryFilter = Builders<RefInfo>.Filter.Exists(x => x.ExpiresAtUtc) & Builders<RefInfo>.Filter.Lt(x => x.ExpiresAtUtc, utcNow);
			using (IAsyncCursor<RefInfo> cursor = await _refCollection.Find(queryFilter).ToCursorAsync(cancellationToken))
			{
				List<DeleteOneModel<RefInfo>> requests = new List<DeleteOneModel<RefInfo>>();
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					requests.Clear();

					foreach (RefInfo refInfo in cursor.Current)
					{
						_logger.LogInformation("Expired ref {NamespaceId}:{RefName}", refInfo.NamespaceId, refInfo.Name);
						FilterDefinition<RefInfo> filter = Builders<RefInfo>.Filter.Expr(x => x.Id == refInfo.Id && x.ExpiresAtUtc == refInfo.ExpiresAtUtc);
						requests.Add(new DeleteOneModel<RefInfo>(filter));
						AddGcCheckRecord(refInfo.NamespaceId, refInfo.TargetBlobId, gcState.LastImportBlobInfoId);
						AddRefToCache(refInfo.NamespaceId, refInfo.Name, default);
						namespaceRefCounts[refInfo.NamespaceId] = namespaceRefCounts.GetValueOrDefault(refInfo.NamespaceId) + 1;
					}

					if (requests.Count > 0)
					{
						await _refCollection.BulkWriteAsync(requests, cancellationToken: cancellationToken);
					}
				}
			}

			foreach ((NamespaceId nsId, long count) in namespaceRefCounts)
			{
				GetOrAddCounters(nsId).AddRefsExpired(count);
			}
		}

		/// <inheritdoc/>
		async Task<bool> RemoveRefAsync(NamespaceId namespaceId, RefName name, CancellationToken cancellationToken = default)
		{
			FilterDefinition<RefInfo> filter = Builders<RefInfo>.Filter.Expr(x => x.NamespaceId == namespaceId && x.Name == name);
			return await RemoveRefInternalAsync(namespaceId, name, filter, cancellationToken);
		}

		/// <summary>
		/// Deletes a ref document that has reached its expiry time
		/// </summary>
		async Task RemoveExpiredRefAsync(RefInfo refDocument, CancellationToken cancellationToken = default)
		{
			FilterDefinition<RefInfo> filter = Builders<RefInfo>.Filter.Expr(x => x.Id == refDocument.Id && x.ExpiresAtUtc == refDocument.ExpiresAtUtc);
			await RemoveRefInternalAsync(refDocument.NamespaceId, refDocument.Name, filter, cancellationToken);
		}

		async Task<bool> RemoveRefInternalAsync(NamespaceId namespaceId, RefName name, FilterDefinition<RefInfo> filter, CancellationToken cancellationToken = default)
		{
			RefInfo? oldRefInfo = await _refCollection.FindOneAndDeleteAsync<RefInfo>(filter, cancellationToken: cancellationToken);
			AddRefToCache(namespaceId, name, default);

			_logger.LogDebug("Removed ref {Name} from {Target}", name, oldRefInfo?.Target);

			if (oldRefInfo != null)
			{
				_logger.LogInformation("Deleted ref {NamespaceId}:{RefName}", namespaceId, name);
				GcState gcState = await _gcState.GetAsync(cancellationToken);
				AddGcCheckRecord(namespaceId, oldRefInfo.TargetBlobId, gcState.LastImportBlobInfoId);
				return true;
			}

			return false;
		}

		/// <inheritdoc/>
		async Task<RefInfo?> TryReadRefAsync(NamespaceId namespaceId, RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			RefCacheValue? entry;
			if (!_memoryCache.TryGetValue(name, out entry) || entry == null || RefCacheTime.IsStaleCacheEntry(entry.Time, cacheTime))
			{
				RefInfo? refDocument = await _refCollection.Find(x => x.NamespaceId == namespaceId && x.Name == name).FirstOrDefaultAsync(cancellationToken);
				entry = AddRefToCache(namespaceId, name, refDocument);
			}

			if (entry.Value == null)
			{
				return null;
			}

			if (entry.Value.ExpiresAtUtc != null)
			{
				DateTime utcNow = _clock.UtcNow;
				if (entry.Value.HasExpired(utcNow))
				{
					await RemoveExpiredRefAsync(entry.Value, cancellationToken);
					return default;
				}
				if (entry.Value.RequiresTouch(utcNow))
				{
					await _refCollection.UpdateOneAsync(x => x.Id == entry.Value.Id, Builders<RefInfo>.Update.Set(x => x.ExpiresAtUtc, utcNow + entry.Value.Lifetime!.Value), cancellationToken: cancellationToken);
				}
			}

			return entry.Value;
		}

		/// <inheritdoc/>
		async Task AddRefAsync(NamespaceId namespaceId, RefName name, HashedBlobRefValue value, RefOptions? options = null, CancellationToken cancellationToken = default)
		{
			string path = value.Locator.BaseLocator.ToString();

			BlobInfo? newBlobInfo = await _blobCollection.Find(x => x.NamespaceId == namespaceId && x.Path == path).FirstOrDefaultAsync(cancellationToken);
			if (newBlobInfo == null)
			{
				throw new Exception($"Invalid/unknown blob identifier '{path}' in namespace {namespaceId}");
			}

			RefInfo newRefInfo = new RefInfo(namespaceId, name, value.Hash, value.Locator, newBlobInfo.Id);

			if (options != null && options.Lifetime.HasValue)
			{
				newRefInfo.ExpiresAtUtc = _clock.UtcNow + options.Lifetime.Value;
				if (options.Extend ?? true)
				{
					newRefInfo.Lifetime = options.Lifetime;
				}
			}

			RefInfo? oldRefInfo = await _refCollection.FindOneAndReplaceAsync<RefInfo>(x => x.NamespaceId == namespaceId && x.Name == name, newRefInfo, new FindOneAndReplaceOptions<RefInfo> { IsUpsert = true }, cancellationToken);
			if (oldRefInfo != null)
			{
				GcState gcState = await _gcState.GetAsync(cancellationToken);
				AddGcCheckRecord(namespaceId, oldRefInfo.TargetBlobId, gcState.LastImportBlobInfoId);
			}

			_logger.LogDebug("Added ref {NamespaceId}:{Name} to {Target}", namespaceId, name, value);
			AddRefToCache(namespaceId, name, newRefInfo);
		}

		#endregion

		#region GC

		uint GetGcTimestamp() => GetGcTimestamp(_clock.UtcNow);
		static uint GetGcTimestamp(DateTime utcTime) => (uint)((utcTime - DateTime.UnixEpoch).Ticks / TimeSpan.TicksPerMinute);

		static RedisKey GetRedisKey(string suffix) => $"storage:{suffix}";
		static RedisKey GetRedisKey(NamespaceId namespaceId, string suffix) => GetRedisKey($"{namespaceId}:{suffix}");

		static RedisSortedSetKey<ObjectId> GetGcCheckSet(NamespaceId namespaceId) => new RedisSortedSetKey<ObjectId>(GetRedisKey(namespaceId, "check"));

		static RedisKey GetDeferredScanSet(NamespaceId namespaceId) => GetRedisKey(namespaceId, "gc-deferred");

		/// <summary>
		/// Find the next namespace to run GC on
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		async ValueTask TickGcAsync(CancellationToken cancellationToken)
		{
			HashSet<NamespaceId> ranNamespaceIds = new HashSet<NamespaceId>();

			DateTime utcNow = _clock.UtcNow;
			for (; ; )
			{
				StorageConfig storageConfig = _storageConfig.CurrentValue;
				if (!storageConfig.EnableGc && !storageConfig.EnableGcVerification)
				{
					break;
				}

				State state = CreateState(storageConfig);

				// Synchronize the list of configured namespaces with the GC state object
				GcState gcState = await _gcState.GetAsync(cancellationToken);
				if (!Enumerable.SequenceEqual(storageConfig.Namespaces.Select(x => x.Id.Text.Text).OrderBy(x => x), gcState.Namespaces.Select(x => x.Id.Text.Text).OrderBy(x => x)))
				{
					gcState = await _gcState.UpdateAsync(s => SyncNamespaceList(s, storageConfig.Namespaces), cancellationToken);
				}

				// Find all the namespaces that need to have GC run on them
				List<(DateTime, GcNamespaceState)> pending = new List<(DateTime, GcNamespaceState)>();
				foreach (GcNamespaceState namespaceState in gcState.Namespaces)
				{
					if (!ranNamespaceIds.Contains(namespaceState.Id))
					{
						NamespaceConfig? namespaceConfig;
						if (storageConfig.TryGetNamespace(namespaceState.Id, out namespaceConfig))
						{
							DateTime time = namespaceState.LastTime + TimeSpan.FromHours(namespaceConfig.GcFrequencyHrs);
							if (time < utcNow)
							{
								pending.Add((time, namespaceState));
							}
						}
					}
				}
				pending.SortBy(x => x.Item1);

				// If there's nothing left to GC, bail out
				if (pending.Count == 0)
				{
					break;
				}

				// Update the first one we can acquire a lock for
				foreach ((_, GcNamespaceState namespaceState) in pending)
				{
					NamespaceId namespaceId = namespaceState.Id;
					if (ranNamespaceIds.Add(namespaceId))
					{
						RedisKey key = GetRedisKey(namespaceId, "lock");
#pragma warning disable CA2000 // Dispose objects before losing scope
						using (RedisLock namespaceLock = new RedisLock(_redisService.GetDatabase(), key))
						{
							if (await namespaceLock.AcquireAsync(TimeSpan.FromMinutes(20.0)))
							{
								try
								{
									await TickGcForNamespaceAsync(state.Namespaces[namespaceId], gcState.LastImportBlobInfoId, utcNow, cancellationToken);
								}
								catch (OperationCanceledException ex)
								{
									_logger.LogInformation(ex, "Cancelled GC pass for {NamespaceId}", namespaceId);
								}
								catch (Exception ex)
								{
									_logger.LogError(ex, "Exception while running garbage collection: {Message}", ex.Message);
								}
								break;
							}
						}
#pragma warning restore CA2000 // Dispose objects before losing scope
					}
				}
			}
		}

		class GcSweepState
		{
			long _score;
			int _numItemsRemoved;
			long _bytesFreed;
			int _numItemsChecked;

			public int NumRemovedItems
				=> _numItemsRemoved;

			public long BytesFreed => Interlocked.Read(ref _bytesFreed);
			public int NumItemsChecked => _numItemsChecked;

			public GcSweepState(double score)
				=> _score = BitConverter.DoubleToInt64Bits(score);

			public double GetNextScore()
				=> BitConverter.Int64BitsToDouble(Interlocked.Increment(ref _score));

			public void OnRemovedItem(long length)
			{
				Interlocked.Increment(ref _numItemsRemoved);
				Interlocked.Add(ref _bytesFreed, length);
			}

			public void OnCheckedItem()
				=> Interlocked.Increment(ref _numItemsChecked);
		}

		async Task TickGcForNamespaceAsync(NamespaceInfo namespaceInfo, ObjectId lastImportBlobInfoId, DateTime utcNow, CancellationToken cancellationToken)
		{
			// Runs the garbage collector over a particular namespace.
			// 
			// Horde assigns a unique id to each blob in the DB when uploaded, and we maintain a reverse index of all things that reference a blob. At any
			// moment in time, we can query this index to determine if a blob is referenced. (TODO: May want a grace period where we consider a ref's old AND
			// new value to handle races for updates?)
			//
			// Since we're not a CAS and never recycle ids, we can guarantee that any blobs that do not have any references cannot be reintroduced into the
			// live set, and don't have to worry about races with unreferenced blobs becoming referenced again.
			//
			// To prevent newly added blobs being GC'd before their references are present, we have a maximum object id that we consider for GC based on
			// creation timestamp. After the grace period elapses, all blobs are added to the reachability check set.
			//
			// For reachability analysis, we maintain a set of blob ids that need checking in Redis. Whenever a ref to a blob is removed, we add it to the set
			// with a score derived from the current time. If a second request to check a blob changes, we update the time. We leave the blob id in the set 
			// until we've done our reachability checks on it, and only remove it IFF the score matches the original value - preventing the second check
			// request being removed from the set.
			//
			// Querying the reachability set in score order ensures that heavily referenced blobs are continually pushed to the end, reducing the number of times
			// we need to do redundant checks on it.
			// 
			// There is a potential for the enumeration to skip over blobs; either because the check set drains before new items are added, or because
			// new scores are assigned before items are actually pushed to Redis. This isn't really a problem; we'll catch them the next
			// iteration and will process them eventually.

			using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
			using IDisposable? notification = _storageConfig.OnChange((_, _) => cancellationSource.Cancel());

			StorageConfig storageConfig = _storageConfig.CurrentValue;
			if (storageConfig.EnableGc || storageConfig.EnableGcVerification)
			{
				IStorageNamespace client = this.GetNamespace(namespaceInfo.Id);

				Stopwatch timer = Stopwatch.StartNew();
				_logger.LogInformation("Running garbage collection for namespace {NamespaceId}...", namespaceInfo.Id);

				// Get the current GC timestamp, in minutes since the Unix Epoch
				GcSweepState sweepState = new GcSweepState(GetGcTimestamp(utcNow));

				AsyncEvent queueChangeEvent = new AsyncEvent();
				Channel<SortedSetEntry<ObjectId>> channel = Channel.CreateBounded<SortedSetEntry<ObjectId>>(new BoundedChannelOptions(128));

				await using AsyncPipeline pipeline = new AsyncPipeline(cancellationSource.Token);
				_ = pipeline.AddTask(ctx => FindBlobsForReachabilityCheckAsync(namespaceInfo, channel.Writer, queueChangeEvent, ctx));
				pipeline.AddTasks(storageConfig.GcWorkerCount, channel.Reader, (entry, ctx) => CheckReachabilityAsync(namespaceInfo, entry, lastImportBlobInfoId, sweepState, storageConfig, queueChangeEvent, ctx));
				await pipeline.WaitForCompletionAsync();

				// Update the GC timestamp
				await _gcState.UpdateAsync(state => state.FindOrAddNamespace(namespaceInfo.Id).LastTime = utcNow, cancellationToken);
				_logger.LogInformation("Finished garbage collection for namespace {NamespaceId} in {TimeSecs}s ({NumItems} removed)", namespaceInfo.Id, timer.Elapsed.TotalSeconds, sweepState.NumRemovedItems);

				GetOrAddCounters(namespaceInfo.Id).AddSweep(sweepState.NumRemovedItems, sweepState.NumItemsChecked, sweepState.BytesFreed, timer.Elapsed.TotalMilliseconds);
			}
		}

		// Reads batches of blobs from the check set and queues them for reachability checking
		async Task FindBlobsForReachabilityCheckAsync(NamespaceInfo namespaceInfo, ChannelWriter<SortedSetEntry<ObjectId>> writer, AsyncEvent queueChangeEvent, CancellationToken cancellationToken)
		{
			// Keep a set of items we previously added to the queue, so we don't queue them again.
			HashSet<SortedSetEntry<ObjectId>> queuedItems = new HashSet<SortedSetEntry<ObjectId>>();

			Stopwatch? timer = null;

			RedisSortedSetKey<ObjectId> checkSet = GetGcCheckSet(namespaceInfo.Id);
			for (; ; )
			{
				// Capture the current state of the queue change event. We can use this to speculatively update the queue while being able to check for changes elsewhere
				queueChangeEvent.Reset();
				Task queueChangeTask = queueChangeEvent.Task;

				// Print the current queue stats
				if (timer == null || timer.Elapsed > TimeSpan.FromSeconds(30))
				{
					long length = await _redisService.GetDatabase().SortedSetLengthAsync(checkSet);
					_logger.LogInformation("Garbage collection queue for namespace {NamespaceId} ({QueueName}) has {Length} entries", namespaceInfo.Id, checkSet.Inner, length);
					timer = Stopwatch.StartNew();
				}

				// Get the current state of the queue. If it's empty, we're done.
				SortedSetEntry<ObjectId>[] entries = await _redisService.GetDatabase().SortedSetRangeByRankWithScoresAsync(checkSet, 0, 1024);
				if (entries.Length == 0)
				{
					_logger.LogInformation("Garbage collection complete for namespace {NamespaceId}", namespaceInfo.Id);
					writer.Complete();
					break;
				}

				// Write the entries to the channel
				HashSet<SortedSetEntry<ObjectId>> nextQueuedItems = new HashSet<SortedSetEntry<ObjectId>>(entries.Length);
				foreach (SortedSetEntry<ObjectId> entry in entries)
				{
					ObjectId blobInfoId = entry.Element;
					if (!queuedItems.Contains(entry))
					{
						await writer.WriteAsync(entry, cancellationToken);
					}
					nextQueuedItems.Add(entry);
				}
				queuedItems = nextQueuedItems;

				// Wait for something to be processed before running again
				await queueChangeTask;
			}
		}

		// Checks whether an individual blob can be removed
		async ValueTask CheckReachabilityAsync(NamespaceInfo namespaceInfo, SortedSetEntry<ObjectId> entry, ObjectId lastImportBlobInfoId, GcSweepState state, StorageConfig storageConfig, AsyncEvent queueChangeEvent, CancellationToken cancellationToken)
		{
			ObjectId blobInfoId = new ObjectId(((byte[]?)entry.ElementValue)!);

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StorageService)}.{nameof(TickGcForNamespaceAsync)}");
			span.SetAttribute("BlobId", blobInfoId.ToString());

			long startTimestamp = Stopwatch.GetTimestamp();
			RedisSortedSetKey<ObjectId> checkSet = GetGcCheckSet(namespaceInfo.Id);
			if (!await IsBlobReferencedAsync(blobInfoId, cancellationToken))
			{
				BlobInfo? info;
				if (storageConfig.EnableGc)
				{
					info = await _blobCollection.FindOneAndDeleteAsync(x => x.Id == blobInfoId, cancellationToken: cancellationToken);
				}
				else
				{
					info = await _blobCollection.FindOneAndUpdateAsync(x => x.Id == blobInfoId, Builders<BlobInfo>.Update.Set(x => x.GcVersion, CurrentGcVersion), cancellationToken: cancellationToken);
				}

				if (info != null)
				{
					if (info.Imports != null)
					{
						SortedSetEntry<ObjectId>[] entries = info.Imports.Where(x => x <= lastImportBlobInfoId).Select(x => new SortedSetEntry<ObjectId>(x, state.GetNextScore())).ToArray();
						await _redisService.GetDatabase().SortedSetAddAsync(checkSet, entries);
					}

					ObjectKey objectKey = GetObjectKey(new BlobLocator(info.Path));
					_logger.LogDebug("Deleting {NamespaceId} blob {BlobId}, key: {ObjectKey} ({ImportCount} imports)", namespaceInfo.Id, blobInfoId, objectKey, info.Imports?.Count ?? 0);

					if (storageConfig.EnableGc)
					{
						await namespaceInfo.Store.DeleteAsync(objectKey, cancellationToken);
					}

					_gcBlobsDeletedCounter.Add(1, new KeyValuePair<string, object?>("namespace", namespaceInfo.Id.ToString()));
					state.OnRemovedItem(info.Length);
				}
			}

			state.OnCheckedItem();

			double elapsedMs = Stopwatch.GetElapsedTime(startTimestamp).TotalMilliseconds;
			_gcReachabilityCheckDuration.Record(elapsedMs, new KeyValuePair<string, object?>("namespace", namespaceInfo.Id.ToString()));

			// Remove the item from the check set iff the score is the same
			for (; ; )
			{
				ITransaction transaction = _redisService.GetDatabase().CreateTransaction();
				ConditionResult scoreEqualCondition = transaction.AddCondition(checkSet.SortedSetEqual(entry.Element, entry.Score));
				_ = transaction.SortedSetRemoveAsync(checkSet, entry.Element);

				if (await transaction.ExecuteAsync() || !scoreEqualCondition.WasSatisfied)
				{
					break;
				}
			}

			// Flag that the queue has been updated
			queueChangeEvent.Latch();
		}

		static void SyncNamespaceList(GcState state, List<NamespaceConfig> namespaces)
		{
			HashSet<NamespaceId> validNamespaceIds = new HashSet<NamespaceId>(namespaces.Select(x => x.Id));
			state.Namespaces.RemoveAll(x => !validNamespaceIds.Contains(x.Id));

			HashSet<NamespaceId> currentNamespaceIds = new HashSet<NamespaceId>(state.Namespaces.Select(x => x.Id));
			foreach (NamespaceConfig config in namespaces)
			{
				if (!currentNamespaceIds.Contains(config.Id))
				{
					state.Namespaces.Add(new GcNamespaceState { Id = config.Id });
				}
			}

			state.Namespaces.SortBy(x => x.Id.Text.Text);
		}

		void AddGcCheckRecord(NamespaceId namespaceId, ObjectId id, ObjectId lastImportBlobId)
		{
			if (id < lastImportBlobId)
			{
				double score = GetGcTimestamp();
				_ = _redisService.GetDatabase().SortedSetAddAsync(GetGcCheckSet(namespaceId), id, score, flags: CommandFlags.FireAndForget);
			}
		}

		/// <summary>
		/// Reads and resets accumulated GC counters for all namespaces
		/// </summary>
		public Dictionary<NamespaceId, (long BlobsDeleted, long BlobsChecked, long BytesFreed, double SweepDurationMs, long BlobsIngested, long EnqueueFailures, long RefsExpired)> ReadAndResetGcCounters()
		{
			Dictionary<NamespaceId, (long, long, long, double, long, long, long)> result = new();
			foreach ((NamespaceId ns, GcCounters counters) in _gcCounters)
			{
				(long BlobsDeleted, long BlobsChecked, long BytesFreed, double SweepDurationMs, long BlobsIngested, long EnqueueFailures, long RefsExpired) snapshot = counters.ReadAndReset();
				if (snapshot.BlobsDeleted > 0 || snapshot.BlobsChecked > 0 || snapshot.BlobsIngested > 0 || snapshot.EnqueueFailures > 0 || snapshot.RefsExpired > 0)
				{
					result[ns] = snapshot;
				}
			}
			return result;
		}

		/// <summary>
		/// Gets per-namespace GC status (last GC time, queue depth) for dashboard
		/// </summary>
		public async Task<(List<(NamespaceId Id, DateTime LastTime)> Namespaces, Dictionary<NamespaceId, long> QueueDepths)> GetGcStatusAsync(CancellationToken cancellationToken)
		{
			GcState gcState = await _gcState.GetAsync(cancellationToken);
			StorageConfig storageConfig = _storageConfig.CurrentValue;

			List<(NamespaceId Id, DateTime LastTime)> namespaces = new List<(NamespaceId, DateTime)>();
			foreach (GcNamespaceState nsState in gcState.Namespaces)
			{
				namespaces.Add((nsState.Id, nsState.LastTime));
			}

			Dictionary<NamespaceId, long> queueDepths = new Dictionary<NamespaceId, long>();
			foreach (NamespaceConfig nsConfig in storageConfig.Namespaces)
			{
				long depth = await _redisService.GetDatabase().SortedSetLengthAsync(GetGcCheckSet(nsConfig.Id)).WaitAsync(cancellationToken);
				queueDepths[nsConfig.Id] = depth;
			}

			return (namespaces, queueDepths);
		}

		#endregion

		#region Length scan

		/// <summary>
		/// Scan new blobs for their sizes
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		internal async ValueTask TickLengthsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StorageService)}.{nameof(TickLengthsAsync)}");

			State state = CreateState(_storageConfig.CurrentValue);

			Channel<BlobInfo> channel = Channel.CreateBounded<BlobInfo>(new BoundedChannelOptions(1000));

			await using AsyncPipeline pipeline = new AsyncPipeline(cancellationToken);
			_ = pipeline.AddTask(ctx => FindBlobsForLengthScanAsync(channel.Writer, ctx));
			pipeline.AddTasks(8, channel.Reader, (entry, ctx) => FindBlobLengthsAsync(entry, state, ctx));
			await pipeline.WaitForCompletionAsync();
		}

		async Task FindBlobsForLengthScanAsync(ChannelWriter<BlobInfo> writer, CancellationToken cancellationToken)
		{
			DateTime latestTimeUtc = _clock.UtcNow - TimeSpan.FromMinutes(30.0);
			ObjectId latestInfoId = ObjectId.GenerateNewId(latestTimeUtc);

			long scannedCount = 0;
			LengthScanState lengthScanState = await _lengthScanState.GetAsync(cancellationToken);

			for (; ; )
			{
				// Reset the state
				if (lengthScanState.Reset)
				{
					_logger.LogInformation("Resetting scan for blob lengths...");
					lengthScanState = await _lengthScanState.UpdateAsync(x => x.DoReset(), cancellationToken);
				}

				// Fetch the next batch of blobs
				List<BlobInfo> current = await _blobCollection.Find(x => x.Id > lengthScanState.LastImportBlobInfoId && x.Id < latestInfoId).SortBy(x => x.Id).Limit(2000).ToListAsync(cancellationToken);
				if (current.Count == 0)
				{
					break;
				}

				// Add a check record for each new blob
				_logger.LogDebug("Finding length for {NumBlobs} blobs ({FirstId} to {LastId})", current.Count, current[0].Id, current[^1].Id);
				foreach (BlobInfo blobInfo in current)
				{
					await writer.WriteAsync(blobInfo, cancellationToken);
				}

				// Update the last imported blob id
				lengthScanState.LastImportBlobInfoId = current[^1].Id;
				if (!await _lengthScanState.TryUpdateAsync(lengthScanState, cancellationToken))
				{
					lengthScanState = await _lengthScanState.GetAsync(cancellationToken);
				}
				scannedCount += current.Count;
			}

			_logger.LogInformation("Added lengths for {NumBlobs} blobs", scannedCount);
			writer.Complete();
		}

		async ValueTask FindBlobLengthsAsync(BlobInfo blobInfo, State state, CancellationToken cancellationToken)
		{
			NamespaceInfo? namespaceInfo;
			if (state.Namespaces.TryGetValue(blobInfo.NamespaceId, out namespaceInfo))
			{
				ObjectKey key = GetObjectKey(blobInfo.Locator);
				long length = await namespaceInfo.Store.GetSizeAsync(key, cancellationToken);
				_logger.LogDebug("Length of blob {BlobId} ({Key}): {Length}", blobInfo.Id, key, length);

				FilterDefinition<BlobInfo> filter = Builders<BlobInfo>.Filter.Eq(x => x.Id, blobInfo.Id);
				UpdateDefinition<BlobInfo> update = Builders<BlobInfo>.Update.Set(x => x.Length, length);
				await _blobCollection.UpdateOneAsync(filter, update, cancellationToken: cancellationToken);
			}
		}

		#endregion
	}
}
