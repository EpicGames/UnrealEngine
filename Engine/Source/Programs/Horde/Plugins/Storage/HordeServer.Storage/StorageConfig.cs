// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Security.Claims;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Storage.ObjectStores;
using HordeServer.Utilities;

#pragma warning disable CA1716 // Rename type Namespace to it no longer conflicts with the reserved language keyword 'Namespace'
#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Storage
{
	/// <summary>
	/// Well-known namespace identifiers
	/// </summary>
	public static class Namespace
	{
		/// <summary>
		/// Log data
		/// </summary>
		public static NamespaceId Logs { get; } = new NamespaceId("horde-logs");
	}

	/// <summary>
	/// Configuration for storage
	/// </summary>
	public class StorageConfig : IPluginConfig
	{
		/// <summary>
		/// ACL for the storage system
		/// </summary>
		[JsonIgnore]
		public AclConfig Acl { get; set; } = null!;

		/// <summary>
		/// Hash of the current object revision to allow detecting changes
		/// </summary>
		[JsonIgnore]
		public string Revision { get; set; } = String.Empty;

		/// <summary>
		/// Whether to enable garbage collection
		/// </summary>
		public bool EnableGc { get; set; } = true;

		/// <summary>
		/// Whether to enable garbage collection in verification mode (nothing deleted, just logging on access to deleted blobs)
		/// </summary>
		public bool EnableGcVerification { get; set; }

		/// <summary>
		/// Number of concurrent GC reachability-check workers. Default 8.
		/// Reachability checks are I/O-bound (MongoDB queries), so higher values
		/// scale throughput roughly linearly up to the database's capacity.
		/// </summary>
		public int GcWorkerCount { get; set; } = 8;

		/// <summary>
		/// Maximum number of entries in a namespace's Redis GC queue before the
		/// blob scanner pauses. Default 50,000.
		/// </summary>
		public long GcQueueLimit { get; set; } = 50000;

		/// <summary>
		/// Whether to await Redis writes when enqueuing blobs for GC checking.
		/// When false (default), uses fire-and-forget which can silently drop
		/// entries on Redis failures. Set true for reliable enqueue.
		/// </summary>
		public bool EnableReliableGcEnqueue { get; set; }

		/// <summary>
		/// Whether to isolate GC queue limits per-namespace. When false (default),
		/// one backed-up namespace pauses the scanner for ALL namespaces. When true,
		/// only the backed-up namespace is paused and its blobs are deferred.
		/// </summary>
		public bool EnablePerNamespaceQueueIsolation { get; set; }

		/// <summary>
		/// Number of blobs to process per blob-ingestion tick (TickBlobsAsync).
		/// Default 500. Raise to increase scan throughput when catching up on a
		/// backlog of unscanned blobs.
		/// </summary>
		public int GcBlobIngestionBatchSize { get; set; } = 500;

		/// <summary>
		/// Number of blob IDs to batch together for the import-reachability check.
		/// Default 1 (current per-blob behavior). When greater than 1, uses $in query to check
		/// multiple blobs in a single MongoDB round-trip. Values of 50-200 recommended.
		/// </summary>
		public int GcBatchSize { get; set; } = 1;

		/// <summary>
		/// List of storage backends
		/// </summary>
		public List<BackendConfig> Backends { get; set; } = new List<BackendConfig>();

		/// <summary>
		/// List of namespaces for storage
		/// </summary>
		public List<NamespaceConfig> Namespaces { get; set; } = new List<NamespaceConfig>();

		private readonly Dictionary<BackendId, BackendConfig> _backendLookup = new Dictionary<BackendId, BackendConfig>();
		private readonly Dictionary<NamespaceId, NamespaceConfig> _namespaceLookup = new Dictionary<NamespaceId, NamespaceConfig>();

		/// <summary>
		/// Called after the config has been read from disk
		/// </summary>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			Acl = configOptions.ParentAcl;

			// Create a lookup for backend configs, allowing later entries to override previous ones
			_backendLookup.Clear();
			foreach (BackendConfig backendConfig in Backends)
			{
				_backendLookup[backendConfig.Id] = backendConfig;
			}

			// Fixup all the inherited properties
			Dictionary<BackendId, BackendConfig> mergedBackendConfigs = new Dictionary<BackendId, BackendConfig>(Backends.Count);
			foreach (BackendConfig backendConfig in Backends)
			{
				MergeBackendConfigs(backendConfig.Id, _backendLookup, mergedBackendConfigs);
			}

			// Compute the hash for each backend
			foreach (BackendConfig backendConfig in Backends)
			{
				using (MemoryStream stream = new MemoryStream())
				{
					JsonSerializerOptions options = new JsonSerializerOptions();
					JsonUtils.ConfigureJsonSerializer(options);

					JsonSerializer.Serialize(stream, backendConfig, options: options);

					backendConfig.Hash = IoHash.Compute(stream.ToArray());
				}
			}

			// Validate the backend config for each namespace
			_namespaceLookup.Clear();
			foreach (NamespaceConfig namespaceConfig in Namespaces)
			{
				BackendConfig? backendConfig;
				if (!_backendLookup.TryGetValue(namespaceConfig.Backend, out backendConfig))
				{
					throw new StorageException($"Missing or invalid backend identifier for namespace {namespaceConfig.Id}");
				}

				namespaceConfig.PostLoad(configOptions.ParentAcl);
				_namespaceLookup[namespaceConfig.Id] = namespaceConfig;
			}

			// Create a unique id to allow detect changes
			byte[] data = JsonSerializer.SerializeToUtf8Bytes<StorageConfig>(this, JsonUtils.DefaultSerializerOptions);
			Revision = IoHash.Compute(data).ToString();
		}

		static BackendConfig MergeBackendConfigs(BackendId backendId, Dictionary<BackendId, BackendConfig> baseIdToConfig, Dictionary<BackendId, BackendConfig> mergedIdToConfig)
		{
			BackendConfig? config;
			if (mergedIdToConfig.TryGetValue(backendId, out config))
			{
				if (config == null)
				{
					throw new InvalidDataException($"Configuration for storage backend '{backendId}' is recursive.");
				}
			}
			else
			{
				mergedIdToConfig.Add(backendId, null!);

				config = baseIdToConfig[backendId];
				if (!config.Base.IsEmpty)
				{
					BackendConfig baseConfig = MergeBackendConfigs(config.Base, baseIdToConfig, mergedIdToConfig);
					config.MergeDefaults(baseConfig);
				}

				mergedIdToConfig[backendId] = config;
			}
			return config;
		}

		/// <summary>
		/// Gets a backend with the given id
		/// </summary>
		/// <param name="backendId">Identifier for the backend</param>
		/// <param name="backendConfig">Receives the backend config on success</param>
		/// <returns>True on success</returns>
		public bool TryGetBackend(BackendId backendId, [NotNullWhen(true)] out BackendConfig? backendConfig) => _backendLookup.TryGetValue(backendId, out backendConfig);

		/// <summary>
		/// Gets a namespace with the given id
		/// </summary>
		/// <param name="namespaceId">Identifier for the backend</param>
		/// <param name="namespaceConfig">Receives the backend config on success</param>
		/// <returns>True on success</returns>
		public bool TryGetNamespace(NamespaceId namespaceId, [NotNullWhen(true)] out NamespaceConfig? namespaceConfig) => _namespaceLookup.TryGetValue(namespaceId, out namespaceConfig);
	}

	/// <summary>
	/// Static settings for the storage system
	/// </summary>
	public class StorageServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Directory to use for the coarse-grained backend cache. This caches full bundles downloaded from the upstream object store.
		/// </summary>
		public string? BundleCacheDir { get; set; }

		/// <summary>
		/// Maximum size of the storage cache on disk. Accepts standard binary suffixes (kb, mb, gb, tb, etc...)
		/// </summary>
		public string BundleCacheSize { get; set; } = "1gb";

		/// <summary>
		/// Accessor for the bundle cache size in bytes
		/// </summary>
		public long BundleCacheSizeBytes => StringUtils.ParseBytesString(BundleCacheSize);

		/// <summary>
		/// Overridden settings for storage backends. Useful for running against a production server with custom backends.
		/// </summary>
		public List<BackendConfig> Backends { get; set; } = new List<BackendConfig>();
	}

	/// <summary>
	/// Types of storage backend to use
	/// </summary>
	public enum StorageBackendType
	{
		/// <summary>
		/// Local filesystem
		/// </summary>
		FileSystem,

		/// <summary>
		/// AWS S3
		/// </summary>
		Aws,

		/// <summary>
		/// Azure blob store
		/// </summary>
		Azure,

		/// <summary>
		/// Google Cloud Storage
		/// </summary>
		Gcs,

		/// <summary>
		/// In-memory only (for testing)
		/// </summary>
		Memory,
	};

	/// <summary>
	/// Common settings object for different providers
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class BackendConfig : IAwsStorageOptions, IAzureStorageOptions, IGcsStorageOptions
	{
		/// <summary>
		/// The storage backend ID
		/// </summary>
		[Required]
		public BackendId Id { get; set; }

		/// <summary>
		/// Base backend to copy default settings from
		/// </summary>
		public BackendId Base { get; set; }

		/// <summary>
		/// Specifies another backend to read from if an object is not found in this one. Can be used when migrating data from one backend to another.
		/// </summary>
		public BackendId Secondary { get; set; }

		/// <inheritdoc/>
		public StorageBackendType? Type { get; set; }

		/// <inheritdoc/>
		public string? BaseDir { get; set; }

		/// <inheritdoc/>
		public string? AwsBucketName { get; set; }

		/// <inheritdoc/>
		public string? AwsBucketPath { get; set; }

		/// <inheritdoc/>
		public AwsCredentialsType? AwsCredentials { get; set; }

		/// <inheritdoc/>
		public string? AwsRole { get; set; }

		/// <inheritdoc/>
		public string? AwsProfile { get; set; }

		/// <inheritdoc/>
		public string? AwsRegion { get; set; }

		/// <inheritdoc/>
		public string? AzureConnectionString { get; set; }

		/// <inheritdoc/>
		public string? AzureContainerName { get; set; }

		/// <inheritdoc/>
		public string? RelayServer { get; set; }

		/// <inheritdoc/>
		public string? RelayToken { get; set; }

		/// <inheritdoc/>
		public string? GcsBucketName { get; set; }

		/// <inheritdoc/>
		public string? GcsBucketPath { get; set; }

		/// <summary>
		/// Hash of this backend config. Used for caching backend instances.
		/// </summary>
		[JsonIgnore]
		internal IoHash Hash { get; set; }

		/// <summary>
		/// Merge default values from another object
		/// </summary>
		/// <param name="other">Object to merge defaults from</param>
		public void MergeDefaults(BackendConfig other)
		{
			foreach (PropertyInfo propertyInfo in GetType().GetProperties(BindingFlags.Instance | BindingFlags.Public))
			{
				if (propertyInfo.GetValue(this) == null)
				{
					propertyInfo.SetValue(this, propertyInfo.GetValue(other));
				}
			}
		}
	}

	/// <summary>
	/// Configuration of a particular namespace
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class NamespaceConfig
	{
		/// <summary>
		/// Identifier for this namespace
		/// </summary>
		[Required]
		public NamespaceId Id { get; set; }

		/// <summary>
		/// Backend to use for this namespace
		/// </summary>
		[Required]
		public BackendId Backend { get; set; }

		/// <summary>
		/// Prefix for items within this namespace
		/// </summary>
		public string Prefix { get; set; } = String.Empty;

		/// <summary>
		/// How frequently to run garbage collection, in hours.
		/// </summary>
		public double GcFrequencyHrs { get; set; } = 0.1;

		/// <summary>
		/// How long to keep newly uploaded orphanned objects before allowing them to be deleted, in hours.
		/// </summary>
		public double GcDelayHrs { get; set; } = 6.0;

		/// <summary>
		/// Support querying exports by their aliases
		/// </summary>
		public bool EnableAliases { get; set; }

		/// <summary>
		/// Access list for this namespace
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Callback once the configuration has been read from disk
		/// </summary>
		public void PostLoad(AclConfig parentAcl)
		{
			Acl.PostLoad(parentAcl, $"namespace:{Id}", AclConfig.GetActions([typeof(StorageAclAction)]));
		}

		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to validate</param>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);
	}
}
