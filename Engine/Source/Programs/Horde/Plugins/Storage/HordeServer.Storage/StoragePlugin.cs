// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.ObjectStores;
using HordeServer.Plugins;
using HordeServer.Storage.ObjectStores;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Storage
{
	/// <summary>
	/// Entry point for the storage plugin
	/// </summary>
	[Plugin("Storage", GlobalConfigType = typeof(StorageConfig), ServerConfigType = typeof(StorageServerConfig))]
	public class StoragePlugin : IPluginStartup
	{
		readonly IServerInfo _serverInfo;
		readonly StorageServerConfig _staticConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public StoragePlugin(IServerInfo serverInfo, StorageServerConfig staticConfig)
		{
			_serverInfo = serverInfo;
			_staticConfig = staticConfig;
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{ }

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<StorageService>();
			services.AddSingleton<IStorageService>(sp => sp.GetRequiredService<StorageService>());
			services.AddScoped(sp => sp.GetRequiredService<StorageService>().CreateStorageClient(sp.GetRequiredService<IOptionsSnapshot<StorageConfig>>().Value));

			services.AddSingleton<IObjectStoreFactory, ObjectStoreFactory>();
			services.AddSingleton<AwsObjectStoreFactory>();
			services.AddSingleton<AzureObjectStoreFactory>();
			services.AddSingleton<GcsObjectStoreFactory>();
			services.AddSingleton<FileObjectStoreFactory>();

			services.AddSingleton<BundleCache>();
			services.AddSingleton<StorageBackendCache>(CreateStorageBackendCache);
			services.AddSingleton<GcMetricsCollection>();
			services.AddSingleton<CollectionStatsService>();

			if (_serverInfo.IsRunModeActive(RunMode.Worker) && !_serverInfo.ReadOnlyMode)
			{
				services.AddHostedService(provider => provider.GetRequiredService<StorageService>());
				services.AddHostedService(provider => provider.GetRequiredService<GcMetricsCollection>());
				services.AddHostedService(provider => provider.GetRequiredService<CollectionStatsService>());
			}
		}

		StorageBackendCache CreateStorageBackendCache(IServiceProvider serviceProvider)
		{
			IServerInfo serverInfo = serviceProvider.GetRequiredService<IServerInfo>();
			DirectoryReference cacheDir = DirectoryReference.Combine(serverInfo.DataDir, String.IsNullOrEmpty(_staticConfig.BundleCacheDir) ? "Cache" : _staticConfig.BundleCacheDir);
			return new StorageBackendCache(cacheDir, _staticConfig.BundleCacheSizeBytes, serviceProvider.GetRequiredService<ILogger<StorageBackendCache>>());
		}
	}

	/// <summary>
	/// Helper methods for storage config
	/// </summary>
	public static class StoragePluginExtensions
	{
		/// <summary>
		/// Configures the storage plugin
		/// </summary>
		public static void AddStorageConfig(this IDictionary<PluginName, IPluginConfig> dictionary, StorageConfig storageConfig)
			=> dictionary[new PluginName("Storage")] = storageConfig;
	}
}
