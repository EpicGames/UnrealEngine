// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data.Common;
using HordeServer.Dialects;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Authentication provider for an analytics service. Generic over the plugin's <typeparamref name="TConfig"/> so DI can bind a distinct instance per plugin — each reading its own config for its own OAuth endpoint, scope, secret ID, and fragment key.
	/// Without the type parameter, every consumer would share the same provider and the same config section.
	/// </summary>
	/// <typeparam name="TConfig">The plugin's analytics config type.</typeparam>
	public interface IAuthenticationProvider<TConfig> where TConfig : class, IOAuthClientCredentialsConfig
	{
		/// <summary>
		/// Gets the token.
		/// </summary>
		/// <returns>The token.</returns>
		Task<string?> GetTokenAsync();

		/// <summary>
		/// Gets the connection string authentication fragment.
		/// </summary>
		/// <returns>The connection string auth fragment.</returns>
		/// <remarks>
		///		As an example, https://learn.microsoft.com/en-us/azure/databricks/integrations/odbc/authentication#authentication-pass-through.
		///		In the standard ODBConnection format of Key=Value. Not ';' terminated.
		/// </remarks>
		Task<KeyValuePair<string, string>?> GetConnectionStringAuthenticationFragmentAsync();
	}

	/// <summary>
	/// Backend-agnostic base class for analytics services. Resolves an <see cref="IAnalyticsDataSource"/> from DI using the keyed-services pattern, with the key driven by <see cref="IAnalyticsBackendSelectorConfig.BackendName"/> on the consumer's config. This lets two consumers in the same deployment bind to different backend plugins (e.g. StructuredAnalytics on Odbc, PerformanceTrends on ClickHouse).
	///
	/// Subclasses build queries against the data source's <see cref="ISqlDialect"/> and route table identifiers through the formatter.
	/// </summary>
	/// <typeparam name="TConfig">The plugin's config type. Must implement <see cref="IAnalyticsBackendSelectorConfig"/> so the abstract can read <c>BackendName</c>.</typeparam>
	public abstract class AbstractAnalyticsService<TConfig> : IDisposable where TConfig : class, IAnalyticsBackendSelectorConfig
	{
		#region -- Members --

		private readonly ILogger _logger;
		private readonly IOptionsMonitor<TConfig> _config;
		private readonly IAnalyticsDataSource _dataSource;
		private bool _isDisposed = false;

		/// <summary>
		/// The logger.
		/// </summary>
		protected ILogger Logger => _logger;

		/// <summary>
		/// The config used for options.
		/// </summary>
		protected IOptionsMonitor<TConfig> Config => _config;

		/// <summary>
		/// The SQL dialect that pairs with connections from <see cref="OpenConnectionAsync"/>.
		/// </summary>
		protected ISqlDialect Dialect => _dataSource.Dialect;

		/// <summary>
		/// Canonical column name carrying the per-record telemetry timestamp (stamped at construction by <see cref="EpicGames.Analytics.AbstractTelemetryRecord"/>'s ctor and serialized by every sink). Use this for time-range filters and ORDER-BY in consumer queries instead of a string literal.
		/// </summary>
		protected const string TelemetryTimestampColumn = EpicGames.Analytics.AbstractTelemetryRecord.TelemetryTimestampColumnName;

		#endregion -- Members --

		#region -- IDisposable Api --

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);

			GC.SuppressFinalize(this);
		}

		#endregion -- IDisposeable Api --

		#region -- Protected Api --

		/// <summary>
		/// Constructor. Resolves the keyed <see cref="IAnalyticsDataSource"/> matching <see cref="IAnalyticsBackendSelectorConfig.BackendName"/> on the supplied config. The selected data source carries both its <see cref="ISqlDialect"/>.
		/// </summary>
		/// <param name="config">Configuration object that instruments the service.</param>
		/// <param name="serviceProvider">DI service provider used to resolve the keyed analytics data source.</param>
		/// <param name="logger">Logger.</param>
		protected AbstractAnalyticsService(IOptionsMonitor<TConfig> config, IServiceProvider serviceProvider, ILogger logger)
		{
			_config = config;
			_logger = logger;

			string backendName = config.CurrentValue.BackendName;
			if (String.IsNullOrEmpty(backendName))
			{
				throw new InvalidOperationException(
					$"BackendName must be set on the {typeof(TConfig).Name} block to a key registered by an enabled backend plugin (e.g. \"Odbc\" for OdbcTelemetry, \"ClickHouse\" for ClickHouseTelemetry).");
			}

			_dataSource = serviceProvider.GetRequiredKeyedService<IAnalyticsDataSource>(backendName);
		}

		/// <summary>
		/// Opens a new <see cref="DbConnection"/> via the registered <see cref="IAnalyticsDataSource"/>. The caller owns the returned connection and must dispose it.
		/// </summary>
		protected Task<DbConnection> OpenConnectionAsync(CancellationToken cancellationToken = default)
		{
			return _dataSource.OpenAsync(cancellationToken);
		}

		/// <summary>
		/// Disposes the instance.
		/// </summary>
		/// <param name="disposing">Whether this is originating from a <see cref="Dispose()"/> call or not.</param>
		protected virtual void Dispose(bool disposing)
		{
			if (_isDisposed)
			{
				return;
			}

			_isDisposed = true;
		}

		#endregion -- Protected Api --
	}
}
