// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Streams;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Streams;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Analytics Controller for structured analytics queries.
	/// </summary>
	[Authorize]
	[ApiController]
	public class AnalyticsController : HordeControllerBase
	{
		private readonly AnalyticsService _analyticsService;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly IOptionsSnapshot<StructuredAnalyticsConfig> _analyticsConfig;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="analyticsService">Analytics service to use in the controller.</param>
		/// <param name="buildConfig">Build config for authorization.</param>
		/// <param name="analyticsConfig">Analytics config for controller settings.</param>
		public AnalyticsController(AnalyticsService analyticsService, IOptionsSnapshot<BuildConfig> buildConfig, IOptionsSnapshot<StructuredAnalyticsConfig> analyticsConfig)
		{
			_analyticsService = analyticsService;
			_buildConfig = buildConfig;
			_analyticsConfig = analyticsConfig;
		}

		#region -- Job Summary Endpoints --

		/// <summary>
		/// Gets job summary telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of job summary telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/jobsummary")]
		[ProducesResponseType(typeof(List<JobSummaryTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetJobSummariesAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<JobSummaryTelemetry> results = await _analyticsService.GetJobSummariesAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Job Summary Endpoints --

		#region -- Commit Endpoints --

		/// <summary>
		/// Gets commit telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of commit telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/commits")]
		[ProducesResponseType(typeof(List<CommitTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetCommitsAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<CommitTelemetry> results = await _analyticsService.GetCommitsAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Commit Endpoints --

		#region -- Issue Endpoints --

		/// <summary>
		/// Gets issue telemetry records.
		/// </summary>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of issue telemetry records.</returns>
		/// <remarks>Issues are not stream-filtered. Access requires admin claims unless AllowNonAdminQueryAccess  is enabled in config.</remarks>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/issues")]
		[ProducesResponseType(typeof(List<IssueTelemetry>), StatusCodes.Status200OK)]
		[ProducesResponseType(StatusCodes.Status403Forbidden)]
		public async Task<IActionResult> GetIssuesAsync(
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Require admin access unless AllowNonAdminIssueAccess is enabled
			if (!_analyticsConfig.Value.AllowNonAdminQueryAccess && !User.HasAdminClaim())
			{
				return Forbid("Access to issue telemetry requires admin privileges.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<IssueTelemetry> results = await _analyticsService.GetIssuesAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Issue Endpoints --

		#region -- Issue Span Endpoints --

		/// <summary>
		/// Gets issue span telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="maxDate">The maximum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of issue span telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/issuespans")]
		[ProducesResponseType(typeof(List<IssueSpanTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetIssueSpansAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<IssueSpanTelemetry> results = await _analyticsService.GetIssueSpansAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Issue Span Endpoints --

		#region -- Job Label Endpoints --

		/// <summary>
		/// Gets job label telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="maxDate">The maximum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of job label telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/joblabels")]
		[ProducesResponseType(typeof(List<JobLabelTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetJobLabelsAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<JobLabelTelemetry> results = await _analyticsService.GetJobLabelsAsync(filter, cancellationToken);
			return Ok(results);
		}

		/// <summary>
		/// Gets job label summary telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="maxDate">The maximum date to filter by (against the record's <c>telemetry_timestamp</c>).</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of job label summary telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/joblabelsummary")]
		[ProducesResponseType(typeof(List<JobLabelSummaryTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetJobLabelSummariesAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<JobLabelSummaryTelemetry> results = await _analyticsService.GetJobLabelSummariesAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Job Label Endpoints --

		#region -- Job Step Ref Endpoints --

		/// <summary>
		/// Gets job step ref telemetry records.
		/// </summary>
		/// <param name="streamIds">The stream IDs to filter by. Required parameter.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of job step ref telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/jobsteprefs")]
		[ProducesResponseType(typeof(List<JobStepRefTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetJobStepRefsAsync(
			[FromQuery] string[] streamIds,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Validate streams parameter is provided
			if (streamIds == null || streamIds.Length == 0)
			{
				return BadRequest("At least one stream must be specified.");
			}

			// Filter to only authorized streams
			StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
			if (authorizedStreams.Length == 0)
			{
				return Forbid("User is not authorized to view any of the requested streams.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				StreamIds = authorizedStreams,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<JobStepRefTelemetry> results = await _analyticsService.GetJobStepRefsAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Job Step Ref Endpoints --

		#region -- Lease Endpoints --

		/// <summary>
		/// Gets lease complete telemetry records.
		/// </summary>
		/// <param name="streamIds">Optional stream IDs to filter by. If not provided, returns all leases.</param>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of lease complete telemetry records.</returns>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/leasecompletes")]
		[ProducesResponseType(typeof(List<LeaseCompleteTelemetry>), StatusCodes.Status200OK)]
		public async Task<IActionResult> GetLeaseCompletesAsync(
			[FromQuery] string[]? streamIds = null,
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				MinDate = minDate,
				MaxDate = maxDate
			};

			// If streams are specified, filter to authorized ones
			if (streamIds != null && streamIds.Length > 0)
			{
				StreamId[] authorizedStreams = FilterAuthorizedStreams(streamIds);
				if (authorizedStreams.Length == 0)
				{
					return Forbid("User is not authorized to view any of the requested streams.");
				}
				filter.StreamIds = authorizedStreams;
			}

			List<LeaseCompleteTelemetry> results = await _analyticsService.GetLeaseCompletesAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Lease Endpoints --

		#region -- Agent Telemetry Endpoints --

		/// <summary>
		/// Gets agent telemetry records.
		/// </summary>
		/// <param name="count">The number of records to retrieve.</param>
		/// <param name="minDate">The minimum date to filter by.</param>
		/// <param name="maxDate">The maximum date to filter by.</param>
		/// <param name="cancellationToken">Cancellation token.</param>
		/// <returns>A list of agent telemetry records.</returns>
		/// <remarks>Agent telemetry is not stream-filtered. Access requires admin claims unless AllowNonAdminQueryAccess is enabled in config.</remarks>
		[HttpGet]
		[Route("/api/v1/structuredanalytics/agenttelemetry")]
		[ProducesResponseType(typeof(List<AgentTelemetry>), StatusCodes.Status200OK)]
		[ProducesResponseType(StatusCodes.Status403Forbidden)]
		public async Task<IActionResult> GetAgentTelemetryAsync(
			[FromQuery] int count = StructuredAnalyticsFilter.DefaultRecordCount,
			[FromQuery] DateTime? minDate = null,
			[FromQuery] DateTime? maxDate = null,
			CancellationToken cancellationToken = default)
		{
			// Require admin access unless AllowNonAdminQueryAccess is enabled
			if (!_analyticsConfig.Value.AllowNonAdminQueryAccess && !User.HasAdminClaim())
			{
				return Forbid("Access to agent telemetry requires admin privileges.");
			}

			StructuredAnalyticsFilter filter = new StructuredAnalyticsFilter
			{
				RecordCount = count,
				MinDate = minDate,
				MaxDate = maxDate
			};

			List<AgentTelemetry> results = await _analyticsService.GetAgentTelemetryAsync(filter, cancellationToken);
			return Ok(results);
		}

		#endregion -- Agent Telemetry Endpoints --

		#region -- Private Helpers --

		/// <summary>
		/// Filters the provided stream IDs to only include streams the user is authorized to access.
		/// </summary>
		/// <param name="streamIds">The array of stream identifiers to filter.</param>
		/// <returns>An array containing only the authorized stream IDs.</returns>
		private StreamId[] FilterAuthorizedStreams(string[]? streamIds)
		{
			if (streamIds == null || streamIds.Length == 0)
			{
				return Array.Empty<StreamId>();
			}

			List<StreamId> authorized = new List<StreamId>(streamIds.Length);

			foreach (string streamIdStr in streamIds)
			{
				if (String.IsNullOrEmpty(streamIdStr))
				{
					continue;
				}

				StreamId streamId = new StreamId(streamIdStr);
				if (IsAuthorizedForStream(streamId))
				{
					authorized.Add(streamId);
				}
			}

			return authorized.ToArray();
		}

		/// <summary>
		/// Checks if the user is authorized to access a single stream.
		/// </summary>
		/// <param name="streamId">The stream ID to check authorization for.</param>
		/// <returns>True if authorized, false otherwise.</returns>
		private bool IsAuthorizedForStream(StreamId streamId)
		{
			// Check if stream-based auth is disabled
			if (_analyticsConfig.Value.DisableStreamBasedAuth)
			{
				return true;
			}

			// Attempt Horde ACL validation
			if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? config) && config.Authorize(StreamAclAction.ViewStream, User))
			{
				return true;
			}

			return false;
		}

		#endregion -- Private Helpers --
	}
}
