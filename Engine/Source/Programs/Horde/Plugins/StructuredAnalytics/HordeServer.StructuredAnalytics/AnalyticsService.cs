// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using Dapper;
using EpicGames.Analytics;
using EpicGames.Analytics.Generated;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.VersionControl.Perforce;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Service responsible for obtaining analytics data. Backend-agnostic: opens connections via <see cref="IAnalyticsDataSource"/> and reads results via <see cref="DbDataReader"/>.
	/// </summary>
	public sealed class AnalyticsService : AbstractAnalyticsService<StructuredAnalyticsConfig>
	{
		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="config">Configuration object that instruments the service.</param>
		/// <param name="serviceProvider">DI service provider used to resolve the keyed analytics data source named by <c>BackendName</c>. The data source bundles the table-name formatter and SQL dialect.</param>
		/// <param name="logger">Logger.</param>
		public AnalyticsService(IOptionsMonitor<StructuredAnalyticsConfig> config, IServiceProvider serviceProvider, ILogger<AnalyticsService> logger) : base(config, serviceProvider, logger)
		{
			TypeHandlers.RegisterTypeHandlers();
			ColumnHandlers.RegisterDapperColumnMapping<JobSummaryTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<JobLabelTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<JobLabelSummaryTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<JobStepRefTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<CommitTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<IssueTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<IssueSpanTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<LeaseCompleteTelemetry>();
			ColumnHandlers.RegisterDapperColumnMapping<AgentTelemetry>();
		}

		#region -- Public API --

		/// <summary>
		/// Obtains the job summaries.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of job summaries.</returns>
		public async Task<List<JobSummaryTelemetry>> GetJobSummariesAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_Jobs_JobSummaryTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_Jobs_JobSummaryTelemetryGen.SchemaVersion,
				Dialect);

			List<JobSummaryTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Jobs_JobSummaryTelemetryGen.EventName},
							{HordeServer_Jobs_JobSummaryTelemetryGen.SchemaVersion},
							{HordeServer_Jobs_JobSummaryTelemetryGen.StreamId},
							{HordeServer_Jobs_JobSummaryTelemetryGen.TemplateId},
							{HordeServer_Jobs_JobSummaryTelemetryGen.JobId},
							{HordeServer_Jobs_JobSummaryTelemetryGen.CommitId},
							{HordeServer_Jobs_JobSummaryTelemetryGen.CodeCommitId},
							{HordeServer_Jobs_JobSummaryTelemetryGen.CreateTimeUtc},
							{HordeServer_Jobs_JobSummaryTelemetryGen.FinishTimeUtc},
							{HordeServer_Jobs_JobSummaryTelemetryGen.JobWallTime},
							{HordeServer_Jobs_JobSummaryTelemetryGen.JobStepsTotalTime},
							{HordeServer_Jobs_JobSummaryTelemetryGen.PassRatio},
							{HordeServer_Jobs_JobSummaryTelemetryGen.PassWithWarningRatio},
							{HordeServer_Jobs_JobSummaryTelemetryGen.FailureRatio},
							{HordeServer_Jobs_JobSummaryTelemetryGen.WarningRatio},
							{HordeServer_Jobs_JobSummaryTelemetryGen.StepWarningCount},
							{HordeServer_Jobs_JobSummaryTelemetryGen.StepFailureCount},
							{HordeServer_Jobs_JobSummaryTelemetryGen.StepPassCount},
							{HordeServer_Jobs_JobSummaryTelemetryGen.StepTotalCount},
							{HordeServer_Jobs_JobSummaryTelemetryGen.IsPreflight}
						FROM {Dialect.FormatTableName(HordeServer_Jobs_JobSummaryTelemetryGen.SchemaName, HordeServer_Jobs_JobSummaryTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Jobs_JobSummaryTelemetryGen.CreateTimeUtc} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<JobSummaryTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the commit telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of commit telemetry records.</returns>
		public async Task<List<CommitTelemetry>> GetCommitsAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_VersionControl_Perforce_CommitTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_VersionControl_Perforce_CommitTelemetryGen.SchemaVersion,
				Dialect);

			List<CommitTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.EventName},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.SchemaVersion},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.SubmittedBy},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.StreamId},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.Change},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.OriginalChange},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.SubmittedAt},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.ExternalIssueKey},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.RobomergeAuthorTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.RobomergeSourceTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.RobomergeOwnerTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.RobomergeBotTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.ReviewTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.PreflightTag},
							{HordeServer_VersionControl_Perforce_CommitTelemetryGen.Virtualized}
						FROM {Dialect.FormatTableName(HordeServer_VersionControl_Perforce_CommitTelemetryGen.SchemaName, HordeServer_VersionControl_Perforce_CommitTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_VersionControl_Perforce_CommitTelemetryGen.SubmittedAt} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<CommitTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the issue telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of issue telemetry records.</returns>
		public async Task<List<IssueTelemetry>> GetIssuesAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			// Issues don't have a stream_id column directly - filter against the wire-format-guaranteed telemetry_timestamp.
			WhereClauseBuilder builder = new WhereClauseBuilder(Dialect);
			builder.AndIfNotNull(filter.MinDate, AbstractTelemetryRecord.TelemetryTimestampColumnName, ">=", DbType.DateTime);
			builder.AndIfNotNull(filter.MaxDate, AbstractTelemetryRecord.TelemetryTimestampColumnName, "<=", DbType.DateTime);
			builder.AndIfNotNull(filter.MinSchemaVersion, HordeServer_Issues_IssueTelemetryGen.SchemaVersion, ">=", DbType.Int32);
			WhereClauseResult whereClause = builder.Build();

			List<IssueTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Issues_IssueTelemetryGen.EventName},
							{HordeServer_Issues_IssueTelemetryGen.SchemaVersion},
							{HordeServer_Issues_IssueTelemetryGen.Context},
							{HordeServer_Issues_IssueTelemetryGen.Id},
							{HordeServer_Issues_IssueTelemetryGen.AcknowledgedAt},
							{HordeServer_Issues_IssueTelemetryGen.CreatedAt},
							{HordeServer_Issues_IssueTelemetryGen.OwnerId},
							{HordeServer_Issues_IssueTelemetryGen.ExternalIssueKey},
							{HordeServer_Issues_IssueTelemetryGen.FixChange},
							{HordeServer_Issues_IssueTelemetryGen.FixedSystemic},
							{HordeServer_Issues_IssueTelemetryGen.RootChange},
							{HordeServer_Issues_IssueTelemetryGen.LastSeenAt},
							{HordeServer_Issues_IssueTelemetryGen.NominatedAt},
							{HordeServer_Issues_IssueTelemetryGen.NominatedById},
							{HordeServer_Issues_IssueTelemetryGen.ResolvedAt},
							{HordeServer_Issues_IssueTelemetryGen.ResolvedById},
							{HordeServer_Issues_IssueTelemetryGen.Severity},
							{HordeServer_Issues_IssueTelemetryGen.Summary},
							{HordeServer_Issues_IssueTelemetryGen.VerifiedAt},
							{HordeServer_Issues_IssueTelemetryGen.DuplicateIssueId},
							{HordeServer_Issues_IssueTelemetryGen.RootCauseOwnerId},
							{HordeServer_Issues_IssueTelemetryGen.RootCauseCategory},
							{HordeServer_Issues_IssueTelemetryGen.RootCauseSummary}
						FROM {Dialect.FormatTableName(HordeServer_Issues_IssueTelemetryGen.SchemaName, HordeServer_Issues_IssueTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Issues_IssueTelemetryGen.CreatedAt} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<IssueTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the issue span telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of issue span telemetry records.</returns>
		public async Task<List<IssueSpanTelemetry>> GetIssueSpansAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_Issues_IssueSpanTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_Issues_IssueSpanTelemetryGen.SchemaVersion,
				Dialect);

			List<IssueSpanTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Issues_IssueSpanTelemetryGen.EventName},
							{HordeServer_Issues_IssueSpanTelemetryGen.SchemaVersion},
							{HordeServer_Issues_IssueSpanTelemetryGen.Id},
							{HordeServer_Issues_IssueSpanTelemetryGen.IssueId},
							{HordeServer_Issues_IssueSpanTelemetryGen.Fingerprint},
							{HordeServer_Issues_IssueSpanTelemetryGen.FirstFailure},
							{HordeServer_Issues_IssueSpanTelemetryGen.LastFailure},
							{HordeServer_Issues_IssueSpanTelemetryGen.StreamId},
							{HordeServer_Issues_IssueSpanTelemetryGen.StreamName},
							{HordeServer_Issues_IssueSpanTelemetryGen.TemplateRefId}
						FROM {Dialect.FormatTableName(HordeServer_Issues_IssueSpanTelemetryGen.SchemaName, HordeServer_Issues_IssueSpanTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Issues_IssueSpanTelemetryGen.IssueId} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<IssueSpanTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the job label telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of job label telemetry records.</returns>
		public async Task<List<JobLabelTelemetry>> GetJobLabelsAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_Jobs_JobLabelTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_Jobs_JobLabelTelemetryGen.SchemaVersion,
				Dialect);

			List<JobLabelTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Jobs_JobLabelTelemetryGen.EventName},
							{HordeServer_Jobs_JobLabelTelemetryGen.SchemaVersion},
							{HordeServer_Jobs_JobLabelTelemetryGen.JobId},
							{HordeServer_Jobs_JobLabelTelemetryGen.GraphContentHash},
							{HordeServer_Jobs_JobLabelTelemetryGen.TemplateId},
							{HordeServer_Jobs_JobLabelTelemetryGen.StreamId},
							{HordeServer_Jobs_JobLabelTelemetryGen.CommitId},
							{HordeServer_Jobs_JobLabelTelemetryGen.Category},
							{HordeServer_Jobs_JobLabelTelemetryGen.Label},
							{HordeServer_Jobs_JobLabelTelemetryGen.Outcome},
							{HordeServer_Jobs_JobLabelTelemetryGen.State}
						FROM {Dialect.FormatTableName(HordeServer_Jobs_JobLabelTelemetryGen.SchemaName, HordeServer_Jobs_JobLabelTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Jobs_JobLabelTelemetryGen.JobId} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<JobLabelTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the job label summary telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of job label summary telemetry records.</returns>
		public async Task<List<JobLabelSummaryTelemetry>> GetJobLabelSummariesAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_Jobs_JobLabelSummaryTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_Jobs_JobLabelSummaryTelemetryGen.SchemaVersion,
				Dialect);

			List<JobLabelSummaryTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					// Note: Array columns (SuccessfulLabels, WarningLabels, etc.) may require special handling per backend
					string sql = $@"
						SELECT
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.EventName},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.SchemaVersion},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.JobId},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.GraphContentHash},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.TemplateId},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.StreamId},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.CommitId},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.SuccessfulLabels},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.WarningLabels},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.FailedLabels},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.SuccessfulCategories},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.WarningCategories},
							{HordeServer_Jobs_JobLabelSummaryTelemetryGen.FailedCategories}
						FROM {Dialect.FormatTableName(HordeServer_Jobs_JobLabelSummaryTelemetryGen.SchemaName, HordeServer_Jobs_JobLabelSummaryTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Jobs_JobLabelSummaryTelemetryGen.JobId} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<JobLabelSummaryTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the job step ref telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of job step ref telemetry records.</returns>
		public async Task<List<JobStepRefTelemetry>> GetJobStepRefsAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			WhereClauseResult whereClause = filter.BuildWhereClause(
				HordeServer_Jobs_JobStepRefTelemetryGen.StreamId,
				dateColumn: null,
				HordeServer_Jobs_JobStepRefTelemetryGen.SchemaVersion,
				Dialect);

			List<JobStepRefTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Jobs_JobStepRefTelemetryGen.EventName},
							{HordeServer_Jobs_JobStepRefTelemetryGen.SchemaVersion},
							{HordeServer_Jobs_JobStepRefTelemetryGen.Id},
							{HordeServer_Jobs_JobStepRefTelemetryGen.JobId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.BatchId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.StepId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.BatchInitTime},
							{HordeServer_Jobs_JobStepRefTelemetryGen.BatchWaitTime},
							{HordeServer_Jobs_JobStepRefTelemetryGen.Change},
							{HordeServer_Jobs_JobStepRefTelemetryGen.JobName},
							{HordeServer_Jobs_JobStepRefTelemetryGen.FinishTime},
							{HordeServer_Jobs_JobStepRefTelemetryGen.JobStartTime},
							{HordeServer_Jobs_JobStepRefTelemetryGen.StepName},
							{HordeServer_Jobs_JobStepRefTelemetryGen.State},
							{HordeServer_Jobs_JobStepRefTelemetryGen.Outcome},
							{HordeServer_Jobs_JobStepRefTelemetryGen.PoolId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.StartTime},
							{HordeServer_Jobs_JobStepRefTelemetryGen.StreamId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.TemplateId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.AgentId},
							{HordeServer_Jobs_JobStepRefTelemetryGen.UpdateIssues},
							{HordeServer_Jobs_JobStepRefTelemetryGen.Duration},
							{HordeServer_Jobs_JobStepRefTelemetryGen.Message},
							{HordeServer_Jobs_JobStepRefTelemetryGen.MessageSeverity}
						FROM {Dialect.FormatTableName(HordeServer_Jobs_JobStepRefTelemetryGen.SchemaName, HordeServer_Jobs_JobStepRefTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Jobs_JobStepRefTelemetryGen.StartTime} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<JobStepRefTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the lease complete telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of lease complete telemetry records.</returns>
		public async Task<List<LeaseCompleteTelemetry>> GetLeaseCompletesAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			// Leases can optionally be filtered by stream
			WhereClauseBuilder builder = new WhereClauseBuilder(Dialect);
			if (filter.StreamIds != null && filter.StreamIds.Length > 0)
			{
				builder.BeginGroupAnd();
				for (int i = 0; i < filter.StreamIds.Length; ++i)
				{
					builder.Or(HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.StreamId, "=", filter.StreamIds[i].ToString(), DbType.AnsiString);
				}
				builder.EndGroup();
			}
			builder.AndIfNotNull(filter.MinDate, AbstractTelemetryRecord.TelemetryTimestampColumnName, ">=", DbType.DateTime);
			builder.AndIfNotNull(filter.MaxDate, AbstractTelemetryRecord.TelemetryTimestampColumnName, "<=", DbType.DateTime);
			builder.AndIfNotNull(filter.MinSchemaVersion, HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.SchemaVersion, ">=", DbType.Int32);
			WhereClauseResult whereClause = builder.Build();

			List<LeaseCompleteTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.EventName},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.SchemaVersion},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.LeaseId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.ParentId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.Name},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.AgentId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.SessionId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.StreamId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.PoolId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.LogId},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.StartTimeUtc},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.FinishTimeUtc},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.DurationSecs},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.Outcome},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.OutcomeReason},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.SetupTimeSecs},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.TeardownTimeSecs},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.CpuCount},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.GlobalCpuUtilization},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.CpuUserTimeSecs},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.CpuSystemTimeSecs},
							{HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.CpuIdleTimeSecs}
						FROM {Dialect.FormatTableName(HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.SchemaName, HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Agents_Leases_LeaseCompleteTelemetryGen.StartTimeUtc} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<LeaseCompleteTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		/// <summary>
		/// Obtains the agent telemetry records.
		/// </summary>
		/// <param name="filter">The filter to apply to the query.</param>
		/// <param name="cancellationToken">The cancellation token.</param>
		/// <returns>The list of agent telemetry records.</returns>
		public async Task<List<AgentTelemetry>> GetAgentTelemetryAsync(StructuredAnalyticsFilter filter, CancellationToken cancellationToken)
		{
			// Agent telemetry doesn't have stream filtering - use schema-version-only filter
			WhereClauseBuilder builder = new WhereClauseBuilder(Dialect);
			builder.AndIfNotNull(filter.MinSchemaVersion, HordeServer_Agents_AgentTelemetryGen.SchemaVersion, ">=", DbType.Int32);
			WhereClauseResult whereClause = builder.Build();

			List<AgentTelemetry> results = [];
			using (DbConnection dbConnection = await OpenConnectionAsync(cancellationToken))
			{
				using (DbCommand cmd = dbConnection.CreateCommand())
				{
					whereClause.ApplyTo(cmd);

					string sql = $@"
						SELECT
							{HordeServer_Agents_AgentTelemetryGen.EventName},
							{HordeServer_Agents_AgentTelemetryGen.SchemaVersion},
							{HordeServer_Agents_AgentTelemetryGen.AgentId},
							{HordeServer_Agents_AgentTelemetryGen.UserCpu},
							{HordeServer_Agents_AgentTelemetryGen.IdleCpu},
							{HordeServer_Agents_AgentTelemetryGen.SystemCpu},
							{HordeServer_Agents_AgentTelemetryGen.FreeRamMb},
							{HordeServer_Agents_AgentTelemetryGen.UsedRamMb},
							{HordeServer_Agents_AgentTelemetryGen.TotalRamMb},
							{HordeServer_Agents_AgentTelemetryGen.WorkingFreeDiskMb},
							{HordeServer_Agents_AgentTelemetryGen.WorkingTotalDiskMb},
							{HordeServer_Agents_AgentTelemetryGen.SystemFreeDiskMb},
							{HordeServer_Agents_AgentTelemetryGen.SystemTotalDiskMb}
						FROM {Dialect.FormatTableName(HordeServer_Agents_AgentTelemetryGen.SchemaName, HordeServer_Agents_AgentTelemetryGen.TableName)}
						{whereClause.Clause}
						ORDER BY {HordeServer_Agents_AgentTelemetryGen.AgentId} DESC
						LIMIT {filter.RecordCount}";

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities
					cmd.CommandType = CommandType.Text;

					using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
					{
						results = reader.Parse<AgentTelemetry>().AsList();
						await reader.CloseAsync();
					}
				}
			}

			return results;
		}

		#endregion -- Public API --
	}
}
