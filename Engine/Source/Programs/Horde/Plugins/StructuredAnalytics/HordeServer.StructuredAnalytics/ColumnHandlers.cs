// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.ComponentModel.DataAnnotations.Schema;
using System.Reflection;
using Dapper;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Column mapper helpers.
	/// </summary>
	public static class ColumnHandlers
	{
		private static readonly ConcurrentDictionary<Type, bool> s_registeredTypes = new();

		/// <summary>
		/// Registers column mappings for a provided type. The type should have <see cref="ColumnAttribute"/> annotations.
		/// </summary>
		/// <typeparam name="T">The type to map.</typeparam>
		public static void RegisterDapperColumnMapping<T>()
		{
			if (s_registeredTypes.TryAdd(typeof(T), true))
			{
				SqlMapper.SetTypeMap(typeof(T),
					new CustomPropertyTypeMap(
						typeof(T),
						(type, columnName) =>
						{
							return type.GetProperties(BindingFlags.Public | BindingFlags.Instance)
								.FirstOrDefault(p =>
									p.GetCustomAttribute<ColumnAttribute>()?.Name?
										.Equals(columnName, StringComparison.OrdinalIgnoreCase) == true
								)!;
						}
					)
				);
			}
		}
	}
}
