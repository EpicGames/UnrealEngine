// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using HordeServer.Analytics.Schemas;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics.Controllers
{
	/// <summary>
	/// Controller for managing telemetry schemas
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("/api/v1/telemetry-schemas")]
	public class TelemetrySchemasController : HordeControllerBase
	{
		readonly TelemetrySchemaService _schemaService;
		readonly SchemaInferenceService _inferenceService;
		readonly ITelemetrySchemaCollection _schemaCollection;
		readonly IUserCollection _userCollection;
		readonly ISchemaMigrator? _schemaMigrator;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		readonly ILogger<TelemetrySchemasController> _logger;

		/// <summary>
		/// Constructor. <paramref name="schemaMigrator"/> is optional — sinks that
		/// don't need migrations (or aren't yet implemented for a given backend) can
		/// run without one and the controller falls back to writing schemas without
		/// any DDL changes (matching the pre-migrator behaviour).
		/// </summary>
		public TelemetrySchemasController(
			TelemetrySchemaService schemaService,
			SchemaInferenceService inferenceService,
			ITelemetrySchemaCollection schemaCollection,
			IUserCollection userCollection,
			IOptionsSnapshot<BuildConfig> buildConfig,
			ILogger<TelemetrySchemasController> logger,
			ISchemaMigrator? schemaMigrator = null)
		{
			_schemaService = schemaService;
			_inferenceService = inferenceService;
			_schemaCollection = schemaCollection;
			_userCollection = userCollection;
			_schemaMigrator = schemaMigrator;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <summary>
		/// Checks if the current user has admin write permissions
		/// </summary>
		bool AuthorizeAdmin()
		{
			return _buildConfig.Value.Authorize(AdminAclAction.AdminWrite, User);
		}

		#region Schema Endpoints

		/// <summary>
		/// Gets all schemas (latest version per event)
		/// </summary>
		/// <param name="includeNested">Whether to include nested schemas</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>List of latest schemas</returns>
		[HttpGet]
		[ProducesResponseType(typeof(List<GetTelemetrySchemaResponse>), 200)]
		public async Task<ActionResult<List<GetTelemetrySchemaResponse>>> GetAllSchemasAsync(
			[FromQuery] bool includeNested = false,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<ITelemetrySchema> schemas = await _schemaCollection.GetAllLatestSchemasAsync(includeNested, cancellationToken);
			return Ok(schemas.Select(ToResponse).ToList());
		}

		/// <summary>
		/// Gets the latest schema for a specific event
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The latest schema</returns>
		[HttpGet("{eventName}")]
		[ProducesResponseType(typeof(GetTelemetrySchemaResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<GetTelemetrySchemaResponse>> GetSchemaAsync(
			string eventName,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			ITelemetrySchema? schema = await _schemaCollection.GetLatestSchemaAsync(eventName, cancellationToken);
			if (schema == null)
			{
				return NotFound();
			}

			return Ok(ToResponse(schema));
		}

		/// <summary>
		/// Gets all versions of a schema
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>All versions of the schema</returns>
		[HttpGet("{eventName}/versions")]
		[ProducesResponseType(typeof(List<GetTelemetrySchemaResponse>), 200)]
		public async Task<ActionResult<List<GetTelemetrySchemaResponse>>> GetSchemaVersionsAsync(
			string eventName,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<ITelemetrySchema> schemas = await _schemaCollection.GetSchemaVersionsAsync(eventName, cancellationToken);
			return Ok(schemas.Select(ToResponse).ToList());
		}

		/// <summary>
		/// Gets a specific version of a schema
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="version">The version number</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The specific schema version</returns>
		[HttpGet("{eventName}/versions/{version:int}")]
		[ProducesResponseType(typeof(GetTelemetrySchemaResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<GetTelemetrySchemaResponse>> GetSchemaVersionAsync(
			string eventName,
			int version,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			ITelemetrySchema? schema = await _schemaCollection.GetSchemaAsync(eventName, version, cancellationToken);
			if (schema == null)
			{
				return NotFound();
			}

			return Ok(ToResponse(schema));
		}

		/// <summary>
		/// Manually creates a new schema (typically used for initial setup or testing)
		/// </summary>
		/// <param name="request">The schema to create</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The created schema</returns>
		[HttpPost]
		[ProducesResponseType(typeof(GetTelemetrySchemaResponse), 201)]
		[ProducesResponseType(400)]
		public async Task<ActionResult<GetTelemetrySchemaResponse>> CreateSchemaAsync(
			[FromBody] CreateTelemetrySchemaRequest request,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			// Validate request
			if (String.IsNullOrWhiteSpace(request.EventName))
			{
				return BadRequest("EventName is required");
			}
			if (String.IsNullOrWhiteSpace(request.TableName))
			{
				return BadRequest("TableName is required");
			}
			if (request.Columns == null || request.Columns.Count == 0)
			{
				return BadRequest("At least one column is required");
			}

			// Get next version number
			int nextVersion = await _schemaCollection.GetHighestVersionAsync(request.EventName, cancellationToken) + 1;

			// Get current user
			IUser? user = await _userCollection.GetUserAsync(User, cancellationToken);

			// Convert request columns to schema columns
			List<SchemaColumn> columns = request.Columns.Select((c, i) => new SchemaColumn
			{
				PropertyName = c.PropertyName,
				ColumnName = c.ColumnName,
				ClrTypeName = c.ClrTypeName,
				DataType = c.DataType,
				ArrayElementType = c.ArrayElementType,
				NestedSchemaEventName = c.NestedSchemaEventName,
				IsNullable = c.IsNullable,
				Order = i
			}).ToList();

			// Apply backend migrations BEFORE writing Mongo + bumping Redis. If the DDL
			// fails the metadata never advances, so callers (and the cluster) stay
			// consistent with what the underlying tables actually look like. First-version
			// schemas have no plan (PlanAsync returns null) — the sink lazy-creates the
			// table on first event. Migrator absent => skip silently (matches pre-migrator behaviour).
			if (_schemaMigrator != null)
			{
				MigrationPlan? plan = await _schemaMigrator.PlanAsync(
					request.EventName, columns, request.TableName, request.SchemaName, cancellationToken);
				if (plan != null && plan.Steps.Count > 0)
				{
					await _schemaMigrator.ApplyAsync(plan, cancellationToken);
				}
			}

			ITelemetrySchema schema = await _schemaCollection.AddSchemaAsync(
				request.EventName,
				request.TableName,
				request.SchemaName,
				nextVersion,
				columns,
				request.ClrTypeName,
				request.IsNestedSchema,
				user?.Id,
				cancellationToken);

			_logger.LogInformation("Manually created schema {EventName} version {Version} by user {UserId}",
				request.EventName, nextVersion, user?.Id);

			// ASP.NET Core's MvcOptions.SuppressAsyncSuffixInActionNames defaults to true,
			// so the action's registered name strips the "Async" suffix — we must pass the
			// stripped form for CreatedAtAction's URL lookup to succeed. Using
			// nameof(GetSchemaVersionAsync) here would 500 with "No route matches the
			// supplied values" during result formatting.
			return CreatedAtAction(
				"GetSchemaVersion",
				new { eventName = schema.EventName, version = schema.Version },
				ToResponse(schema));
		}

		/// <summary>
		/// Infers a schema proposal from a single sample JSON payload. Does not write —
		/// the returned proposal is intended to pre-fill the dashboard's authoring form
		/// so the admin can review and submit via POST /api/v1/telemetry-schemas.
		/// </summary>
		/// <param name="request">The event name and sample payload to infer from</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The proposal, any nested-schema proposals, and warnings about uncertain inferences</returns>
		[HttpPost("infer")]
		[RequestSizeLimit(256 * 1024)] // 256 KB cap on the sample payload
		[ProducesResponseType(typeof(InferSchemaResponse), 200)]
		[ProducesResponseType(400)]
		public ActionResult<InferSchemaResponse> InferSchema(
			[FromBody] InferSchemaRequest request,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			if (request == null || String.IsNullOrWhiteSpace(request.EventName))
			{
				return BadRequest("EventName is required");
			}

			if (request.SamplePayload.ValueKind == JsonValueKind.Undefined)
			{
				return BadRequest("samplePayload is required");
			}

			if (request.SamplePayload.ValueKind != JsonValueKind.Object)
			{
				return BadRequest($"samplePayload must be a JSON object; received {request.SamplePayload.ValueKind}");
			}

			cancellationToken.ThrowIfCancellationRequested();

			InferenceResult result = _inferenceService.Infer(
				request.EventName,
				request.TableName,
				request.SchemaName,
				request.SamplePayload);

			return Ok(new InferSchemaResponse
			{
				Proposal = result.Proposal,
				NestedProposals = result.NestedProposals,
				Warnings = result.Warnings.Select(w => new InferenceWarningResponse
				{
					Path = w.Path,
					Message = w.Message
				}).ToList()
			});
		}

		/// <summary>
		/// Computes the migration plan that would run if the supplied schema were
		/// approved/created right now. Dry-run only — no DDL is executed and no
		/// metadata is written. Used by the dashboard to show admins exactly what
		/// destructive operations (column drops, type changes, table renames) a
		/// pending change will perform before they confirm.
		///
		/// First-version schemas (no existing version for the event) return a plan
		/// with no steps and IsDestructive=false — the sink will lazy-create the
		/// table on first event.
		/// </summary>
		[HttpPost("preview-migration")]
		[ProducesResponseType(typeof(MigrationPlanResponse), 200)]
		[ProducesResponseType(400)]
		public async Task<ActionResult<MigrationPlanResponse>> PreviewMigrationAsync(
			[FromBody] CreateTelemetrySchemaRequest request,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			if (request == null || String.IsNullOrWhiteSpace(request.EventName))
			{
				return BadRequest("EventName is required");
			}
			if (String.IsNullOrWhiteSpace(request.TableName))
			{
				return BadRequest("TableName is required");
			}

			// No migrator registered: there's nothing the dashboard needs to confirm —
			// the create/approve path will also skip migration. Return an empty plan
			// so the dashboard can short-circuit straight to submit.
			if (_schemaMigrator == null)
			{
				return Ok(new MigrationPlanResponse());
			}

			List<SchemaColumn> columns = (request.Columns ?? new List<CreateSchemaColumnRequest>()).Select((c, i) => new SchemaColumn
			{
				PropertyName = c.PropertyName,
				ColumnName = c.ColumnName,
				ClrTypeName = c.ClrTypeName,
				DataType = c.DataType,
				ArrayElementType = c.ArrayElementType,
				NestedSchemaEventName = c.NestedSchemaEventName,
				IsNullable = c.IsNullable,
				Order = i
			}).ToList();

			MigrationPlan? plan = await _schemaMigrator.PlanAsync(
				request.EventName, columns, request.TableName, request.SchemaName, cancellationToken);

			if (plan == null)
			{
				// First version — empty plan signals "nothing to do".
				return Ok(new MigrationPlanResponse { EventName = request.EventName });
			}

			return Ok(ToMigrationResponse(plan));
		}

		/// <summary>
		/// Deletes a specific schema version
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="version">The version number to delete</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>No content if deleted</returns>
		[HttpDelete("{eventName}/versions/{version:int}")]
		[ProducesResponseType(204)]
		[ProducesResponseType(404)]
		public async Task<ActionResult> DeleteSchemaVersionAsync(
			string eventName,
			int version,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			bool deleted = await _schemaCollection.DeleteSchemaAsync(eventName, version, cancellationToken);
			if (!deleted)
			{
				return NotFound();
			}

			_logger.LogInformation("Deleted schema {EventName} version {Version}", eventName, version);

			return NoContent();
		}

		#endregion

		#region Pending Update Endpoints

		/// <summary>
		/// Gets all pending schema updates
		/// </summary>
		/// <param name="includeNested">Whether to include nested schemas</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>List of pending updates</returns>
		[HttpGet("pending")]
		[ProducesResponseType(typeof(List<GetPendingSchemaUpdateResponse>), 200)]
		public async Task<ActionResult<List<GetPendingSchemaUpdateResponse>>> GetPendingUpdatesAsync(
			[FromQuery] bool includeNested = false,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<IPendingSchemaUpdate> pending = await _schemaCollection.GetAllPendingUpdatesAsync(includeNested, cancellationToken);
			return Ok(pending.Select(ToPendingResponse).ToList());
		}

		/// <summary>
		/// Gets a pending update for a specific event
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The pending update</returns>
		[HttpGet("pending/{eventName}")]
		[ProducesResponseType(typeof(GetPendingSchemaUpdateResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<GetPendingSchemaUpdateResponse>> GetPendingUpdateAsync(
			string eventName,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			IPendingSchemaUpdate? pending = await _schemaCollection.GetPendingUpdateAsync(eventName, cancellationToken);
			if (pending == null)
			{
				return NotFound();
			}

			return Ok(ToPendingResponse(pending));
		}

		/// <summary>
		/// Approves a pending schema update, creating a new schema version
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The newly created schema</returns>
		[HttpPost("pending/{eventName}/approve")]
		[ProducesResponseType(typeof(GetTelemetrySchemaResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<GetTelemetrySchemaResponse>> ApproveSchemaAsync(
			string eventName,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			IUser? user = await _userCollection.GetUserAsync(User, cancellationToken);
			if (user == null)
			{
				return Unauthorized();
			}

			// Apply backend migrations BEFORE the schema service commits the new version.
			// Same ordering rationale as CreateSchemaAsync: a DDL failure must not leave
			// Mongo metadata advanced past the underlying table shape.
			if (_schemaMigrator != null)
			{
				IPendingSchemaUpdate? pending = await _schemaCollection.GetPendingUpdateAsync(eventName, cancellationToken);
				if (pending == null)
				{
					return NotFound();
				}

				MigrationPlan? plan = await _schemaMigrator.PlanAsync(
					eventName,
					pending.ProposedColumns,
					pending.ProposedTableName,
					pending.ProposedSchemaName,
					cancellationToken);
				if (plan != null && plan.Steps.Count > 0)
				{
					await _schemaMigrator.ApplyAsync(plan, cancellationToken);
				}
			}

			ITelemetrySchema? schema = await _schemaService.ApproveSchemaAsync(eventName, user.Id, cancellationToken);
			if (schema == null)
			{
				return NotFound();
			}

			return Ok(ToResponse(schema));
		}

		/// <summary>
		/// Rejects a pending schema update
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>No content if rejected</returns>
		[HttpPost("pending/{eventName}/reject")]
		[ProducesResponseType(204)]
		[ProducesResponseType(404)]
		public async Task<ActionResult> RejectSchemaAsync(
			string eventName,
			CancellationToken cancellationToken = default)
		{
			if (!AuthorizeAdmin())
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			bool rejected = await _schemaService.RejectSchemaAsync(eventName, cancellationToken);
			if (!rejected)
			{
				return NotFound();
			}

			return NoContent();
		}

		#endregion

		#region Response Mapping

		static GetTelemetrySchemaResponse ToResponse(ITelemetrySchema schema)
		{
			return new GetTelemetrySchemaResponse
			{
				Id = schema.Id.ToString(),
				EventName = schema.EventName,
				TableName = schema.TableName,
				SchemaName = schema.SchemaName,
				Version = schema.Version,
				Columns = schema.Columns.Select(ToColumnResponse).ToList(),
				CreatedAtUtc = schema.CreatedAtUtc,
				ApprovedByUserId = schema.ApprovedByUserId?.ToString(),
				ClrTypeName = schema.ClrTypeName,
				IsNestedSchema = schema.IsNestedSchema
			};
		}

		static GetPendingSchemaUpdateResponse ToPendingResponse(IPendingSchemaUpdate pending)
		{
			return new GetPendingSchemaUpdateResponse
			{
				Id = pending.Id.ToString(),
				EventName = pending.EventName,
				ProposedVersion = pending.ProposedVersion,
				ProposedTableName = pending.ProposedTableName,
				ProposedSchemaName = pending.ProposedSchemaName,
				ProposedColumns = pending.ProposedColumns.Select(ToColumnResponse).ToList(),
				DetectedAtUtc = pending.DetectedAtUtc,
				ChangeType = pending.ChangeType.ToString(),
				ChangeDescription = pending.ChangeDescription,
				Comparison = pending.Comparison != null ? ToComparisonResponse(pending.Comparison) : null,
				ClrTypeName = pending.ClrTypeName,
				IsNestedSchema = pending.IsNestedSchema
			};
		}

		static SchemaColumnResponse ToColumnResponse(SchemaColumn column)
		{
			return new SchemaColumnResponse
			{
				PropertyName = column.PropertyName,
				ColumnName = column.ColumnName,
				ClrTypeName = column.ClrTypeName,
				DataType = column.DataType.ToString(),
				ArrayElementType = column.ArrayElementType?.ToString(),
				NestedSchemaEventName = column.NestedSchemaEventName,
				IsNullable = column.IsNullable,
				Order = column.Order
			};
		}

		static SchemaComparisonResponse ToComparisonResponse(SchemaComparison comparison)
		{
			return new SchemaComparisonResponse
			{
				AddedColumns = comparison.AddedColumns,
				RemovedColumns = comparison.RemovedColumns,
				ModifiedColumns = comparison.ModifiedColumns.Select(c => new ColumnChangeResponse
				{
					ColumnName = c.ColumnName,
					OldClrTypeName = c.OldClrTypeName,
					NewClrTypeName = c.NewClrTypeName,
					OldDataType = c.OldDataType?.ToString(),
					NewDataType = c.NewDataType?.ToString(),
					OldIsNullable = c.OldIsNullable,
					NewIsNullable = c.NewIsNullable
				}).ToList(),
				TableNameChanged = comparison.TableNameChanged,
				OldTableName = comparison.OldTableName
			};
		}

		static MigrationPlanResponse ToMigrationResponse(MigrationPlan plan)
		{
			return new MigrationPlanResponse
			{
				EventName = plan.EventName,
				TableName = plan.TableName,
				SchemaName = plan.SchemaName,
				IsDestructive = plan.IsDestructive,
				Warnings = plan.Warnings,
				Steps = plan.Steps.Select(s => new MigrationStepResponse
				{
					Kind = s.Kind.ToString(),
					ColumnName = s.ColumnName,
					Detail = s.Detail
				}).ToList()
			};
		}

		#endregion
	}

	#region Request/Response DTOs

	/// <summary>
	/// Response for a telemetry schema
	/// </summary>
	public class GetTelemetrySchemaResponse
	{
		/// <summary>Unique identifier</summary>
		public string Id { get; set; } = null!;

		/// <summary>The telemetry event name</summary>
		public string EventName { get; set; } = null!;

		/// <summary>The database table name</summary>
		public string TableName { get; set; } = null!;

		/// <summary>The schema/database name</summary>
		public string? SchemaName { get; set; }

		/// <summary>Schema version number</summary>
		public int Version { get; set; }

		/// <summary>Column definitions</summary>
		public List<SchemaColumnResponse> Columns { get; set; } = new();

		/// <summary>When this schema was created</summary>
		public DateTime CreatedAtUtc { get; set; }

		/// <summary>User who approved this schema</summary>
		public string? ApprovedByUserId { get; set; }

		/// <summary>The CLR type name</summary>
		public string? ClrTypeName { get; set; }

		/// <summary>Whether this is a nested schema</summary>
		public bool IsNestedSchema { get; set; }
	}

	/// <summary>
	/// Response for a schema column
	/// </summary>
	public class SchemaColumnResponse
	{
		/// <summary>C# property name</summary>
		public string PropertyName { get; set; } = null!;

		/// <summary>Database column name</summary>
		public string ColumnName { get; set; } = null!;

		/// <summary>CLR type name</summary>
		public string ClrTypeName { get; set; } = null!;

		/// <summary>Fundamental data type</summary>
		public string DataType { get; set; } = null!;

		/// <summary>Array element type if DataType is Array</summary>
		public string? ArrayElementType { get; set; }

		/// <summary>Nested schema event name if DataType is Object</summary>
		public string? NestedSchemaEventName { get; set; }

		/// <summary>Whether the column is nullable</summary>
		public bool IsNullable { get; set; }

		/// <summary>Column order</summary>
		public int? Order { get; set; }
	}

	/// <summary>
	/// Response for a pending schema update
	/// </summary>
	public class GetPendingSchemaUpdateResponse
	{
		/// <summary>Unique identifier</summary>
		public string Id { get; set; } = null!;

		/// <summary>The telemetry event name</summary>
		public string EventName { get; set; } = null!;

		/// <summary>Proposed version number</summary>
		public int ProposedVersion { get; set; }

		/// <summary>Proposed table name</summary>
		public string ProposedTableName { get; set; } = null!;

		/// <summary>Proposed schema/database name</summary>
		public string? ProposedSchemaName { get; set; }

		/// <summary>Proposed columns</summary>
		public List<SchemaColumnResponse> ProposedColumns { get; set; } = new();

		/// <summary>When this update was detected</summary>
		public DateTime DetectedAtUtc { get; set; }

		/// <summary>Type of change (New/Modified)</summary>
		public string ChangeType { get; set; } = null!;

		/// <summary>Human-readable description</summary>
		public string ChangeDescription { get; set; } = null!;

		/// <summary>Detailed comparison</summary>
		public SchemaComparisonResponse? Comparison { get; set; }

		/// <summary>CLR type name</summary>
		public string? ClrTypeName { get; set; }

		/// <summary>Whether this is a nested schema</summary>
		public bool IsNestedSchema { get; set; }
	}

	/// <summary>
	/// Response for schema comparison
	/// </summary>
	public class SchemaComparisonResponse
	{
		/// <summary>Added column names</summary>
		public List<string> AddedColumns { get; set; } = new();

		/// <summary>Removed column names</summary>
		public List<string> RemovedColumns { get; set; } = new();

		/// <summary>Modified columns</summary>
		public List<ColumnChangeResponse> ModifiedColumns { get; set; } = new();

		/// <summary>Whether table name changed</summary>
		public bool TableNameChanged { get; set; }

		/// <summary>Old table name if changed</summary>
		public string? OldTableName { get; set; }
	}

	/// <summary>
	/// Response for a column change
	/// </summary>
	public class ColumnChangeResponse
	{
		/// <summary>Column name</summary>
		public string ColumnName { get; set; } = null!;

		/// <summary>Old CLR type</summary>
		public string? OldClrTypeName { get; set; }

		/// <summary>New CLR type</summary>
		public string? NewClrTypeName { get; set; }

		/// <summary>Old data type</summary>
		public string? OldDataType { get; set; }

		/// <summary>New data type</summary>
		public string? NewDataType { get; set; }

		/// <summary>Old nullable state</summary>
		public bool? OldIsNullable { get; set; }

		/// <summary>New nullable state</summary>
		public bool? NewIsNullable { get; set; }
	}

	/// <summary>
	/// Request to create a schema manually
	/// </summary>
	public class CreateTelemetrySchemaRequest
	{
		/// <summary>The telemetry event name</summary>
		public string EventName { get; set; } = null!;

		/// <summary>The database table name</summary>
		public string TableName { get; set; } = null!;

		/// <summary>The schema/database name</summary>
		public string? SchemaName { get; set; }

		/// <summary>Column definitions</summary>
		public List<CreateSchemaColumnRequest> Columns { get; set; } = new();

		/// <summary>CLR type name</summary>
		public string? ClrTypeName { get; set; }

		/// <summary>Whether this is a nested schema</summary>
		public bool IsNestedSchema { get; set; }
	}

	/// <summary>
	/// Request to create a schema column
	/// </summary>
	public class CreateSchemaColumnRequest
	{
		/// <summary>C# property name</summary>
		public string PropertyName { get; set; } = null!;

		/// <summary>Database column name</summary>
		public string ColumnName { get; set; } = null!;

		/// <summary>CLR type name</summary>
		public string ClrTypeName { get; set; } = null!;

		/// <summary>Fundamental data type</summary>
		public SchemaDataType DataType { get; set; }

		/// <summary>Array element type if DataType is Array</summary>
		public SchemaDataType? ArrayElementType { get; set; }

		/// <summary>Nested schema event name if DataType is Object</summary>
		public string? NestedSchemaEventName { get; set; }

		/// <summary>Whether the column is nullable</summary>
		public bool IsNullable { get; set; }
	}

	/// <summary>
	/// Request body for POST /api/v1/telemetry-schemas/infer.
	/// </summary>
	public class InferSchemaRequest
	{
		/// <summary>The telemetry event name. Required.</summary>
		public string EventName { get; set; } = null!;

		/// <summary>Optional table name. Defaults to a snake_case form of the event name.</summary>
		public string? TableName { get; set; }

		/// <summary>Optional schema/database name. Null falls through to the sink's DefaultDatabaseName.</summary>
		public string? SchemaName { get; set; }

		/// <summary>A single sample JSON payload representing one example of the event. Must be a JSON object.</summary>
		public JsonElement SamplePayload { get; set; }
	}

	/// <summary>
	/// Response for POST /api/v1/telemetry-schemas/infer. The admin reviews the proposal,
	/// edits it in the dashboard form, and submits via the existing POST / endpoint.
	/// </summary>
	public class InferSchemaResponse
	{
		/// <summary>Top-level proposal to pre-fill the dashboard form.</summary>
		public CreateTelemetrySchemaRequest Proposal { get; set; } = null!;

		/// <summary>One proposal per nested object discovered in the sample. Submit these before the parent so NestedSchemaEventName references resolve.</summary>
		public List<CreateTelemetrySchemaRequest> NestedProposals { get; set; } = new();

		/// <summary>Warnings about uncertain inferences (null fields, empty arrays, mixed-type arrays, ambiguous date strings).</summary>
		public List<InferenceWarningResponse> Warnings { get; set; } = new();
	}

	/// <summary>
	/// A single warning emitted during schema inference.
	/// </summary>
	public class InferenceWarningResponse
	{
		/// <summary>JSON path to the property that triggered the warning.</summary>
		public string Path { get; set; } = null!;

		/// <summary>Human-readable explanation.</summary>
		public string Message { get; set; } = null!;
	}

	/// <summary>
	/// Response for POST /api/v1/telemetry-schemas/preview-migration. Empty plans
	/// (no steps, no warnings) indicate either a first-time schema or no migrator
	/// is registered for the active backend — the dashboard can submit without
	/// confirmation in either case.
	/// </summary>
	public class MigrationPlanResponse
	{
		/// <summary>The event name being migrated.</summary>
		public string EventName { get; set; } = "";

		/// <summary>The proposed table name.</summary>
		public string TableName { get; set; } = "";

		/// <summary>The proposed database/schema name (null = backend default).</summary>
		public string? SchemaName { get; set; }

		/// <summary>True when the plan contains drops, type changes, or table renames.</summary>
		public bool IsDestructive { get; set; }

		/// <summary>Human-readable warnings the dashboard surfaces before approval.</summary>
		public List<string> Warnings { get; set; } = new();

		/// <summary>Ordered DDL steps that will run if approved.</summary>
		public List<MigrationStepResponse> Steps { get; set; } = new();
	}

	/// <summary>One DDL operation in a <see cref="MigrationPlanResponse"/>.</summary>
	public class MigrationStepResponse
	{
		/// <summary>Operation kind (AddColumn, DropColumn, DropAndAddColumn, NewTableLocation).</summary>
		public string Kind { get; set; } = "";

		/// <summary>Column name affected. Empty for table/database-level steps.</summary>
		public string ColumnName { get; set; } = "";

		/// <summary>Free-form detail for display (type label, "old -> new", etc.).</summary>
		public string? Detail { get; set; }
	}

	#endregion
}
