// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using HordeServer.Analytics;

namespace HordeServer.Dialects
{
	/// <summary>
	/// Describes the SQL flavor of an analytics backend so consumers can build dialect-appropriate parameter placeholders, identifier quotes, and command parameters without knowing which backend is wired up.
	///
	/// Each backend plugin supplies one implementation alongside its <see cref="IAnalyticsDataSource"/>. The Phase 0 audit confirmed every today-shipping query is portable across backends, so the dialect surface is intentionally minimal — only the differences that matter on the wire.
	/// </summary>
	public interface ISqlDialect
	{
		/// <summary>
		/// Renders the placeholder text that goes into the SQL command for a parameter. Examples: <c>?</c> (ODBC, positional), <c>@p0</c> (some named-binding dialects), <c>{p0:Int32}</c> (ClickHouse).
		/// </summary>
		/// <param name="canonicalName">Auto-generated parameter name (e.g. <c>p0</c>, <c>p1</c>). Dialects that bind positionally ignore this; named-binding dialects use it.</param>
		/// <param name="dbType">Logical type of the parameter, used by typed-placeholder dialects.</param>
		string FormatParameterPlaceholder(string canonicalName, DbType dbType);

		/// <summary>
		/// Adds a single parameter to <paramref name="command"/>, creating it via the command's own factory and binding by name (or position, depending on dialect).
		/// </summary>
		/// <param name="command">The command to receive the parameter.</param>
		/// <param name="canonicalName">Auto-generated parameter name matching the placeholder text from <see cref="FormatParameterPlaceholder"/>.</param>
		/// <param name="value">The parameter value (null becomes <see cref="DBNull.Value"/>).</param>
		/// <param name="dbType">Logical type of the parameter.</param>
		void AddParameter(DbCommand command, string canonicalName, object? value, DbType dbType);

		/// <summary>
		/// Wraps an identifier (table name, column name, schema name) in the dialect's quoting characters. Examples: <c>"col"</c>, <c>`col`</c>, <c>[col]</c>, or pass-through if no quoting is required.
		/// </summary>
		string QuoteIdentifier(string identifier);

		/// <summary>
		/// Formats a fully-qualified table reference suitable for inclusion in a <c>FROM</c> clause. Combines the optional schema/database name with the table name using the dialect's conventions for separators and quoting. ClickHouse may quote tables containing dots; standard ODBC backends use plain <c>schema.table</c>.
		/// </summary>
		/// <param name="schemaName">The schema/database name (e.g. <c>ingest</c>), or null to use the backend's default database.</param>
		/// <param name="tableName">The unqualified table name. May itself contain dots (e.g. <c>horde.state_job_summary</c>) when codegen produces a dot-bearing logical name.</param>
		string FormatTableName(string? schemaName, string tableName);
	}
}
