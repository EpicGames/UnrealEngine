// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using HordeServer.Analytics;

namespace HordeServer.Dialects
{
	/// <summary>
	/// Default <see cref="ISqlDialect"/> used by <see cref="WhereClauseBuilder"/> when no backend-specific dialect is supplied. Emits positional <c>?</c> placeholders matching ODBC/Simba behavior, with no identifier quoting and generic <see cref="DbCommand.CreateParameter"/> binding.
	///
	/// Backend plugins ship their own dialect (e.g. <c>OdbcDialect</c>, <c>ClickHouseSqlDialect</c>) that wins over this default once registered in DI; this class exists for tests and for filter call sites that have no DI access.
	/// </summary>
	public sealed class PositionalQuestionMarkDialect : ISqlDialect
	{
		/// <summary>
		/// Singleton instance.
		/// </summary>
		public static PositionalQuestionMarkDialect Instance { get; } = new();

		private PositionalQuestionMarkDialect()
		{
		}

		/// <inheritdoc/>
		public string FormatParameterPlaceholder(string canonicalName, DbType dbType) => "?";

		/// <inheritdoc/>
		public void AddParameter(DbCommand command, string canonicalName, object? value, DbType dbType)
		{
			DbParameter parameter = command.CreateParameter();
			parameter.ParameterName = canonicalName;
			parameter.DbType = dbType;
			parameter.Value = value ?? DBNull.Value;
			command.Parameters.Add(parameter);
		}

		/// <inheritdoc/>
		public string QuoteIdentifier(string identifier) => identifier;

		/// <inheritdoc/>
		public string FormatTableName(string? schemaName, string tableName)
		{
			return String.IsNullOrEmpty(schemaName) ? tableName : $"{schemaName}.{tableName}";
		}
	}
}
