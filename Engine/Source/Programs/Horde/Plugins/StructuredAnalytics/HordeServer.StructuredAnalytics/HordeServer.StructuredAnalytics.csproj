<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="Polly" Version="8.2.1" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />

  </ItemGroup>
  <PropertyGroup>
    <EmitCompilerGeneratedFiles>true</EmitCompilerGeneratedFiles>
    <CompilerGeneratedFilesOutputPath>$(BaseIntermediateOutputPath)g</CompilerGeneratedFilesOutputPath>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.Generators\EpicGames.Analytics.Generators.csproj" OutputItemType="Analyzer" ReferenceOutputAssembly="false" />
    <ProjectReference Include="..\..\Build\HordeServer.Build\HordeServer.Build.csproj" />
  </ItemGroup>
</Project>