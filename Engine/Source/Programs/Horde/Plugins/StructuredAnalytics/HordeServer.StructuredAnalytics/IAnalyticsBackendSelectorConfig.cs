// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics
{
	/// <summary>
	/// Marker interface implemented by every consumer-plugin config that drives an <see cref="AbstractAnalyticsService{TConfig}"/>. Exposes <see cref="BackendName"/>, which selects the keyed <see cref="IAnalyticsDataSource"/> registration the consumer should bind to.
	///
	/// Each backend plugin (<c>OdbcTelemetry</c>, <c>ClickHouseTelemetry</c>, ...) registers its data source under a well-known key (<c>"Odbc"</c>, <c>"ClickHouse"</c>, ...). Consumers name the key they want here, and DI resolves the right backend per-consumer — so two consumers in the same deployment can hit different backends.
	/// </summary>
	public interface IAnalyticsBackendSelectorConfig
	{
		/// <summary>
		/// The key naming the registered <see cref="IAnalyticsDataSource"/> this consumer should bind to. Must match a key registered by an enabled backend plugin.
		/// </summary>
		string BackendName { get; }
	}
}
