// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics
{
	/// <summary>
	/// Connection configuration consumed by ODBC-based analytics services (<see cref="AbstractAnalyticsService{TConfig}"/>). Combines the OAuth fields from <see cref="IOAuthClientCredentialsConfig"/> with the ODBC connection-string concerns specific to query-side services.
	///
	/// Sinks and other code paths that need the OAuth fields but not ODBC (e.g. ClickHouseTelemetry, which talks to ClickHouse over its native HTTP protocol) implement <see cref="IOAuthClientCredentialsConfig"/> directly instead of this richer interface.
	/// </summary>
	public interface IAnalyticsConnectionConfig : IOAuthClientCredentialsConfig
	{
		/// <summary>
		/// Connection string for ODBC connector.
		/// </summary>
		/// <remarks>Not ';' terminated.</remarks>
		string ODBCConnectionString { get; }

		/// <summary>
		/// Connection string refresh time, in minutes.
		/// </summary>
		int? ODBCConnectionStringRefreshTimeMins { get; }
	}
}
