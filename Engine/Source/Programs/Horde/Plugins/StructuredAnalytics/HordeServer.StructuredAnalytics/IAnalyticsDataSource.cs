// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data.Common;
using HordeServer.Dialects;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Backend-agnostic source of analytics database connections. Each backend plugin (<c>OdbcTelemetry</c>, <c>ClickHouseTelemetry</c>, <c>DatabricksTelemetry</c>, ...) registers exactly one implementation; consumer plugins (<c>StructuredAnalytics</c>, <c>PerformanceTrends</c>) take this in via DI and never reference <c>System.Data.Odbc</c> directly.
	///
	/// The bundled <see cref="Dialect"/> describes the SQL flavor of whatever backend opened the connection (parameter placeholder syntax, identifier quoting). Bundling avoids a separate DI lookup and guarantees the dialect always matches the connection.
	/// </summary>
	public interface IAnalyticsDataSource
	{
		/// <summary>
		/// Opens a new <see cref="DbConnection"/> against the configured analytics backend. The caller owns the connection and is responsible for disposing it.
		/// </summary>
		Task<DbConnection> OpenAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// SQL dialect that pairs with connections returned from <see cref="OpenAsync"/>. Owns parameter-placeholder formatting, identifier quoting, and table-name composition for this backend.
		/// </summary>
		ISqlDialect Dialect { get; }
	}
}
