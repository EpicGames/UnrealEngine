// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics
{
	/// <summary>
	/// Common shape for any analytics-backend config that drives the generic OAuth2 client-credentials token exchange.
	///
	/// Each backend plugin's config (e.g. <c>OdbcTelemetryConfig</c>, <c>ClickHouseTelemetryConfig</c>, <c>DatabricksTelemetryConfig</c>) implements this directly — or via a stricter interface that adds backend-specific fields — so it can be the <c>TConfig</c> of <see cref="OAuthClientCredentialsTokenProvider{TConfig}"/>.
	/// </summary>
	public interface IOAuthClientCredentialsConfig
	{
		/// <summary>
		/// HTTPS endpoint for the OAuth2 client-credentials token exchange. When empty, the auth provider is inert: every call returns null, so connection strings (and HTTP requests) flow through untouched. Required only for token-authenticated backends.
		/// </summary>
		string OAuthTokenEndpoint { get; }

		/// <summary>
		/// OAuth2 scope passed in the client-credentials request (form field <c>scope</c>). Empty omits the field. Defaults to <c>all-apis</c> for Databricks; other providers may need a URL-shaped scope or none at all.
		/// </summary>
		string OAuthScope { get; }

		/// <summary>
		/// Connection-string key the access token is folded into when building the final ODBC connection string. Defaults to <c>Auth_AccessToken</c> for the Simba Spark ODBC driver. Other ODBC drivers may use different keys.
		/// 
		/// Irrelevant for HTTP-based backends that consume the token via the raw <see cref="IAuthenticationProvider{TConfig}.GetTokenAsync"/> path (e.g. as an Authorization: Bearer header).
		/// </summary>
		string OAuthAccessTokenFragmentKey { get; }

		/// <summary>
		/// Secret-store ID under which the OAuth client_id and client_secret are stored. Override per deployment if multiple OAuth providers coexist.
		/// </summary>
		string OAuthCredentialsSecretId { get; }
	}
}
