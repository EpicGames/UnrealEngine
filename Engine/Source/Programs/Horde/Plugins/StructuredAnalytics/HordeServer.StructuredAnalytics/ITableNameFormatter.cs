// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics
{
	/// <summary>
	/// Interface for formatting table names for different database backends.
	/// Implementations provide database-specific quoting and formatting rules.
	/// </summary>
	public interface ITableNameFormatter
	{
		/// <summary>
		/// Formats a fully qualified table name for the target database.
		/// </summary>
		/// <param name="schema">The schema/database name (e.g., "ingest").</param>
		/// <param name="tableName">The table name (e.g., "horde.state_job_summary").</param>
		/// <returns>The properly formatted and quoted table name for the target database.</returns>
		string FormatTableName(string? schema, string tableName);
	}
}
