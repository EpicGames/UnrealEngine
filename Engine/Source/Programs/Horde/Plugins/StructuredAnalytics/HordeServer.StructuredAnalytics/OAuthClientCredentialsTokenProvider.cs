// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Secrets;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Generic OAuth2 client-credentials token provider for analytics ODBC connections. 
	/// Exchanges a client_id/client_secret pair (read from the secret store) for a short-lived access token at the configured OAuthTokenEndpoint then folds the token into the ODBC connection string under the configured fragment key.
	/// </summary>
	/// <remarks>https://datatracker.ietf.org/doc/html/rfc6749#section-4.4 (client_credentials grant)</remarks>
	public class OAuthClientCredentialsTokenProvider<TConfig> : IAuthenticationProvider<TConfig> where TConfig : class, IOAuthClientCredentialsConfig
	{
		private const int DEFAULT_REFRESH_BUFFER = 5;
		private const int DEFAULT_REFRESH_MIN = 5;

		private string? _cachedToken = String.Empty;
		private DateTime _tokenExpiry;

		private readonly IOptionsMonitor<TConfig> _config;
		private readonly IServiceProvider _serviceProvider;
		private readonly IHttpClientFactory _httpClientFactory;
		private readonly ILogger<OAuthClientCredentialsTokenProvider<TConfig>> _logger;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="config">Plugin configuration carrying the OAuth endpoint, scope, fragment key, and secret ID.</param>
		/// <param name="httpClientFactory">HttpClientFactory used for the token exchange request.</param>
		/// <param name="serviceProvider">Service provider for resolving the secret collection per call.</param>
		/// <param name="logger">Logger.</param>
		public OAuthClientCredentialsTokenProvider(
			IOptionsMonitor<TConfig> config,
			IHttpClientFactory httpClientFactory,
			IServiceProvider serviceProvider,
			ILogger<OAuthClientCredentialsTokenProvider<TConfig>> logger)
		{
			_config = config;
			_httpClientFactory = httpClientFactory;
			_serviceProvider = serviceProvider;
			_logger = logger;

			if (String.IsNullOrEmpty(_config.CurrentValue.OAuthTokenEndpoint))
			{
				_logger.LogDebug("OAuth token endpoint not configured; provider is inert. Connection strings will pass through unmodified.");
			}
		}

		#region -- Interface API --

		/// <inheritdoc/>
		public async Task<KeyValuePair<string, string>?> GetConnectionStringAuthenticationFragmentAsync()
		{
			string? token = await GetTokenAsync();

			if (token == null)
			{
				return null;
			}

			string fragmentKey = _config.CurrentValue.OAuthAccessTokenFragmentKey;
			if (String.IsNullOrEmpty(fragmentKey))
			{
				_logger.LogWarning("OAuth access token retrieved but {FragmentKey} is empty; cannot fold into connection string.",
					nameof(IOAuthClientCredentialsConfig.OAuthAccessTokenFragmentKey));
				return null;
			}

			return new KeyValuePair<string, string>(fragmentKey, token);
		}

		/// <inheritdoc/>
		public async Task<string?> GetTokenAsync()
		{
			// Inert when no endpoint is set — the no-config path. Quiet, no error spam.
			string endpointConfig = _config.CurrentValue.OAuthTokenEndpoint;
			if (String.IsNullOrEmpty(endpointConfig))
			{
				return null;
			}

			if (!String.IsNullOrEmpty(_cachedToken) && DateTime.UtcNow < _tokenExpiry)
			{
				return _cachedToken;
			}

			try
			{
				await using AsyncServiceScope scope = _serviceProvider.CreateAsyncScope();
				ISecretCollection? secretCollection = scope.ServiceProvider.GetService<ISecretCollection>();

				string secretId = _config.CurrentValue.OAuthCredentialsSecretId;
				if (String.IsNullOrEmpty(secretId) || secretCollection == null)
				{
					_logger.LogError("OAuth credentials secret ID is empty; cannot retrieve client_id/client_secret.");
					return null;
				}

				ISecret? secret = await secretCollection.GetAsync(new SecretId(secretId), CancellationToken.None);

				if (secret == null)
				{
					_logger.LogError("Unable to obtain secret '{SecretId}' for token retrieval.", secretId);
					return null;
				}

				string clientId = secret.Data["client_id"];
				string clientSecret = secret.Data["client_secret"];

				using (HttpClient http = _httpClientFactory.CreateClient())
				{
					if (!Uri.TryCreate(endpointConfig, UriKind.Absolute, out Uri? tokenEndpoint))
					{
						_logger.LogError("Unable to create Uri from configured OAuthTokenEndpoint - check configuration.");
						return null;
					}

					if (!tokenEndpoint.Scheme.Equals(Uri.UriSchemeHttps, StringComparison.OrdinalIgnoreCase))
					{
						throw new InvalidOperationException("Insecure URI scheme detected for OAuthTokenEndpoint. HTTPS is required.");
					}

					// Build the form-encoded client-credentials request. The scope field is
					// optional per RFC 6749 §4.4; omit it when the deployment configures
					// an empty scope so we don't send "scope=" to providers that reject it.
					List<KeyValuePair<string, string>> formFields = new()
					{
						new KeyValuePair<string, string>("client_id", clientId),
						new KeyValuePair<string, string>("client_secret", clientSecret),
						new KeyValuePair<string, string>("grant_type", "client_credentials")
					};

					string oauthScope = _config.CurrentValue.OAuthScope;
					if (!String.IsNullOrEmpty(oauthScope))
					{
						formFields.Add(new KeyValuePair<string, string>("scope", oauthScope));
					}

					using FormUrlEncodedContent content = new FormUrlEncodedContent(formFields);
					using HttpResponseMessage response = await http.PostAsync(tokenEndpoint, content);
					response.EnsureSuccessStatusCode();
					string json = await response.Content.ReadAsStringAsync();

					using JsonDocument result = JsonDocument.Parse(json);
					if (!result.RootElement.TryGetProperty("access_token", out JsonElement tokenElement))
					{
						throw new InvalidOperationException("access_token not found in JSON response.");
					}

					string? retrievedToken = tokenElement.GetString();

					if (String.IsNullOrEmpty(retrievedToken))
					{
						throw new InvalidOperationException("access_token value is null or empty.");
					}

					_cachedToken = retrievedToken!;
					int refreshMinCount;

					if (!result.RootElement.TryGetProperty("expires_in", out JsonElement expiryElement))
					{
						_logger.LogWarning("Could not retrieve an expiry time; defaulting to {DefaultRefresh} minutes.", DEFAULT_REFRESH_MIN);
						refreshMinCount = DEFAULT_REFRESH_MIN;
					}
					else
					{
						// Expires_in is reported in seconds. Subtract a safety buffer so we
						// never hand out a token that's about to expire mid-request, but
						// don't go negative for very short-lived tokens.
						int bufferedRefresh = expiryElement.GetInt32() / 60;
						refreshMinCount = bufferedRefresh > DEFAULT_REFRESH_BUFFER ? bufferedRefresh - DEFAULT_REFRESH_BUFFER : bufferedRefresh;
					}

					_tokenExpiry = DateTime.UtcNow.AddMinutes(refreshMinCount);
				}
			}
			catch (Exception ex)
			{
				_logger.LogError("Unable to obtain OAuth token: {Exception}", ex.Message);
				_tokenExpiry = DateTime.UtcNow;
				_cachedToken = null;
			}

			return _cachedToken;
		}

		#endregion -- Interface API --
	}
}
