// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Plans and applies destination-side schema migrations for a telemetry sink.
	/// One implementation per backend (ClickHouse, BigQuery, etc.). The schema
	/// service computes a backend-agnostic <see cref="MigrationPlan"/>, then this
	/// migrator translates it into the backend's DDL.
	///
	/// The controller uses this in two phases:
	///   1. <see cref="PlanAsync"/> for a dry-run preview returned to the dashboard
	///      so admins can see destructive operations before approving.
	///   2. <see cref="ApplyAsync"/> as part of the create/approve flow, run BEFORE
	///      the new schema version is committed so a DDL failure leaves the system
	///      consistent (Mongo metadata never gets ahead of the actual table shape).
	/// </summary>
	public interface ISchemaMigrator
	{
		/// <summary>
		/// Computes the steps needed to evolve the current latest stored version of
		/// <paramref name="eventName"/> into <paramref name="proposedColumns"/>. Returns
		/// <c>null</c> when no current version exists — first-time schemas need no
		/// migration; the sink will lazy-create the table on its first event.
		/// </summary>
		Task<MigrationPlan?> PlanAsync(string eventName, IReadOnlyList<SchemaColumn> proposedColumns, string proposedTableName, string? proposedSchemaName, CancellationToken cancellationToken = default);

		/// <summary>
		/// Executes the DDL described by <paramref name="plan"/>. No-op for plans
		/// with no steps. Throws on any DDL failure — callers must not commit the
		/// new schema version if this throws.
		/// </summary>
		Task ApplyAsync(MigrationPlan plan, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Backend-agnostic description of the work required to evolve a stored
	/// schema's underlying table from the current shape to a proposed shape.
	/// </summary>
	public class MigrationPlan
	{
		/// <summary>The event name being migrated.</summary>
		public string EventName { get; set; } = null!;

		/// <summary>The proposed table name.</summary>
		public string TableName { get; set; } = null!;

		/// <summary>The proposed database/schema name (null = backend default).</summary>
		public string? SchemaName { get; set; }

		/// <summary>
		/// Ordered DDL steps. Apply in order; on any failure, abort and do not
		/// commit the metadata change.
		/// </summary>
		public List<MigrationStep> Steps { get; set; } = new();

		/// <summary>
		/// Human-readable warnings the dashboard surfaces to the admin before
		/// approval. Populated for every destructive step (drops, type changes,
		/// renames, table/database changes).
		/// </summary>
		public List<string> Warnings { get; set; } = new();

		/// <summary>
		/// True when the plan contains any irreversible operation: column drops,
		/// type changes (drop+readd), or a table/database rename (old data
		/// orphaned). The dashboard uses this to gate a confirmation dialog.
		/// </summary>
		public bool IsDestructive { get; set; }
	}

	/// <summary>One DDL operation in a <see cref="MigrationPlan"/>.</summary>
	public class MigrationStep
	{
		/// <summary>The kind of operation.</summary>
		public MigrationStepKind Kind { get; set; }

		/// <summary>Column name affected. Empty for table/database-level steps.</summary>
		public string ColumnName { get; set; } = "";

		/// <summary>
		/// Free-form detail for display (e.g. "Int64 -> String" for a type change,
		/// or the old table name for a rename).
		/// </summary>
		public string? Detail { get; set; }

		/// <summary>
		/// Portable column definition for AddColumn and DropAndAddColumn steps —
		/// carries the StructuredAnalytics-native types (<see cref="SchemaDataType"/>,
		/// <see cref="SchemaColumn.IsNullable"/>, etc.). Backend migrators translate
		/// to their own DDL via their existing type mappers at apply time. Null for
		/// DropColumn and NewTableLocation steps which need no type info.
		/// </summary>
		public SchemaColumn? Column { get; set; }
	}

	/// <summary>
	/// Distinct kinds of migration step. The dashboard categorises these into
	/// "safe" (additive) and "destructive" (drops, type changes, table renames)
	/// for the warning UX.
	/// </summary>
	public enum MigrationStepKind
	{
		/// <summary>ALTER TABLE ... ADD COLUMN. Non-destructive. Existing rows get NULL/default.</summary>
		AddColumn,

		/// <summary>ALTER TABLE ... DROP COLUMN. Destroys all stored data for that column.</summary>
		DropColumn,

		/// <summary>
		/// Drop existing column then add fresh with the new type. Used for both
		/// renames (different name) and type changes (same name). Destroys all
		/// stored data for that column.
		/// </summary>
		DropAndAddColumn,

		/// <summary>
		/// Table or database name has changed in the new version. No DDL is run
		/// against the existing table; the sink will lazy-create the new table
		/// on its first event. Old data is left in place at the old name.
		/// </summary>
		NewTableLocation
	}
}
