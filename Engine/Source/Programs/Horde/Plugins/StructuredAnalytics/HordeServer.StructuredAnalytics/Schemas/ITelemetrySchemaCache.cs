// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// In-memory snapshot of the latest telemetry schemas, kept fresh across worker
	/// instances via a Redis-backed version counter on <see cref="ITelemetrySchemaCollection"/>.
	/// Hot-path consumers should read <see cref="Schemas"/> rather than calling
	/// <see cref="ITelemetrySchemaCollection.GetLatestSchemaAsync"/> per event.
	/// </summary>
	public interface ITelemetrySchemaCache
	{
		/// <summary>
		/// Snapshot of the latest schemas keyed by event name. The dictionary reference is
		/// replaced atomically on refresh; readers see an internally consistent snapshot.
		/// </summary>
		IReadOnlyDictionary<string, ITelemetrySchema> Schemas { get; }

		/// <summary>
		/// Raised after the cache has been updated. Handlers receive the new snapshot.
		/// Subscribe with <c>+=</c>; unsubscribe with <c>-=</c>. Handlers are invoked on
		/// the refresh ticker thread, one at a time, with exceptions logged and isolated —
		/// keep handlers fast and avoid blocking work.
		/// </summary>
		event Action<IReadOnlyDictionary<string, ITelemetrySchema>> SchemasChanged;
	}
}
