// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Collection for managing telemetry schemas and pending updates
	/// </summary>
	public interface ITelemetrySchemaCollection
	{
		#region Schema Operations

		/// <summary>
		/// Adds a new schema version
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="tableName">The database table name</param>
		/// <param name="schemaName">The schema/database name (e.g., "ingest")</param>
		/// <param name="version">The schema version</param>
		/// <param name="columns">Column definitions</param>
		/// <param name="clrTypeName">The CLR type name</param>
		/// <param name="isNestedSchema">Whether this is a nested schema</param>
		/// <param name="approvedByUserId">User who approved (null for auto-created)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The created schema</returns>
		Task<ITelemetrySchema> AddSchemaAsync(
			string eventName,
			string tableName,
			string? schemaName,
			int version,
			List<SchemaColumn> columns,
			string? clrTypeName,
			bool isNestedSchema,
			UserId? approvedByUserId = null,
			CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the latest schema version for an event name
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The latest schema or null if not found</returns>
		Task<ITelemetrySchema?> GetLatestSchemaAsync(string eventName, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a specific schema version
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="version">The version number</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The schema or null if not found</returns>
		Task<ITelemetrySchema?> GetSchemaAsync(string eventName, int version, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets all versions of a schema
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>All schema versions ordered by version descending</returns>
		Task<List<ITelemetrySchema>> GetSchemaVersionsAsync(string eventName, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the latest schema for all event names
		/// </summary>
		/// <param name="includeNested">Whether to include nested schemas</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>List of latest schemas</returns>
		Task<List<ITelemetrySchema>> GetAllLatestSchemasAsync(bool includeNested = false, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a specific schema version
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="version">The version number</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>True if deleted, false if not found</returns>
		Task<bool> DeleteSchemaAsync(string eventName, int version, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the highest version number for an event name
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The highest version number, or 0 if no schemas exist</returns>
		Task<int> GetHighestVersionAsync(string eventName, CancellationToken cancellationToken = default);

		#endregion

		#region Pending Update Operations

		/// <summary>
		/// Creates or updates a pending schema update (supersedes existing)
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="proposedVersion">The proposed version number</param>
		/// <param name="proposedTableName">The proposed table name</param>
		/// <param name="proposedSchemaName">The proposed schema/database name</param>
		/// <param name="proposedColumns">The proposed columns</param>
		/// <param name="changeType">Type of change</param>
		/// <param name="changeDescription">Human-readable description</param>
		/// <param name="comparison">Detailed comparison</param>
		/// <param name="clrTypeName">The CLR type name</param>
		/// <param name="isNestedSchema">Whether this is a nested schema</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The created/updated pending update</returns>
		Task<IPendingSchemaUpdate> UpsertPendingUpdateAsync(
			string eventName,
			int proposedVersion,
			string proposedTableName,
			string? proposedSchemaName,
			List<SchemaColumn> proposedColumns,
			SchemaChangeType changeType,
			string changeDescription,
			SchemaComparison? comparison,
			string? clrTypeName,
			bool isNestedSchema,
			CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a pending update for an event name
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The pending update or null if not found</returns>
		Task<IPendingSchemaUpdate?> GetPendingUpdateAsync(string eventName, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets all pending updates
		/// </summary>
		/// <param name="includeNested">Whether to include nested schemas</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>List of all pending updates</returns>
		Task<List<IPendingSchemaUpdate>> GetAllPendingUpdatesAsync(bool includeNested = false, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a pending update
		/// </summary>
		/// <param name="eventName">The telemetry event name</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>True if deleted, false if not found</returns>
		Task<bool> DeletePendingUpdateAsync(string eventName, CancellationToken cancellationToken = default);

		#endregion
	}
}
