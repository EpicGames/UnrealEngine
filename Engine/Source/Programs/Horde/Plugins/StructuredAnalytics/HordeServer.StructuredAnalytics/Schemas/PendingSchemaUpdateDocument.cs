// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Interface for a pending schema update awaiting admin approval
	/// </summary>
	public interface IPendingSchemaUpdate
	{
		/// <summary>
		/// Unique identifier
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// The telemetry event name this update applies to
		/// </summary>
		string EventName { get; }

		/// <summary>
		/// The proposed version number (highest existing + 1)
		/// </summary>
		int ProposedVersion { get; }

		/// <summary>
		/// The proposed table name
		/// </summary>
		string ProposedTableName { get; }

		/// <summary>
		/// The proposed schema/database name
		/// </summary>
		string? ProposedSchemaName { get; }

		/// <summary>
		/// The proposed column definitions
		/// </summary>
		IReadOnlyList<SchemaColumn> ProposedColumns { get; }

		/// <summary>
		/// When this pending update was detected
		/// </summary>
		DateTime DetectedAtUtc { get; }

		/// <summary>
		/// Type of change (New or Modified)
		/// </summary>
		SchemaChangeType ChangeType { get; }

		/// <summary>
		/// Human-readable description of the changes
		/// </summary>
		string ChangeDescription { get; }

		/// <summary>
		/// Detailed comparison with previous schema (null for new schemas)
		/// </summary>
		SchemaComparison? Comparison { get; }

		/// <summary>
		/// The fully qualified CLR type name
		/// </summary>
		string? ClrTypeName { get; }

		/// <summary>
		/// True if this is a nested schema
		/// </summary>
		bool IsNestedSchema { get; }
	}

	/// <summary>
	/// MongoDB document representing a pending schema change awaiting approval
	/// </summary>
	class PendingSchemaUpdateDocument : IPendingSchemaUpdate
	{
		/// <summary>
		/// Unique identifier
		/// </summary>
		[BsonRequired, BsonId]
		public ObjectId Id { get; set; }

		/// <summary>
		/// The telemetry event name this update applies to.
		/// Unique index ensures only one pending update per event.
		/// </summary>
		[BsonRequired]
		public string EventName { get; set; } = null!;

		/// <summary>
		/// The proposed version number (highest existing + 1)
		/// </summary>
		[BsonRequired]
		public int ProposedVersion { get; set; }

		/// <summary>
		/// The proposed table name
		/// </summary>
		[BsonRequired]
		public string ProposedTableName { get; set; } = null!;

		/// <summary>
		/// The proposed schema/database name
		/// </summary>
		[BsonIgnoreIfNull]
		public string? ProposedSchemaName { get; set; }

		/// <summary>
		/// The proposed column definitions
		/// </summary>
		[BsonRequired]
		public List<SchemaColumn> ProposedColumns { get; set; } = new();

		IReadOnlyList<SchemaColumn> IPendingSchemaUpdate.ProposedColumns => ProposedColumns;

		/// <summary>
		/// When this pending update was detected
		/// </summary>
		[BsonRequired]
		public DateTime DetectedAtUtc { get; set; }

		/// <summary>
		/// Type of change (New or Modified)
		/// </summary>
		[BsonRequired]
		public SchemaChangeType ChangeType { get; set; }

		/// <summary>
		/// Human-readable description of the changes
		/// </summary>
		[BsonRequired]
		public string ChangeDescription { get; set; } = null!;

		/// <summary>
		/// Detailed comparison with previous schema (null for new schemas)
		/// </summary>
		[BsonIgnoreIfNull]
		public SchemaComparison? Comparison { get; set; }

		/// <summary>
		/// The fully qualified CLR type name
		/// </summary>
		[BsonIgnoreIfNull]
		public string? ClrTypeName { get; set; }

		/// <summary>
		/// True if this is a nested schema
		/// </summary>
		[BsonRequired]
		public bool IsNestedSchema { get; set; }

		/// <summary>
		/// Private constructor for MongoDB deserialization
		/// </summary>
		[BsonConstructor]
		private PendingSchemaUpdateDocument()
		{
		}

		/// <summary>
		/// Creates a new pending schema update document
		/// </summary>
		public PendingSchemaUpdateDocument(
			string eventName,
			int proposedVersion,
			string proposedTableName,
			string? proposedSchemaName,
			List<SchemaColumn> proposedColumns,
			SchemaChangeType changeType,
			string changeDescription,
			SchemaComparison? comparison,
			string? clrTypeName,
			bool isNestedSchema)
		{
			Id = ObjectId.GenerateNewId();
			EventName = eventName;
			ProposedVersion = proposedVersion;
			ProposedTableName = proposedTableName;
			ProposedSchemaName = proposedSchemaName;
			ProposedColumns = proposedColumns;
			ChangeType = changeType;
			ChangeDescription = changeDescription;
			Comparison = comparison;
			ClrTypeName = clrTypeName;
			IsNestedSchema = isNestedSchema;
			DetectedAtUtc = DateTime.UtcNow;
		}
	}
}
