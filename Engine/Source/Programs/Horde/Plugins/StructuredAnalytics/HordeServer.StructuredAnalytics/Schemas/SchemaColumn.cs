// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Fundamental data types supported by telemetry schemas.
	/// These are database-agnostic types that sinks map to their native types.
	/// </summary>
	public enum SchemaDataType
	{
		/// <summary>
		/// String type. CLR mappings: string, char, Guid, enums
		/// </summary>
		String,

		/// <summary>
		/// 64-bit integer. CLR mappings: byte, sbyte, short, ushort, int, uint, long, ulong, TimeSpan (as ticks)
		/// </summary>
		Int64,

		/// <summary>
		/// Double-precision floating point. CLR mappings: float, double, decimal
		/// </summary>
		Double,

		/// <summary>
		/// Boolean. CLR mappings: bool
		/// </summary>
		Bool,

		/// <summary>
		/// Date and time. CLR mappings: DateTime, DateTimeOffset
		/// </summary>
		DateTime,

		/// <summary>
		/// Nested object with its own schema. CLR mappings: complex types
		/// References a nested schema via NestedSchemaEventName
		/// </summary>
		Object,

		/// <summary>
		/// Array/list of elements. CLR mappings: T[], List&lt;T&gt;, IEnumerable&lt;T&gt;
		/// Element type specified via ArrayElementType
		/// </summary>
		Array
	}

	/// <summary>
	/// Represents a column in a telemetry schema
	/// </summary>
	public class SchemaColumn
	{
		/// <summary>
		/// The C# property name
		/// </summary>
		[BsonRequired]
		public string PropertyName { get; set; } = null!;

		/// <summary>
		/// The database column name (from [Column] attribute)
		/// </summary>
		[BsonRequired]
		public string ColumnName { get; set; } = null!;

		/// <summary>
		/// The CLR type name (e.g., "System.String", "System.Int32", "System.Nullable`1[[System.DateTime]]")
		/// Used for advanced scenarios like enum parsing via TypeHandlers
		/// </summary>
		[BsonRequired]
		public string ClrTypeName { get; set; } = null!;

		/// <summary>
		/// The fundamental schema data type
		/// </summary>
		[BsonRequired]
		public SchemaDataType DataType { get; set; }

		/// <summary>
		/// If DataType is Array, specifies the element type
		/// </summary>
		[BsonIgnoreIfNull]
		public SchemaDataType? ArrayElementType { get; set; }

		/// <summary>
		/// If DataType is Object (or Array of Object), references the nested schema's event name.
		/// Nested schemas use synthetic event names like "__nested.{FullTypeName}"
		/// </summary>
		[BsonIgnoreIfNull]
		public string? NestedSchemaEventName { get; set; }

		/// <summary>
		/// Whether the column is nullable
		/// </summary>
		[BsonRequired]
		public bool IsNullable { get; set; }

		/// <summary>
		/// Optional ordering hint to preserve property order from reflection
		/// </summary>
		[BsonIgnoreIfNull]
		public int? Order { get; set; }
	}

	/// <summary>
	/// Describes a change to a single column
	/// </summary>
	public class ColumnChange
	{
		/// <summary>
		/// The column name that changed
		/// </summary>
		[BsonRequired]
		public string ColumnName { get; set; } = null!;

		/// <summary>
		/// The old CLR type name (null if column was added)
		/// </summary>
		[BsonIgnoreIfNull]
		public string? OldClrTypeName { get; set; }

		/// <summary>
		/// The new CLR type name (null if column was removed)
		/// </summary>
		[BsonIgnoreIfNull]
		public string? NewClrTypeName { get; set; }

		/// <summary>
		/// The old data type (null if column was added)
		/// </summary>
		[BsonIgnoreIfNull]
		public SchemaDataType? OldDataType { get; set; }

		/// <summary>
		/// The new data type (null if column was removed)
		/// </summary>
		[BsonIgnoreIfNull]
		public SchemaDataType? NewDataType { get; set; }

		/// <summary>
		/// The old nullable state (null if column was added)
		/// </summary>
		[BsonIgnoreIfNull]
		public bool? OldIsNullable { get; set; }

		/// <summary>
		/// The new nullable state (null if column was removed)
		/// </summary>
		[BsonIgnoreIfNull]
		public bool? NewIsNullable { get; set; }
	}

	/// <summary>
	/// Detailed comparison between two schema versions
	/// </summary>
	public class SchemaComparison
	{
		/// <summary>
		/// Column names that were added
		/// </summary>
		public List<string> AddedColumns { get; set; } = new();

		/// <summary>
		/// Column names that were removed
		/// </summary>
		public List<string> RemovedColumns { get; set; } = new();

		/// <summary>
		/// Columns that changed type or nullability
		/// </summary>
		public List<ColumnChange> ModifiedColumns { get; set; } = new();

		/// <summary>
		/// Whether the table name changed
		/// </summary>
		public bool TableNameChanged { get; set; }

		/// <summary>
		/// The old table name if it changed
		/// </summary>
		[BsonIgnoreIfNull]
		public string? OldTableName { get; set; }

		/// <summary>
		/// Returns true if there are any column changes
		/// </summary>
		[BsonIgnore]
		public bool HasColumnChanges => AddedColumns.Count > 0 || RemovedColumns.Count > 0 || ModifiedColumns.Count > 0;

		/// <summary>
		/// Generates a human-readable description of the changes
		/// </summary>
		public string GetChangeDescription()
		{
			List<string> parts = new();

			if (AddedColumns.Count > 0)
			{
				parts.Add($"Added {AddedColumns.Count} column(s): {String.Join(", ", AddedColumns)}");
			}

			if (RemovedColumns.Count > 0)
			{
				parts.Add($"Removed {RemovedColumns.Count} column(s): {String.Join(", ", RemovedColumns)}");
			}

			if (ModifiedColumns.Count > 0)
			{
				parts.Add($"Modified {ModifiedColumns.Count} column(s): {String.Join(", ", ModifiedColumns.Select(c => c.ColumnName))}");
			}

			if (TableNameChanged)
			{
				parts.Add($"Table name changed from '{OldTableName}'");
			}

			return parts.Count > 0 ? String.Join("; ", parts) : "No changes detected";
		}
	}

	/// <summary>
	/// Type of schema change
	/// </summary>
	public enum SchemaChangeType
	{
		/// <summary>
		/// First-time schema (no previous version exists)
		/// </summary>
		New,

		/// <summary>
		/// Modification to existing schema
		/// </summary>
		Modified
	}
}
