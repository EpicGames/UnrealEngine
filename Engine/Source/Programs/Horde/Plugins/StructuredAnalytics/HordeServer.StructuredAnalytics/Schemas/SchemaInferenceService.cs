// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.RegularExpressions;
using HordeServer.Analytics.Controllers;
using Microsoft.Extensions.Logging;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Result of inferring a schema from a sample JSON payload.
	/// </summary>
	public class InferenceResult
	{
		/// <summary>
		/// Top-level proposal the admin will review and submit via POST /api/v1/telemetry-schemas.
		/// </summary>
		public required CreateTelemetrySchemaRequest Proposal { get; set; }

		/// <summary>
		/// One proposal per nested object discovered in the sample. The caller must submit
		/// these before submitting <see cref="Proposal"/> so the parent's
		/// <c>NestedSchemaEventName</c> references resolve at lookup time.
		/// </summary>
		public List<CreateTelemetrySchemaRequest> NestedProposals { get; set; } = new();

		/// <summary>
		/// Warnings about uncertain inferences the admin should review (null fields,
		/// empty arrays, mixed-type arrays, ambiguous date strings).
		/// </summary>
		public List<InferenceWarning> Warnings { get; set; } = new();
	}

	/// <summary>
	/// A single warning emitted during schema inference.
	/// </summary>
	public class InferenceWarning
	{
		/// <summary>
		/// JSON path to the property that triggered the warning (e.g. "details.items[0]").
		/// </summary>
		public required string Path { get; set; }

		/// <summary>
		/// Human-readable explanation.
		/// </summary>
		public required string Message { get; set; }
	}

	/// <summary>
	/// Walks a single sample JSON payload and produces a <see cref="CreateTelemetrySchemaRequest"/>
	/// proposal that an admin can review and submit. Never writes to MongoDB or Redis;
	/// the inference endpoint is a pure pre-fill helper for the schema authoring form.
	/// </summary>
	public class SchemaInferenceService
	{
		readonly ILogger<SchemaInferenceService> _logger;

		// Conservative ISO-8601 matcher. We only propose DateTime for strings that look
		// unambiguously like a date — naive DateTime.TryParse accepts inputs like "1234"
		// (year 1234) which would produce false positives for arbitrary numeric strings.
		static readonly Regex s_iso8601Regex = new(@"^\d{4}-\d{2}-\d{2}([T ]\d{2}:\d{2}(:\d{2})?(\.\d+)?(Z|[+-]\d{2}:?\d{2})?)?$", RegexOptions.Compiled);

		// snake_case helpers — handle camelCase, PascalCase, and acronym-prefixed names.
		static readonly Regex s_acronymBoundary = new(@"([A-Z]+)([A-Z][a-z])", RegexOptions.Compiled);
		static readonly Regex s_wordBoundary = new(@"([a-z\d])([A-Z])", RegexOptions.Compiled);

		/// <summary>
		/// Constructor
		/// </summary>
		public SchemaInferenceService(ILogger<SchemaInferenceService> logger)
		{
			_logger = logger;
		}

		/// <summary>
		/// Infers a schema proposal from a single sample JSON object.
		/// </summary>
		/// <param name="eventName">Telemetry event name. Caller must validate non-empty.</param>
		/// <param name="tableName">Optional table name override. Defaults to the event name lowercased with dots replaced by underscores.</param>
		/// <param name="schemaName">Optional schema/database name. Null falls through to the sink's DefaultDatabaseName at write time.</param>
		/// <param name="samplePayload">A JSON object representing one example of the event payload. Caller must verify ValueKind == Object.</param>
		/// <returns>Inference result with proposal, nested proposals, and warnings.</returns>
		public InferenceResult Infer(string eventName, string? tableName, string? schemaName, JsonElement samplePayload)
		{
			List<InferenceWarning> warnings = new();
			List<CreateTelemetrySchemaRequest> nested = new();

			List<CreateSchemaColumnRequest> columns = InferObjectColumns(
				samplePayload,
				eventName,
				parentJsonPath: "",
				warnings,
				nested);

			CreateTelemetrySchemaRequest proposal = new()
			{
				EventName = eventName,
				TableName = String.IsNullOrWhiteSpace(tableName) ? DefaultTableNameFor(eventName) : tableName,
				SchemaName = String.IsNullOrWhiteSpace(schemaName) ? null : schemaName,
				ClrTypeName = null, // Admin-authored: distinguishes from reflection-detected schemas
				IsNestedSchema = false,
				Columns = columns
			};

			_logger.LogInformation(
				"Inferred schema for {EventName}: {ColumnCount} columns, {NestedCount} nested, {WarningCount} warnings",
				eventName, columns.Count, nested.Count, warnings.Count);

			return new InferenceResult
			{
				Proposal = proposal,
				NestedProposals = nested,
				Warnings = warnings
			};
		}

		/// <summary>
		/// Walks the properties of a JSON object and produces one column per property.
		/// Recurses into nested objects, emitting sibling proposals into <paramref name="nested"/>.
		/// </summary>
		List<CreateSchemaColumnRequest> InferObjectColumns(
			JsonElement obj,
			string contextEventName,
			string parentJsonPath,
			List<InferenceWarning> warnings,
			List<CreateTelemetrySchemaRequest> nested)
		{
			List<CreateSchemaColumnRequest> columns = new();

			foreach (JsonProperty prop in obj.EnumerateObject())
			{
				string propPath = String.IsNullOrEmpty(parentJsonPath) ? prop.Name : $"{parentJsonPath}.{prop.Name}";
				CreateSchemaColumnRequest? column = InferColumn(prop.Name, prop.Value, contextEventName, propPath, warnings, nested);
				if (column != null)
				{
					columns.Add(column);
				}
			}

			return columns;
		}

		/// <summary>
		/// Produces a single column from a JSON property value. Admin-authored schemas
		/// don't have a separate CLR property → DB column mapping; the JSON key is both
		/// the PropertyName the sink looks up at runtime AND the ColumnName the table
		/// uses. Keep them identical so producers' JSON keys match table columns 1:1.
		/// </summary>
		CreateSchemaColumnRequest? InferColumn(
			string propertyName,
			JsonElement value,
			string contextEventName,
			string jsonPath,
			List<InferenceWarning> warnings,
			List<CreateTelemetrySchemaRequest> nested)
		{
			(SchemaDataType dataType, SchemaDataType? arrayElementType, string? nestedSchemaEventName, string? clrTypeName) = InferType(
				value, contextEventName, propertyName, jsonPath, warnings, nested);

			return new CreateSchemaColumnRequest
			{
				PropertyName = propertyName,
				ColumnName = propertyName, // admin-authored: JSON key == column name, no transform
				ClrTypeName = clrTypeName ?? "",
				DataType = dataType,
				ArrayElementType = arrayElementType,
				NestedSchemaEventName = nestedSchemaEventName,
				IsNullable = true
			};
		}

		/// <summary>
		/// Determines the schema type for a JSON value. For objects, also enqueues a
		/// nested proposal in <paramref name="nested"/> via recursive inference.
		/// </summary>
		(SchemaDataType DataType, SchemaDataType? ArrayElementType, string? NestedSchemaEventName, string? ClrTypeName) InferType(
			JsonElement value,
			string contextEventName,
			string propertyName,
			string jsonPath,
			List<InferenceWarning> warnings,
			List<CreateTelemetrySchemaRequest> nested)
		{
			switch (value.ValueKind)
			{
				case JsonValueKind.Null:
				case JsonValueKind.Undefined:
					warnings.Add(new InferenceWarning
					{
						Path = jsonPath,
						Message = "Field is null in sample; inferred as nullable String — admin should confirm or change."
					});
					return (SchemaDataType.String, null, null, null);

				case JsonValueKind.True:
				case JsonValueKind.False:
					return (SchemaDataType.Bool, null, null, null);

				case JsonValueKind.Number:
					// Int64 first — JSON literal "42" resolves; "42.0" falls through to Double.
					return value.TryGetInt64(out _)
						? (SchemaDataType.Int64, null, null, null)
						: (SchemaDataType.Double, null, null, null);

				case JsonValueKind.String:
					string? str = value.GetString();
					if (LooksLikeIso8601DateTime(str))
					{
						warnings.Add(new InferenceWarning
						{
							Path = jsonPath,
							Message = "String value matches ISO 8601 — proposed as DateTime. Change to String if this should be stored as text."
						});
						return (SchemaDataType.DateTime, null, null, null);
					}
					return (SchemaDataType.String, null, null, null);

				case JsonValueKind.Object:
					{
						// Synthetic event name for the nested schema. Stable across invocations
						// for the same eventName + property path so re-running inference produces
						// the same proposal IDs.
						string nestedEventName = TelemetrySchemaDocument.GetNestedSchemaEventName(
							$"{contextEventName}.{jsonPath}");

						List<CreateSchemaColumnRequest> nestedColumns = InferObjectColumns(
							value, nestedEventName, parentJsonPath: "", warnings, nested);

						nested.Add(new CreateTelemetrySchemaRequest
						{
							EventName = nestedEventName,
							TableName = $"{TelemetrySchemaDocument.NestedSchemaPrefix}{ToSnakeCase(propertyName)}",
							SchemaName = null,
							ClrTypeName = null,
							IsNestedSchema = true,
							Columns = nestedColumns
						});

						return (SchemaDataType.Object, null, nestedEventName, null);
					}

				case JsonValueKind.Array:
					return InferArrayType(value, contextEventName, propertyName, jsonPath, warnings, nested);

				default:
					warnings.Add(new InferenceWarning
					{
						Path = jsonPath,
						Message = $"Unrecognised JSON value kind {value.ValueKind}; inferred as nullable String."
					});
					return (SchemaDataType.String, null, null, null);
			}
		}

		/// <summary>
		/// Determines the element type for a JSON array. Empty arrays default to String
		/// with a warning. Mixed-kind arrays fall back to String element + warning.
		/// </summary>
		(SchemaDataType DataType, SchemaDataType? ArrayElementType, string? NestedSchemaEventName, string? ClrTypeName) InferArrayType(
			JsonElement array,
			string contextEventName,
			string propertyName,
			string jsonPath,
			List<InferenceWarning> warnings,
			List<CreateTelemetrySchemaRequest> nested)
		{
			JsonValueKind? firstKind = null;
			JsonElement firstNonNull = default;
			bool mixed = false;
			int index = 0;

			foreach (JsonElement item in array.EnumerateArray())
			{
				if (item.ValueKind == JsonValueKind.Null || item.ValueKind == JsonValueKind.Undefined)
				{
					index++;
					continue;
				}

				if (firstKind == null)
				{
					firstKind = item.ValueKind;
					firstNonNull = item;
				}
				else if (item.ValueKind != firstKind.Value)
				{
					mixed = true;
				}

				index++;
			}

			if (firstKind == null)
			{
				warnings.Add(new InferenceWarning
				{
					Path = jsonPath,
					Message = "Array is empty in sample; inferred element type String — admin should confirm."
				});
				return (SchemaDataType.Array, SchemaDataType.String, null, null);
			}

			if (mixed)
			{
				warnings.Add(new InferenceWarning
				{
					Path = jsonPath,
					Message = "Array contains mixed value kinds; inferred element type String for safety."
				});
				return (SchemaDataType.Array, SchemaDataType.String, null, null);
			}

			// Single-kind array — recursively infer the element type using the first non-null
			// element. Object element types produce a nested-schema proposal whose name is
			// derived from the parent path so siblings of objects are addressable.
			(SchemaDataType elementType, _, string? nestedRef, _) = InferType(
				firstNonNull, contextEventName, propertyName, $"{jsonPath}[]", warnings, nested);

			return (SchemaDataType.Array, elementType, nestedRef, null);
		}

		static bool LooksLikeIso8601DateTime(string? s)
		{
			return !String.IsNullOrEmpty(s) && s_iso8601Regex.IsMatch(s);
		}

		/// <summary>
		/// Default table name when the caller does not supply one. Lowercases the event
		/// name and replaces dots with underscores so identifiers like "State.Agent.Telemetry"
		/// become "state_agent_telemetry" (a valid ClickHouse identifier).
		/// </summary>
		static string DefaultTableNameFor(string eventName)
		{
#pragma warning disable CA1308 // Normalize strings to uppercase
			return eventName.ToLowerInvariant().Replace('.', '_');
#pragma warning restore CA1308 // Normalize strings to uppercase
		}

		/// <summary>
		/// Converts a CamelCase / camelCase / acronym-prefixed name to snake_case.
		/// "JobId" → "job_id"; "HTTPServer" → "http_server"; "agentId" → "agent_id".
		/// </summary>
		internal static string ToSnakeCase(string name)
		{
			if (String.IsNullOrEmpty(name))
			{
				return name;
			}

			// Insert underscore between an acronym sequence and the start of a CamelCase word.
			// "HTTPServer" → "HTTP_Server".
			string firstPass = s_acronymBoundary.Replace(name, "$1_$2");

			// Insert underscore between a lowercase/digit and a following uppercase.
			// "JobId" → "Job_Id"; "value2Name" → "value2_Name".
			string secondPass = s_wordBoundary.Replace(firstPass, "$1_$2");

#pragma warning disable CA1308 // Normalize strings to uppercase
			return secondPass.ToLowerInvariant();
#pragma warning restore CA1308 // Normalize strings to uppercase
		}
	}
}
