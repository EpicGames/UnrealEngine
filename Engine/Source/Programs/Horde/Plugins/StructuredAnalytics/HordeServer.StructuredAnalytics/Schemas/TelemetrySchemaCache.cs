// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeServer.Server;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Polls a Redis version counter set by <see cref="TelemetrySchemaCollection"/> on
	/// schema mutations. When the counter advances, reloads the latest schemas from
	/// MongoDB into an in-memory snapshot and notifies subscribers.
	///
	/// Owning the polling loop here means each Horde worker runs at most one ticker and
	/// one MongoDB reload per change, regardless of how many sinks or plugins consume
	/// the schemas.
	/// </summary>
	public sealed class TelemetrySchemaCache : ITelemetrySchemaCache, IHostedService, IAsyncDisposable
	{
		readonly ITelemetrySchemaCollection _schemaCollection;
		readonly IRedisService _redisService;
		readonly ITicker _refreshTicker;
		readonly ILogger _logger;

		// Case-insensitive lookups: producers and schema authors may not agree on
		// casing for an event name (e.g. "State.Agent.Telemetry" vs "state.agent.telemetry").
		// The cache resolves via OrdinalIgnoreCase so the sink's hot path never misses
		// a registered schema due to a case mismatch.
		volatile Dictionary<string, ITelemetrySchema> _schemas = new(StringComparer.OrdinalIgnoreCase);
		long _lastSeenVersion = -1;
		bool _disposed;

		/// <inheritdoc/>
		public IReadOnlyDictionary<string, ITelemetrySchema> Schemas => _schemas;

		/// <inheritdoc/>
		public event Action<IReadOnlyDictionary<string, ITelemetrySchema>>? SchemasChanged;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetrySchemaCache(
			ITelemetrySchemaCollection schemaCollection,
			IRedisService redisService,
			IClock clock,
			ILogger<TelemetrySchemaCache> logger)
		{
			_schemaCollection = schemaCollection;
			_redisService = redisService;
			_logger = logger;

			TimeSpan refreshInterval = TimeSpan.FromMinutes(1);
			_refreshTicker = clock.AddTicker<TelemetrySchemaCache>(refreshInterval, RefreshIfChangedAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			try
			{
				long version = await ReadVersionAsync();
				await ReloadAsync(version, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Initial telemetry schema cache load failed; cache empty until next refresh tick");
			}

			await _refreshTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _refreshTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_disposed)
			{
				return;
			}
			_disposed = true;
			await _refreshTicker.DisposeAsync();
		}

		/// <summary>
		/// Ticker callback. Reads the Redis version counter; if it has advanced past the
		/// last-seen value, reloads the cache from MongoDB. A Redis or Mongo failure leaves
		/// the existing cache in place — never clears it on error.
		/// </summary>
		async ValueTask RefreshIfChangedAsync(CancellationToken cancellationToken)
		{
			long current;
			try
			{
				current = await ReadVersionAsync();
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Schema cache refresh: failed to read Redis version key, retaining existing cache");
				return;
			}

			if (current == Interlocked.Read(ref _lastSeenVersion))
			{
				return;
			}

			try
			{
				await ReloadAsync(current, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Schema cache refresh: failed to reload from MongoDB at version {Version}", current);
			}
		}

		/// <summary>
		/// Reads the current Redis schema-version counter. Returns 0 if the key is unset.
		/// </summary>
		async Task<long> ReadVersionAsync()
		{
			IDatabase redis = _redisService.GetDatabase();
			RedisValue value = await redis.StringGetAsync(TelemetrySchemaCollection.RedisSchemaVersionKey);
			return value.IsNull ? 0L : (long)value;
		}

		/// <summary>
		/// Loads all latest schemas (including nested) from MongoDB, atomically replaces
		/// the in-memory snapshot, records the new last-seen version, and notifies
		/// subscribers.
		/// </summary>
		async Task ReloadAsync(long version, CancellationToken cancellationToken)
		{
			List<ITelemetrySchema> schemas = await _schemaCollection.GetAllLatestSchemasAsync(includeNested: true, cancellationToken);

			Dictionary<string, ITelemetrySchema> next = new(schemas.Count, StringComparer.OrdinalIgnoreCase);
			foreach (ITelemetrySchema schema in schemas)
			{
				next[schema.EventName] = schema;
			}

			_schemas = next;
			Interlocked.Exchange(ref _lastSeenVersion, version);

			_logger.LogDebug("Telemetry schema cache refreshed to version {Version}: {Count} schemas", version, next.Count);

			NotifySubscribers(next);
		}

		/// <summary>
		/// Invokes each subscriber with the new snapshot. A faulty subscriber is logged
		/// and isolated so it cannot block other subscribers or take down the ticker.
		/// </summary>
		void NotifySubscribers(IReadOnlyDictionary<string, ITelemetrySchema> snapshot)
		{
			Action<IReadOnlyDictionary<string, ITelemetrySchema>>? handlers = SchemasChanged;
			if (handlers == null)
			{
				return;
			}

			foreach (Delegate handler in handlers.GetInvocationList())
			{
				try
				{
					((Action<IReadOnlyDictionary<string, ITelemetrySchema>>)handler)(snapshot);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Telemetry schema cache subscriber threw while handling change notification");
				}
			}
		}
	}
}
