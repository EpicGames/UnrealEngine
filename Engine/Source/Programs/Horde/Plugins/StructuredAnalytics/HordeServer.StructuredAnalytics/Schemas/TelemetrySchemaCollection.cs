// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Driver;
using StackExchange.Redis;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// MongoDB implementation of the telemetry schema collection
	/// </summary>
	public class TelemetrySchemaCollection : ITelemetrySchemaCollection
	{
		/// <summary>
		/// Redis key holding a monotonic counter of schema collection mutations.
		/// Subscribers (e.g. ClickHouseTelemetrySink) compare against their last-seen value
		/// to detect cluster-wide schema changes without polling MongoDB.
		/// </summary>
		public const string RedisSchemaVersionKey = "structured_analytics:schemas:version";

		readonly IMongoCollection<TelemetrySchemaDocument> _schemas;
		readonly IMongoCollection<PendingSchemaUpdateDocument> _pendingUpdates;
		readonly IRedisService _redisService;
		readonly ILogger<TelemetrySchemaCollection> _logger;

		// Index for efficiently getting latest schema by event name
		// Compound (EventName ASC, Version DESC) allows single index scan for latest version lookup
		readonly MongoIndex<TelemetrySchemaDocument> _eventNameVersionIndex;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetrySchemaCollection(IMongoService mongoService, IRedisService redisService, ILogger<TelemetrySchemaCollection> logger)
		{
			_redisService = redisService;
			_logger = logger;

			// Schema collection with indexes
			List<MongoIndex<TelemetrySchemaDocument>> schemaIndexes = new();

			// PRIMARY INDEX: Compound (EventName ASC, Version DESC) - enables efficient:
			// 1. Lookup by EventName (prefix match)
			// 2. Getting latest version (sort already in index order)
			// 3. Range queries on version within an event
			_eventNameVersionIndex = MongoIndex.Create<TelemetrySchemaDocument>(
				keys => keys.Ascending(x => x.EventName).Descending(x => x.Version));
			schemaIndexes.Add(_eventNameVersionIndex);

			// UNIQUE constraint: Ensure no duplicate (EventName, Version) combinations
			schemaIndexes.Add(MongoIndex.Create<TelemetrySchemaDocument>(
				keys => keys.Ascending(x => x.EventName).Ascending(x => x.Version),
				unique: true));

			// Index for filtering by nested schema flag
			schemaIndexes.Add(keys => keys.Ascending(x => x.IsNestedSchema));

			// Index for listing schemas by creation time
			schemaIndexes.Add(keys => keys.Descending(x => x.CreatedAtUtc));

			_schemas = mongoService.GetCollection<TelemetrySchemaDocument>("StructuredAnalytics.TelemetrySchemas", schemaIndexes);

			// Pending updates collection with unique index on EventName
			List<MongoIndex<PendingSchemaUpdateDocument>> pendingIndexes = new();

			// UNIQUE: Only one pending update per event (new detections supersede)
			pendingIndexes.Add(MongoIndex.Create<PendingSchemaUpdateDocument>(
				keys => keys.Ascending(x => x.EventName),
				unique: true));

			// Index for filtering by nested schema flag
			pendingIndexes.Add(keys => keys.Ascending(x => x.IsNestedSchema));

			// Index for listing by detection time
			pendingIndexes.Add(keys => keys.Descending(x => x.DetectedAtUtc));

			_pendingUpdates = mongoService.GetCollection<PendingSchemaUpdateDocument>("StructuredAnalytics.PendingSchemaUpdates", pendingIndexes);
		}

		#region Schema Operations

		/// <summary>
		/// Canonical form of every event name that crosses this collection's API surface.
		/// Both writes (AddSchemaAsync, UpsertPendingUpdateAsync) and reads (Get*, Delete*)
		/// normalize before touching MongoDB so casing differences between admin authoring,
		/// REST API producers, and reflection-detected schemas can never produce silent
		/// duplicates or missed lookups.
		/// </summary>
#pragma warning disable CA1308 // Normalize strings to uppercase
		static string Normalize(string eventName) => eventName.ToLowerInvariant();
#pragma warning restore CA1308 // Normalize strings to uppercase

		/// <inheritdoc/>
		public async Task<ITelemetrySchema> AddSchemaAsync(string eventName, string tableName, string? schemaName, int version, List<SchemaColumn> columns, string? clrTypeName, bool isNestedSchema, UserId? approvedByUserId = null, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			TelemetrySchemaDocument document = new TelemetrySchemaDocument(normalizedEventName, tableName, schemaName, version, columns, clrTypeName, isNestedSchema, approvedByUserId);

			await _schemas.InsertOneAsync(document, options: null, cancellationToken);

			_logger.LogInformation("Created schema {EventName} version {Version} with {ColumnCount} columns",
				normalizedEventName, version, columns.Count);

			await BumpRedisSchemaVersionAsync();

			return document;
		}

		/// <inheritdoc/>
		public async Task<ITelemetrySchema?> GetLatestSchemaAsync(string eventName, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			// Uses the (EventName ASC, Version DESC) index efficiently:
			// - Filter on EventName matches prefix
			// - Results already sorted by Version DESC from the index
			// - Limit 1 means we just take the first result (highest version)
			return await _schemas
				.Find(x => x.EventName == normalizedEventName)
				.SortByDescending(x => x.Version)
				.Limit(1)
				.FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<ITelemetrySchema?> GetSchemaAsync(string eventName, int version, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			return await _schemas.Find(x => x.EventName == normalizedEventName && x.Version == version).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<List<ITelemetrySchema>> GetSchemaVersionsAsync(string eventName, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			List<TelemetrySchemaDocument> documents = await _schemas.Find(x => x.EventName == normalizedEventName).SortByDescending(x => x.Version).ToListAsync(cancellationToken);

			return documents.Cast<ITelemetrySchema>().ToList();
		}

		/// <inheritdoc/>
		public async Task<List<ITelemetrySchema>> GetAllLatestSchemasAsync(bool includeNested = false, CancellationToken cancellationToken = default)
		{
			// Build filter
			FilterDefinition<TelemetrySchemaDocument> filter = includeNested ? Builders<TelemetrySchemaDocument>.Filter.Empty : Builders<TelemetrySchemaDocument>.Filter.Eq(x => x.IsNestedSchema, false);

			List<TelemetrySchemaDocument> allSchemas = await _schemas.Find(filter).ToListAsync(cancellationToken);

			List<ITelemetrySchema> latestSchemas = allSchemas.GroupBy(s => s.EventName).Select(g => g.OrderByDescending(s => s.Version).First()).Cast<ITelemetrySchema>().OrderBy(s => s.EventName).ToList();

			return latestSchemas;
		}

		/// <inheritdoc/>
		public async Task<bool> DeleteSchemaAsync(string eventName, int version, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			DeleteResult result = await _schemas.DeleteOneAsync(x => x.EventName == normalizedEventName && x.Version == version, cancellationToken);

			if (result.DeletedCount > 0)
			{
				_logger.LogInformation("Deleted schema {EventName} version {Version}", normalizedEventName, version);
				await BumpRedisSchemaVersionAsync();
			}

			return result.DeletedCount > 0;
		}

		/// <summary>
		/// Atomically bumps the Redis schema-version counter so subscribers in other workers
		/// detect the change on their next refresh tick. Mongo write must succeed first; a
		/// failure here is logged but does not propagate (Mongo is the source of truth and a
		/// missed bump is recovered by the next successful write or a periodic backstop reload).
		/// </summary>
		async Task BumpRedisSchemaVersionAsync()
		{
			if (_redisService.ReadOnlyMode)
			{
				return;
			}

			try
			{
				IDatabase redis = _redisService.GetDatabase();
				await redis.StringIncrementAsync(RedisSchemaVersionKey);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Failed to INCR {Key} after schema mutation; subscribers may not refresh until next bump", RedisSchemaVersionKey);
			}
		}

		/// <inheritdoc/>
		public async Task<int> GetHighestVersionAsync(string eventName, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			// Uses the (EventName ASC, Version DESC) index - single document lookup
			TelemetrySchemaDocument? latest = await _schemas
				.Find(x => x.EventName == normalizedEventName)
				.SortByDescending(x => x.Version)
				.Limit(1)
				.Project(Builders<TelemetrySchemaDocument>.Projection.Include(x => x.Version))
				.As<TelemetrySchemaDocument>()
				.FirstOrDefaultAsync(cancellationToken);

			return latest?.Version ?? 0;
		}

		#endregion

		#region Pending Update Operations

		/// <inheritdoc/>
		public async Task<IPendingSchemaUpdate> UpsertPendingUpdateAsync(
			string eventName,
			int proposedVersion,
			string proposedTableName,
			string? proposedSchemaName,
			List<SchemaColumn> proposedColumns,
			SchemaChangeType changeType,
			string changeDescription,
			SchemaComparison? comparison,
			string? clrTypeName,
			bool isNestedSchema,
			CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			PendingSchemaUpdateDocument document = new PendingSchemaUpdateDocument(
				normalizedEventName,
				proposedVersion,
				proposedTableName,
				proposedSchemaName,
				proposedColumns,
				changeType,
				changeDescription,
				comparison,
				clrTypeName,
				isNestedSchema);

			// Use ReplaceOne with upsert to supersede any existing pending update
			ReplaceOptions options = new ReplaceOptions { IsUpsert = true };

			await _pendingUpdates.ReplaceOneAsync(
				x => x.EventName == normalizedEventName,
				document,
				options,
				cancellationToken);

			_logger.LogInformation("Upserted pending update for {EventName}: {ChangeDescription}",
				normalizedEventName, changeDescription);

			return document;
		}

		/// <inheritdoc/>
		public async Task<IPendingSchemaUpdate?> GetPendingUpdateAsync(string eventName, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			// Uses unique EventName index for direct lookup
			return await _pendingUpdates
				.Find(x => x.EventName == normalizedEventName)
				.FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<List<IPendingSchemaUpdate>> GetAllPendingUpdatesAsync(bool includeNested = false, CancellationToken cancellationToken = default)
		{
			FilterDefinition<PendingSchemaUpdateDocument> filter = includeNested
				? Builders<PendingSchemaUpdateDocument>.Filter.Empty
				: Builders<PendingSchemaUpdateDocument>.Filter.Eq(x => x.IsNestedSchema, false);

			List<PendingSchemaUpdateDocument> documents = await _pendingUpdates
				.Find(filter)
				.SortByDescending(x => x.DetectedAtUtc)
				.ToListAsync(cancellationToken);

			return documents.Cast<IPendingSchemaUpdate>().ToList();
		}

		/// <inheritdoc/>
		public async Task<bool> DeletePendingUpdateAsync(string eventName, CancellationToken cancellationToken = default)
		{
			string normalizedEventName = Normalize(eventName);
			DeleteResult result = await _pendingUpdates.DeleteOneAsync(
				x => x.EventName == normalizedEventName,
				cancellationToken);

			if (result.DeletedCount > 0)
			{
				_logger.LogInformation("Deleted pending update for {EventName}", normalizedEventName);
			}

			return result.DeletedCount > 0;
		}

		#endregion
	}
}
