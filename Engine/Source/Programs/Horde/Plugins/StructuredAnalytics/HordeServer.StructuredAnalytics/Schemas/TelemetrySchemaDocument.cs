// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Interface for a telemetry schema
	/// </summary>
	public interface ITelemetrySchema
	{
		/// <summary>
		/// Unique identifier
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// The telemetry event name (from [TelemetryEvent] attribute).
		/// For nested schemas, uses synthetic format: "__nested.{FullTypeName}"
		/// </summary>
		string EventName { get; }

		/// <summary>
		/// The database table name (from [Table] attribute)
		/// </summary>
		string TableName { get; }

		/// <summary>
		/// The schema/database name (from [Table] attribute Schema property).
		/// May be null/empty if no schema was specified.
		/// </summary>
		string? SchemaName { get; }

		/// <summary>
		/// Schema version number. Latest = highest version for a given EventName.
		/// </summary>
		int Version { get; }

		/// <summary>
		/// Column definitions
		/// </summary>
		IReadOnlyList<SchemaColumn> Columns { get; }

		/// <summary>
		/// When this schema version was created
		/// </summary>
		DateTime CreatedAtUtc { get; }

		/// <summary>
		/// User who approved this schema (null for auto-created first versions)
		/// </summary>
		UserId? ApprovedByUserId { get; }

		/// <summary>
		/// The fully qualified CLR type name that this schema represents
		/// </summary>
		string? ClrTypeName { get; }

		/// <summary>
		/// True if this is a nested schema (referenced by Object columns in other schemas)
		/// </summary>
		bool IsNestedSchema { get; }
	}

	/// <summary>
	/// MongoDB document representing an approved telemetry schema version
	/// </summary>
	class TelemetrySchemaDocument : ITelemetrySchema
	{
		/// <summary>
		/// Prefix for synthetic event names used by nested schemas
		/// </summary>
		public const string NestedSchemaPrefix = "__nested.";

		/// <summary>
		/// Unique identifier
		/// </summary>
		[BsonRequired, BsonId]
		public ObjectId Id { get; set; }

		/// <summary>
		/// The telemetry event name (from [TelemetryEvent] attribute).
		/// For nested schemas, uses synthetic format: "__nested.{FullTypeName}"
		/// </summary>
		[BsonRequired]
		public string EventName { get; set; } = null!;

		/// <summary>
		/// The database table name (from [Table] attribute)
		/// </summary>
		[BsonRequired]
		public string TableName { get; set; } = null!;

		/// <summary>
		/// The schema/database name (from [Table] attribute Schema property).
		/// May be null/empty if no schema was specified.
		/// </summary>
		[BsonIgnoreIfNull]
		public string? SchemaName { get; set; }

		/// <summary>
		/// Schema version number. Latest = highest version for a given EventName.
		/// </summary>
		[BsonRequired]
		public int Version { get; set; }

		/// <summary>
		/// Column definitions
		/// </summary>
		[BsonRequired]
		public List<SchemaColumn> Columns { get; set; } = new();

		IReadOnlyList<SchemaColumn> ITelemetrySchema.Columns => Columns;

		/// <summary>
		/// When this schema version was created
		/// </summary>
		[BsonRequired]
		public DateTime CreatedAtUtc { get; set; }

		/// <summary>
		/// User who approved this schema (null for auto-created first versions)
		/// </summary>
		[BsonIgnoreIfNull]
		public UserId? ApprovedByUserId { get; set; }

		/// <summary>
		/// The fully qualified CLR type name that this schema represents
		/// </summary>
		[BsonIgnoreIfNull]
		public string? ClrTypeName { get; set; }

		/// <summary>
		/// True if this is a nested schema (referenced by Object columns in other schemas)
		/// </summary>
		[BsonRequired]
		public bool IsNestedSchema { get; set; }

		/// <summary>
		/// Private constructor for MongoDB deserialization
		/// </summary>
		[BsonConstructor]
		private TelemetrySchemaDocument()
		{
		}

		/// <summary>
		/// Creates a new schema document
		/// </summary>
		public TelemetrySchemaDocument(
			string eventName,
			string tableName,
			string? schemaName,
			int version,
			List<SchemaColumn> columns,
			string? clrTypeName,
			bool isNestedSchema,
			UserId? approvedByUserId = null)
		{
			Id = ObjectId.GenerateNewId();
			EventName = eventName;
			TableName = tableName;
			SchemaName = schemaName;
			Version = version;
			Columns = columns;
			ClrTypeName = clrTypeName;
			IsNestedSchema = isNestedSchema;
			ApprovedByUserId = approvedByUserId;
			CreatedAtUtc = DateTime.UtcNow;
		}

		/// <summary>
		/// Generates a synthetic event name for a nested schema
		/// </summary>
		public static string GetNestedSchemaEventName(string clrTypeName) => $"{NestedSchemaPrefix}{clrTypeName}";

		/// <summary>
		/// Checks if an event name represents a nested schema
		/// </summary>
		public static bool IsNestedSchemaEventName(string eventName) => eventName.StartsWith(NestedSchemaPrefix, StringComparison.Ordinal);
	}
}
