// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using System.Reflection;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Users;
using Microsoft.Extensions.Logging;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Represents a schema extracted from a CLR type via reflection
	/// </summary>
	public class ReflectedSchema
	{
		/// <summary>
		/// The telemetry event name (from <see cref="TelemetryEventAttribute"/> attribute)
		/// </summary>
		public required string EventName { get; init; }

		/// <summary>
		/// The database table name (from <see cref="TableAttribute"/> attribute)
		/// </summary>
		public required string TableName { get; init; }

		/// <summary>
		/// The schema/database name (from <see cref="TableAttribute"/> attribute Schema property)
		/// </summary>
		public string? SchemaName { get; init; }

		/// <summary>
		/// The CLR type that this schema represents
		/// </summary>
		public required Type ClrType { get; init; }

		/// <summary>
		/// Column definitions
		/// </summary>
		public required List<SchemaColumn> Columns { get; init; }

		/// <summary>
		/// Whether this is a nested schema
		/// </summary>
		public required bool IsNestedSchema { get; init; }

		/// <summary>
		/// Nested types that this schema references.
		/// </summary>
		public required List<Type> ReferencedNestedTypes { get; init; }
	}

	/// <summary>
	/// Service for managing telemetry schemas, including reflection-based schema extraction,
	/// comparison, and approval workflows.
	/// </summary>
	public class TelemetrySchemaService
	{
		readonly ITelemetrySchemaCollection _schemaCollection;
		readonly ILogger<TelemetrySchemaService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetrySchemaService(
			ITelemetrySchemaCollection schemaCollection,
			ILogger<TelemetrySchemaService> logger)
		{
			_schemaCollection = schemaCollection;
			_logger = logger;
		}

		#region Schema Reflection

		/// <summary>
		/// Builds a schema from a CLR type that has <see cref="AnalyticsTableGenAttribute"/> and <see cref="TelemetryEventAttribute"/> attributes
		/// </summary>
		/// <returns>The reflected schema, or null if the type is not a valid telemetry type or an error occurred</returns>
		public ReflectedSchema? BuildSchemaFromType(Type type)
		{
			try
			{
				if (type == null)
				{
					return null;
				}

				// Must have [TelemetryEvent] attribute
				TelemetryEventAttribute? telemetryAttr;
				try
				{
					telemetryAttr = type.GetCustomAttribute<TelemetryEventAttribute>();
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Failed to get TelemetryEventAttribute from type {Type}", type.FullName);
					return null;
				}

				if (telemetryAttr == null)
				{
					return null;
				}

				// Must have [AnalyticsTableGen] attribute
				AnalyticsTableGenAttribute? tableGenAttr;
				try
				{
					tableGenAttr = type.GetCustomAttribute<AnalyticsTableGenAttribute>();
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Failed to get AnalyticsTableGenAttribute from type {Type}", type.FullName);
					return null;
				}

				if (tableGenAttr == null)
				{
					return null;
				}

				// Must have [Table] attribute
				TableAttribute? tableAttr;
				try
				{
					tableAttr = type.GetCustomAttribute<TableAttribute>();
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Failed to get TableAttribute from type {Type}", type.FullName);
					return null;
				}

				if (tableAttr == null || String.IsNullOrEmpty(tableAttr.Name))
				{
					_logger.LogWarning("Type {Type} has [AnalyticsTableGen] but no [Table] attribute", type.FullName);
					return null;
				}

				return BuildSchemaFromTypeInternal(
					type,
					telemetryAttr.EventName,
					tableAttr.Name,
					tableAttr.Schema,
					isNestedSchema: false);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to build schema from type {Type}", type?.FullName ?? "(null)");
				return null;
			}
		}

		/// <summary>
		/// Builds a nested schema from a complex type (no attributes required)
		/// </summary>
		/// <returns>The reflected schema, or null if an error occurred</returns>
		public ReflectedSchema? BuildNestedSchema(Type type)
		{
			try
			{
				if (type == null)
				{
					return null;
				}

				string typeName = type.FullName ?? type.Name;
				string eventName = TelemetrySchemaDocument.GetNestedSchemaEventName(typeName);

				// Nested types don't have [Table] attribute, use a synthetic table name
				string tableName = $"{TelemetrySchemaDocument.NestedSchemaPrefix}{type.Name}";

				return BuildSchemaFromTypeInternal(type, eventName, tableName, schemaName: null, isNestedSchema: true);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to build nested schema from type {Type}", type?.FullName ?? "(null)");
				return null;
			}
		}

		ReflectedSchema? BuildSchemaFromTypeInternal(Type type, string eventName, string tableName, string? schemaName, bool isNestedSchema)
		{
			try
			{
				List<SchemaColumn> columns = new();
				List<Type> referencedNestedTypes = new();
				HashSet<string> seenColumns = new(StringComparer.OrdinalIgnoreCase);
				int order = 0;

				// Walk type hierarchy to get inherited properties too
				Type? current = type;
				while (current != null && current != typeof(object))
				{
					PropertyInfo[] properties;
					try
					{
						properties = current.GetProperties(BindingFlags.Public | BindingFlags.Instance | BindingFlags.DeclaredOnly);
					}
					catch (Exception ex)
					{
						_logger.LogDebug(ex, "Failed to get properties from type {Type}", current.FullName);
						current = current.BaseType;
						continue;
					}

					foreach (PropertyInfo prop in properties)
					{
						try
						{
							ColumnAttribute? columnAttr = prop.GetCustomAttribute<ColumnAttribute>();
							if (columnAttr == null || String.IsNullOrEmpty(columnAttr.Name))
							{
								continue;
							}

							// Avoid duplicates from shadowed properties
							if (seenColumns.Contains(columnAttr.Name))
							{
								continue;
							}

							seenColumns.Add(columnAttr.Name);

							// Map CLR type to fundamental schema type
							Type propertyType;
							try
							{
								propertyType = prop.PropertyType;
							}
							catch (Exception ex)
							{
								_logger.LogDebug(ex, "Failed to get property type for {Property} on {Type}", prop.Name, current.FullName);
								continue;
							}

							(SchemaDataType dataType, SchemaDataType? arrayElementType, Type? nestedType) =
								MapClrTypeToSchemaTypeSafe(propertyType);

							// Track nested types for processing
							if (nestedType != null && !referencedNestedTypes.Contains(nestedType))
							{
								referencedNestedTypes.Add(nestedType);
							}

							string? nestedSchemaEventName = null;
							if (nestedType != null)
							{
								string nestedTypeName = nestedType.FullName ?? nestedType.Name;
								nestedSchemaEventName = TelemetrySchemaDocument.GetNestedSchemaEventName(nestedTypeName);
							}

							columns.Add(new SchemaColumn
							{
								PropertyName = prop.Name,
								ColumnName = columnAttr.Name,
								ClrTypeName = GetClrTypeNameSafe(propertyType),
								DataType = dataType,
								ArrayElementType = arrayElementType,
								NestedSchemaEventName = nestedSchemaEventName,
								IsNullable = IsNullableTypeSafe(propertyType),
								Order = order++
							});
						}
						catch (Exception ex)
						{
							_logger.LogDebug(ex, "Failed to process property {Property} on type {Type}", prop.Name, current.FullName);
							// Continue processing other properties
						}
					}

					try
					{
						Type? baseType = current.BaseType;
						if (baseType == null)
						{
							break;
						}
						current = baseType;
					}
					catch (Exception ex)
					{
						_logger.LogDebug(ex, "Failed to get base type of {Type}", current.FullName);
						break;
					}
				}

				if (columns.Count == 0)
				{
					_logger.LogDebug("Type {Type} has no valid columns with [Column] attribute", type.FullName);
					return null;
				}

				return new ReflectedSchema
				{
					EventName = eventName,
					TableName = tableName,
					SchemaName = schemaName,
					ClrType = type,
					Columns = columns,
					IsNestedSchema = isNestedSchema,
					ReferencedNestedTypes = referencedNestedTypes
				};
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to build schema internally for type {Type}", type.FullName);
				return null;
			}
		}

		/// <summary>
		/// Maps a CLR type to the fundamental schema type system (safe version with try/catch)
		/// </summary>
		/// <returns>Tuple of (DataType, ArrayElementType if Array, NestedType if Object/Array of Object)</returns>
		(SchemaDataType DataType, SchemaDataType? ArrayElementType, Type? NestedType) MapClrTypeToSchemaTypeSafe(Type clrType)
		{
			try
			{
				return MapClrTypeToSchemaType(clrType);
			}
			catch (Exception ex)
			{
				_logger.LogDebug(ex, "Failed to map CLR type {Type}, defaulting to String", clrType.FullName);
				return (SchemaDataType.String, null, null);
			}
		}

		/// <summary>
		/// Maps a CLR type to the fundamental schema type system
		/// </summary>
		static (SchemaDataType DataType, SchemaDataType? ArrayElementType, Type? NestedType) MapClrTypeToSchemaType(Type clrType)
		{
			// Unwrap nullable
			Type underlyingType = Nullable.GetUnderlyingType(clrType) ?? clrType;

			// Check for arrays first
			if (underlyingType.IsArray)
			{
				Type? elementType = underlyingType.GetElementType();
				if (elementType != null)
				{
					(SchemaDataType elementDataType, _, Type? nestedType) = MapClrTypeToSchemaType(elementType);
					return (SchemaDataType.Array, elementDataType, nestedType);
				}
				return (SchemaDataType.Array, SchemaDataType.String, null); // Fallback
			}

			// Check for generic collections (List<T>, IEnumerable<T>, etc.)
			if (underlyingType.IsGenericType)
			{
				Type genericDef = underlyingType.GetGenericTypeDefinition();
				if (genericDef == typeof(List<>) ||
					genericDef == typeof(IList<>) ||
					genericDef == typeof(ICollection<>) ||
					genericDef == typeof(IEnumerable<>) ||
					genericDef == typeof(IReadOnlyList<>) ||
					genericDef == typeof(IReadOnlyCollection<>))
				{
					Type[] genericArgs = underlyingType.GetGenericArguments();
					if (genericArgs.Length > 0)
					{
						Type elementType = genericArgs[0];
						(SchemaDataType elementDataType, _, Type? nestedType) = MapClrTypeToSchemaType(elementType);
						return (SchemaDataType.Array, elementDataType, nestedType);
					}
				}
			}

			// Map primitive and common types
			if (underlyingType == typeof(string))
			{
				return (SchemaDataType.String, null, null);
			}
			if (underlyingType == typeof(char))
			{
				return (SchemaDataType.String, null, null);
			}
			if (underlyingType == typeof(Guid))
			{
				return (SchemaDataType.String, null, null);
			}
			if (underlyingType == typeof(MongoDB.Bson.ObjectId))
			{
				return (SchemaDataType.String, null, null);
			}

			// Integer types -> Int64
			if (underlyingType == typeof(byte))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(sbyte))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(short))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(ushort))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(int))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(uint))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(long))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(ulong))
			{
				return (SchemaDataType.Int64, null, null);
			}
			if (underlyingType == typeof(TimeSpan))
			{
				return (SchemaDataType.Int64, null, null); // Store as ticks
			}

			// Floating point types -> Double
			if (underlyingType == typeof(float))
			{
				return (SchemaDataType.Double, null, null);
			}
			if (underlyingType == typeof(double))
			{
				return (SchemaDataType.Double, null, null);
			}
			if (underlyingType == typeof(decimal))
			{
				return (SchemaDataType.Double, null, null);
			}

			// Boolean
			if (underlyingType == typeof(bool))
			{
				return (SchemaDataType.Bool, null, null);
			}

			// DateTime types
			if (underlyingType == typeof(DateTime))
			{
				return (SchemaDataType.DateTime, null, null);
			}

			if (underlyingType == typeof(DateTimeOffset))
			{
				return (SchemaDataType.DateTime, null, null);
			}
			if (underlyingType == typeof(DateOnly))
			{
				return (SchemaDataType.DateTime, null, null);
			}
			if (underlyingType == typeof(TimeOnly))
			{
				return (SchemaDataType.DateTime, null, null);
			}

			// Enums -> String (parsed via Enum.Parse in TypeHandlers)
			if (underlyingType.IsEnum)
			{
				return (SchemaDataType.String, null, null);
			}

			// Check for "primitive-like" types that serialize as strings
			// These are typically ID types with [JsonSchemaString], TypeConverter, or wrap StringId
			if (IsPrimitiveLikeType(underlyingType))
			{
				return (SchemaDataType.String, null, null);
			}

			// Complex types -> Object with nested schema
			return (SchemaDataType.Object, null, underlyingType);
		}

		/// <summary>
		/// Gets the full CLR type name for storage (safe version)
		/// </summary>
		static string GetClrTypeNameSafe(Type type)
		{
			try
			{
				return type.FullName ?? type.Name;
			}
			catch
			{
				return type.Name;
			}
		}

		/// <summary>
		/// Checks if a type is nullable (reference type or Nullable&lt;T&gt;) - safe version
		/// </summary>
		static bool IsNullableTypeSafe(Type type)
		{
			try
			{
				if (!type.IsValueType)
				{
					return true;
				}
				return Nullable.GetUnderlyingType(type) != null;
			}
			catch
			{
				return true; // Default to nullable on error
			}
		}

		/// <summary>
		/// Determines if a type is "primitive-like" - meaning it has a well-defined string representation
		/// and should not be treated as a nested schema. Common examples: StreamId, UserId, ProjectId, etc.
		/// </summary>
		/// <remarks>
		/// Detection heuristics (in priority order):
		/// 1. Has [JsonSchemaString] attribute - indicates JSON serialization as string
		/// </remarks>
		static bool IsPrimitiveLikeType(Type type)
		{
			try
			{
				// Check for [JsonSchemaString] attribute - strongest signal
				if (type.GetCustomAttribute(typeof(EpicGames.Core.JsonSchemaStringAttribute), false) != null)
				{
					return true;
				}

				return false;
			}
			catch
			{
				// On any reflection error, assume it's not primitive-like
				return false;
			}
		}

		#endregion

		#region Schema Comparison

		/// <summary>
		/// Compares two schemas and returns the differences
		/// </summary>
		public SchemaComparison CompareSchemas(
			IReadOnlyList<SchemaColumn> existingColumns,
			string existingTableName,
			IReadOnlyList<SchemaColumn> newColumns,
			string newTableName)
		{
			SchemaComparison comparison = new SchemaComparison();

			try
			{
				// Check table name change
				if (!String.Equals(existingTableName, newTableName, StringComparison.Ordinal))
				{
					comparison.TableNameChanged = true;
					comparison.OldTableName = existingTableName;
				}

				// Build lookup by column name
				Dictionary<string, SchemaColumn> existingByName = new(StringComparer.OrdinalIgnoreCase);
				foreach (SchemaColumn col in existingColumns)
				{
					if (!String.IsNullOrEmpty(col.ColumnName))
					{
						existingByName[col.ColumnName] = col;
					}
				}

				Dictionary<string, SchemaColumn> newByName = new(StringComparer.OrdinalIgnoreCase);
				foreach (SchemaColumn col in newColumns)
				{
					if (!String.IsNullOrEmpty(col.ColumnName))
					{
						newByName[col.ColumnName] = col;
					}
				}

				// Find added columns (in new but not in existing)
				foreach (SchemaColumn newCol in newColumns)
				{
					if (!String.IsNullOrEmpty(newCol.ColumnName) && !existingByName.ContainsKey(newCol.ColumnName))
					{
						comparison.AddedColumns.Add(newCol.ColumnName);
					}
				}

				// Find removed columns (in existing but not in new)
				foreach (SchemaColumn existingCol in existingColumns)
				{
					if (!String.IsNullOrEmpty(existingCol.ColumnName) && !newByName.ContainsKey(existingCol.ColumnName))
					{
						comparison.RemovedColumns.Add(existingCol.ColumnName);
					}
				}

				// Find modified columns (same name but different type/nullability)
				foreach (SchemaColumn existingCol in existingColumns)
				{
					if (!String.IsNullOrEmpty(existingCol.ColumnName) &&
						newByName.TryGetValue(existingCol.ColumnName, out SchemaColumn? newCol))
					{
						bool typeChanged = existingCol.DataType != newCol.DataType ||
										   existingCol.ArrayElementType != newCol.ArrayElementType ||
										   existingCol.NestedSchemaEventName != newCol.NestedSchemaEventName;
						bool nullabilityChanged = existingCol.IsNullable != newCol.IsNullable;

						if (typeChanged || nullabilityChanged)
						{
							comparison.ModifiedColumns.Add(new ColumnChange
							{
								ColumnName = existingCol.ColumnName,
								OldClrTypeName = existingCol.ClrTypeName,
								NewClrTypeName = newCol.ClrTypeName,
								OldDataType = existingCol.DataType,
								NewDataType = newCol.DataType,
								OldIsNullable = existingCol.IsNullable,
								NewIsNullable = newCol.IsNullable
							});
						}
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error comparing schemas");
				// Return whatever we've collected so far
			}

			return comparison;
		}

		#endregion

		#region Approval Workflow

		/// <summary>
		/// Approves a pending schema update, creating a new schema version
		/// </summary>
		/// <param name="eventName">The event name of the pending update</param>
		/// <param name="approvedByUserId">The user approving the update</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The newly created schema, or null if no pending update exists</returns>
		public async Task<ITelemetrySchema?> ApproveSchemaAsync(
			string eventName,
			UserId approvedByUserId,
			CancellationToken cancellationToken = default)
		{
			try
			{
				// Get the pending update
				IPendingSchemaUpdate? pending = await _schemaCollection.GetPendingUpdateAsync(eventName, cancellationToken);
				if (pending == null)
				{
					_logger.LogWarning("No pending update found for {EventName}", eventName);
					return null;
				}

				// Create the new schema version
				ITelemetrySchema schema = await _schemaCollection.AddSchemaAsync(
					eventName,
					pending.ProposedTableName,
					pending.ProposedSchemaName,
					pending.ProposedVersion,
					pending.ProposedColumns.ToList(),
					pending.ClrTypeName,
					pending.IsNestedSchema,
					approvedByUserId,
					cancellationToken);

				// Delete the pending update
				await _schemaCollection.DeletePendingUpdateAsync(eventName, cancellationToken);

				_logger.LogInformation("Approved schema {EventName} version {Version} by user {UserId}",
					eventName, pending.ProposedVersion, approvedByUserId);

				return schema;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to approve schema {EventName}", eventName);
				throw;
			}
		}

		/// <summary>
		/// Rejects a pending schema update
		/// </summary>
		/// <param name="eventName">The event name of the pending update</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>True if deleted, false if not found</returns>
		public async Task<bool> RejectSchemaAsync(string eventName, CancellationToken cancellationToken = default)
		{
			try
			{
				bool deleted = await _schemaCollection.DeletePendingUpdateAsync(eventName, cancellationToken);

				if (deleted)
				{
					_logger.LogInformation("Rejected pending schema update for {EventName}", eventName);
				}

				return deleted;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to reject schema {EventName}", eventName);
				throw;
			}
		}

		#endregion

		#region Sync Operations

		/// <summary>
		/// Syncs a reflected schema with the database.
		/// Creates new schema if none exists, or creates pending update if columns differ.
		/// </summary>
		/// <param name="reflected">The schema extracted from reflection</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Sync result indicating what action was taken</returns>
		public async Task<SchemaSyncResult> SyncSchemaAsync(ReflectedSchema reflected, CancellationToken cancellationToken = default)
		{
			try
			{
				// Check if schema exists
				ITelemetrySchema? existing = await _schemaCollection.GetLatestSchemaAsync(reflected.EventName, cancellationToken);

				if (existing == null)
				{
					// No existing schema - auto-create version 1
					await _schemaCollection.AddSchemaAsync(
						reflected.EventName,
						reflected.TableName,
						reflected.SchemaName,
						version: 1,
						reflected.Columns,
						reflected.ClrType.FullName,
						reflected.IsNestedSchema,
						approvedByUserId: null, // Auto-created
						cancellationToken);

					_logger.LogInformation("Auto-created schema {EventName} version 1 with {ColumnCount} columns",
						reflected.EventName, reflected.Columns.Count);

					return SchemaSyncResult.Created;
				}

				// Compare columns
				SchemaComparison comparison = CompareSchemas(
					existing.Columns,
					existing.TableName,
					reflected.Columns,
					reflected.TableName);

				if (!comparison.HasColumnChanges)
				{
					// No changes
					return SchemaSyncResult.Unchanged;
				}

				// Column changes detected - create pending update
				int nextVersion = existing.Version + 1;

				await _schemaCollection.UpsertPendingUpdateAsync(
					reflected.EventName,
					nextVersion,
					reflected.TableName,
					reflected.SchemaName,
					reflected.Columns,
					SchemaChangeType.Modified,
					comparison.GetChangeDescription(),
					comparison,
					reflected.ClrType.FullName,
					reflected.IsNestedSchema,
					cancellationToken);

				_logger.LogInformation("Created pending update for {EventName}: {Changes}",
					reflected.EventName, comparison.GetChangeDescription());

				return SchemaSyncResult.PendingCreated;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to sync schema {EventName}", reflected.EventName);
				return SchemaSyncResult.Error;
			}
		}

		#endregion
	}

	/// <summary>
	/// Result of syncing a schema with the database
	/// </summary>
	public enum SchemaSyncResult
	{
		/// <summary>
		/// Schema was auto-created (first version)
		/// </summary>
		Created,

		/// <summary>
		/// Schema exists and matches - no action taken
		/// </summary>
		Unchanged,

		/// <summary>
		/// Schema differs - pending update created
		/// </summary>
		PendingCreated,

		/// <summary>
		/// An error occurred during sync
		/// </summary>
		Error
	}
}
