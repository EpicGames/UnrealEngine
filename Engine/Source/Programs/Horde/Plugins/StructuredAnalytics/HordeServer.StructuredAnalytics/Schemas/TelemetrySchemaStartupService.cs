// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Analytics.Telemetry;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Analytics.Schemas
{
	/// <summary>
	/// Hosted service that scans assemblies at startup to discover and sync telemetry schemas.
	/// </summary>
	public class TelemetrySchemaStartupService : IHostedService
	{
		readonly TelemetrySchemaService _schemaService;
		readonly ILogger<TelemetrySchemaStartupService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetrySchemaStartupService(
			TelemetrySchemaService schemaService,
			ILogger<TelemetrySchemaStartupService> logger)
		{
			_schemaService = schemaService;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("TelemetrySchemaStartupService: Starting schema sync...");

			try
			{
				await ScanAndSyncSchemasAsync(cancellationToken);
			}
			catch (OperationCanceledException)
			{
				_logger.LogInformation("TelemetrySchemaStartupService: Startup cancelled");
				throw;
			}
			catch (Exception ex)
			{
				// Log but don't throw - allow server to start even if sync fails
				_logger.LogError(ex, "TelemetrySchemaStartupService: Failed to sync schemas at startup. The server will continue to operate but schema sync may be incomplete.");
			}
		}

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken)
		{
			return Task.CompletedTask;
		}

		/// <summary>
		/// Scans all loaded assemblies for types with [AnalyticsTableGen] and [TelemetryEvent] attributes,
		/// then syncs them with the database.
		/// </summary>
		async Task ScanAndSyncSchemasAsync(CancellationToken cancellationToken)
		{
			int discoveredCount = 0;
			int createdCount = 0;
			int unchangedCount = 0;
			int pendingCount = 0;
			int errorCount = 0;

			// Track types we've already processed (including nested types)
			HashSet<Type> processedTypes = new();

			// Queue for processing nested types
			Queue<Type> nestedTypesToProcess = new();

			// First pass: discover all top-level telemetry types
			List<ReflectedSchema> discoveredSchemas = new();

			Assembly[] assemblies;
			try
			{
				assemblies = AppDomain.CurrentDomain.GetAssemblies();
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to get loaded assemblies");
				return;
			}

			foreach (Assembly assembly in assemblies)
			{
				// Skip Microsoft/System standard libraries only
				string? assemblyName = null;
				try
				{
					assemblyName = assembly.GetName().Name;
					if (assemblyName != null && ShouldSkipAssembly(assemblyName))
					{
						continue;
					}
				}
				catch
				{
					// Can't get assembly name, skip it
					continue;
				}

				Type[] types;
				try
				{
					types = assembly.GetTypes();
				}
				catch (ReflectionTypeLoadException ex)
				{
					// Some types may fail to load - process the ones that succeeded
					types = ex.Types.Where(t => t != null).Cast<Type>().ToArray();
					_logger.LogDebug("Partial type load for assembly {Assembly}: {LoaderExceptionCount} loader exceptions", assemblyName, ex.LoaderExceptions?.Length ?? 0);
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Failed to get types from assembly {Assembly}", assemblyName);
					continue;
				}

				foreach (Type type in types)
				{
					cancellationToken.ThrowIfCancellationRequested();

					try
					{
						// Skip abstract types and interfaces
						if (type.IsAbstract || type.IsInterface)
						{
							continue;
						}

						// Quick check for required attributes before full processing
						if (!HasRequiredAttributes(type))
						{
							continue;
						}

						// Already processed?
						if (!processedTypes.Add(type))
						{
							continue;
						}

						// Build schema from type
						ReflectedSchema? schema = _schemaService.BuildSchemaFromType(type);
						if (schema == null)
						{
							continue;
						}

						discoveredSchemas.Add(schema);
						discoveredCount++;

						// Queue any nested types for processing
						foreach (Type nestedType in schema.ReferencedNestedTypes)
						{
							if (processedTypes.Add(nestedType))
							{
								nestedTypesToProcess.Enqueue(nestedType);
							}
						}
					}
					catch (Exception ex)
					{
						_logger.LogDebug(ex, "Error processing type {Type}", type.FullName);
						errorCount++;
					}
				}
			}

			_logger.LogInformation("TelemetrySchemaStartupService: Discovered {Count} top-level telemetry types", discoveredCount);

			// Process nested types
			int nestedCount = 0;
			while (nestedTypesToProcess.Count > 0)
			{
				cancellationToken.ThrowIfCancellationRequested();

				Type nestedType = nestedTypesToProcess.Dequeue();

				try
				{
					ReflectedSchema? nestedSchema = _schemaService.BuildNestedSchema(nestedType);
					if (nestedSchema != null)
					{
						discoveredSchemas.Add(nestedSchema);
						nestedCount++;

						// Queue any nested types referenced by this nested type
						foreach (Type subNestedType in nestedSchema.ReferencedNestedTypes)
						{
							if (processedTypes.Add(subNestedType))
							{
								nestedTypesToProcess.Enqueue(subNestedType);
							}
						}
					}
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Error processing nested type {Type}", nestedType.FullName);
					errorCount++;
				}
			}

			if (nestedCount > 0)
			{
				_logger.LogInformation("TelemetrySchemaStartupService: Discovered {Count} nested types", nestedCount);
			}

			// Sync all discovered schemas with the database
			foreach (ReflectedSchema schema in discoveredSchemas)
			{
				cancellationToken.ThrowIfCancellationRequested();

				try
				{
					SchemaSyncResult result = await _schemaService.SyncSchemaAsync(schema, cancellationToken);

					switch (result)
					{
						case SchemaSyncResult.Created:
							createdCount++;
							break;
						case SchemaSyncResult.Unchanged:
							unchangedCount++;
							break;
						case SchemaSyncResult.PendingCreated:
							pendingCount++;
							break;
						case SchemaSyncResult.Error:
							errorCount++;
							break;
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Failed to sync schema {EventName}", schema.EventName);
					errorCount++;
				}
			}

			_logger.LogInformation("TelemetrySchemaStartupService: Sync complete. Discovered: {Discovered}, Created: {Created}, Unchanged: {Unchanged}, Pending: {Pending}, Errors: {Errors}", discoveredCount + nestedCount, createdCount, unchangedCount, pendingCount, errorCount);
		}

		/// <summary>
		/// Quick check for required attributes without full reflection
		/// </summary>
		static bool HasRequiredAttributes(Type type)
		{
			try
			{
				// Check for both required attributes
				return type.GetCustomAttribute<TelemetryEventAttribute>() != null &&
					   type.GetCustomAttribute<AnalyticsTableGenAttribute>() != null;
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Determines if an assembly should be skipped for telemetry scanning.
		/// Only skips Microsoft/System standard library assemblies.
		/// </summary>
		static bool ShouldSkipAssembly(string assemblyName)
		{
			// Only skip Microsoft and System standard library assemblies
			if (assemblyName.StartsWith("System", StringComparison.OrdinalIgnoreCase) ||
				assemblyName.StartsWith("Microsoft", StringComparison.OrdinalIgnoreCase) ||
				assemblyName.StartsWith("mscorlib", StringComparison.OrdinalIgnoreCase) ||
				assemblyName.StartsWith("netstandard", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}

			return false;
		}
	}
}
