// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the StructuredAnalytics plugin.
	///
	/// StructuredAnalytics is backend-agnostic: it consumes whatever <see cref="HordeServer.Analytics.IAnalyticsDataSource"/> the named backend plugin registers under <see cref="BackendName"/>. Connection strings, OAuth fields, and ODBC plumbing live on the backend plugin's config — not here.
	/// </summary>
	public class StructuredAnalyticsConfig : IPluginConfig, IAnalyticsBackendSelectorConfig
	{
		/// <summary>
		/// Selects which backend plugin's <see cref="IAnalyticsDataSource"/> this consumer binds to. Must match a key registered by an enabled backend plugin (e.g. <c>"Odbc"</c> for <c>OdbcTelemetry</c>, <c>"ClickHouse"</c> for <c>ClickHouseTelemetry</c>).
		/// </summary>
		public string BackendName { get; set; } = String.Empty;

		/// <summary>
		/// Whether to disable stream-based authorization for analytics endpoints.
		/// When true, all authenticated users can access analytics data without stream-level ACL checks.
		/// </summary>
		public bool DisableStreamBasedAuth { get; set; } = false;

		/// <summary>
		/// Whether to allow non-admin users to access the endpoints that don't have stream guards.
		/// When false (default), only users with admin claims can access analytics data.
		/// When true, all authenticated users can access analytics telemetry data.
		/// </summary>
		public bool AllowNonAdminQueryAccess { get; set; } = false;

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
		}
	}
}
