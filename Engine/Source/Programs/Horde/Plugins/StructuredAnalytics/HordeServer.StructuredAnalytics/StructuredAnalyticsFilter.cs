// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;
using System.Text;
using EpicGames.Horde.Streams;
using HordeServer.Dialects;

namespace HordeServer.Analytics
{
	/// <summary>
	/// One bound parameter — name, value, and logical <see cref="DbType"/> — produced by <see cref="WhereClauseBuilder"/> and consumed by <see cref="WhereClauseResult.ApplyTo"/>. Backend-agnostic; the concrete <see cref="DbParameter"/> is materialised from the command's own factory at apply time.
	/// </summary>
	public readonly struct ParameterEntry : IEquatable<ParameterEntry>
	{
		/// <summary>
		/// Auto-generated parameter name (e.g. <c>p0</c>, <c>p1</c>) corresponding to the placeholder in the clause text.
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Parameter value. Null is preserved here and converted to <see cref="DBNull.Value"/> by the dialect at bind time.
		/// </summary>
		public object? Value { get; }

		/// <summary>
		/// Logical type of the value.
		/// </summary>
		public DbType DbType { get; }

		/// <summary>
		/// Constructor.
		/// </summary>
		public ParameterEntry(string name, object? value, DbType dbType)
		{
			Name = name;
			Value = value;
			DbType = dbType;
		}

		/// <inheritdoc/>
		public bool Equals(ParameterEntry other) => Name == other.Name && DbType == other.DbType && Equals(Value, other.Value);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is ParameterEntry other && Equals(other);

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(Name, Value, DbType);

		/// <summary>
		/// Equality.
		/// </summary>
		public static bool operator ==(ParameterEntry left, ParameterEntry right) => left.Equals(right);

		/// <summary>
		/// Inequality.
		/// </summary>
		public static bool operator !=(ParameterEntry left, ParameterEntry right) => !left.Equals(right);
	}

	/// <summary>
	/// Fluent builder for constructing SQL WHERE clauses with parameterized queries.
	/// </summary>
	/// <remarks>
	/// Placeholder syntax is dialect-controlled: pass an <see cref="ISqlDialect"/> in the constructor to emit backend-specific placeholders (e.g. <c>{p0:Int32}</c> for ClickHouse). The no-arg constructor defaults to positional <c>?</c> placeholders for ODBC and tests.
	/// </remarks>
	public class WhereClauseBuilder
	{
		private readonly ISqlDialect _dialect;
		private readonly List<string> _clauses = new();
		private readonly List<ParameterEntry> _parameters = new();
		private readonly StringBuilder _currentGroup = new();
		private string _pendingOperator = String.Empty;

		/// <summary>
		/// Constructs a builder that emits positional <c>?</c> placeholders. Used by tests and by call sites that don't have access to a backend's dialect via DI.
		/// </summary>
		public WhereClauseBuilder() : this(PositionalQuestionMarkDialect.Instance)
		{
		}

		/// <summary>
		/// Constructs a builder that delegates placeholder formatting and parameter binding to <paramref name="dialect"/>. Backend-fronted call sites (e.g. <c>AbstractAnalyticsService.Dialect</c>) supply this.
		/// </summary>
		public WhereClauseBuilder(ISqlDialect dialect)
		{
			_dialect = dialect;
		}

		/// <summary>
		/// Adds an AND condition.
		/// </summary>
		/// <param name="column">The column name.</param>
		/// <param name="op">The operator (e.g., "=", ">=", "LIKE").</param>
		/// <param name="value">The parameter value.</param>
		/// <param name="dbType">The logical type of the parameter.</param>
		public WhereClauseBuilder And(string column, string op, object value, DbType dbType)
		{
			if (NeedsOperator())
			{
				_pendingOperator = " AND ";
			}

			AppendCondition(column, op, value, dbType);

			return this;
		}

		/// <summary>
		/// Adds an OR condition.
		/// </summary>
		/// <param name="column">The column name.</param>
		/// <param name="op">The operator (e.g., "=", ">=", "LIKE").</param>
		/// <param name="value">The parameter value.</param>
		/// <param name="dbType">The logical type of the parameter.</param>
		public WhereClauseBuilder Or(string column, string op, object value, DbType dbType)
		{
			if (NeedsOperator())
			{
				_pendingOperator = " OR ";
			}

			AppendCondition(column, op, value, dbType);

			return this;
		}

		/// <summary>
		/// Adds an AND IS NOT NULL condition.
		/// </summary>
		/// <param name="column">The column name.</param>
		public WhereClauseBuilder AndIsNotNull(string column)
		{
			if (NeedsOperator())
			{
				_pendingOperator = " AND ";
			}

			_currentGroup.Append(_pendingOperator);
			_currentGroup.Append($"{column} IS NOT NULL");
			_pendingOperator = String.Empty;

			return this;
		}

		/// <summary>
		/// Adds an AND IS NULL condition.
		/// </summary>
		/// <param name="column">The column name.</param>
		public WhereClauseBuilder AndIsNull(string column)
		{
			if (NeedsOperator())
			{
				_pendingOperator = " AND ";
			}

			_currentGroup.Append(_pendingOperator);
			_currentGroup.Append($"{column} IS NULL");
			_pendingOperator = String.Empty;

			return this;
		}

		/// <summary>
		/// Conditionally adds an AND condition.
		/// </summary>
		/// <param name="condition">Whether to add the condition.</param>
		/// <param name="column">The column name.</param>
		/// <param name="op">The operator.</param>
		/// <param name="value">The parameter value.</param>
		/// <param name="dbType">The logical type of the parameter.</param>
		public WhereClauseBuilder AndIf(bool condition, string column, string op, object value, DbType dbType)
		{
			if (condition)
			{
				And(column, op, value, dbType);
			}

			return this;
		}

		/// <summary>
		/// Conditionally adds an AND condition for non-null/empty strings.
		/// </summary>
		/// <param name="value">The value to check and use.</param>
		/// <param name="column">The column name.</param>
		/// <param name="dbType">The logical type of the parameter.</param>
		public WhereClauseBuilder AndIfNotEmpty(string? value, string column, DbType dbType = DbType.AnsiString)
		{
			if (!String.IsNullOrEmpty(value))
			{
				And(column, "=", value, dbType);
			}

			return this;
		}

		/// <summary>
		/// Conditionally adds an AND condition for non-null values.
		/// </summary>
		/// <typeparam name="T">The value type.</typeparam>
		/// <param name="value">The nullable value to check and use.</param>
		/// <param name="column">The column name.</param>
		/// <param name="op">The operator.</param>
		/// <param name="dbType">The logical type of the parameter.</param>
		public WhereClauseBuilder AndIfNotNull<T>(T? value, string column, string op, DbType dbType) where T : struct
		{
			if (value.HasValue)
			{
				And(column, op, value.Value, dbType);
			}

			return this;
		}

		/// <summary>
		/// Starts a grouped expression with AND (opens parenthesis).
		/// </summary>
		public WhereClauseBuilder BeginGroupAnd()
		{
			FlushCurrentGroup();

			if (_clauses.Count > 0 && _clauses[_clauses.Count - 1] != "(")
			{
				_clauses.Add(" AND ");
			}

			_clauses.Add("(");

			return this;
		}

		/// <summary>
		/// Starts a grouped expression with OR (opens parenthesis).
		/// </summary>
		public WhereClauseBuilder BeginGroupOr()
		{
			FlushCurrentGroup();

			if (_clauses.Count > 0 && _clauses[_clauses.Count - 1] != "(")
			{
				_clauses.Add(" OR ");
			}

			_clauses.Add("(");

			return this;
		}

		/// <summary>
		/// Ends a grouped expression (closes parenthesis).
		/// </summary>
		public WhereClauseBuilder EndGroup()
		{
			FlushCurrentGroup();
			_clauses.Add(")");

			return this;
		}

		/// <summary>
		/// Builds the final WHERE clause result.
		/// </summary>
		/// <returns>A <see cref="WhereClauseResult"/> containing the clause, parameters, and dialect.</returns>
		public WhereClauseResult Build()
		{
			FlushCurrentGroup();

			if (_clauses.Count == 0)
			{
				return new WhereClauseResult(String.Empty, _parameters, _dialect);
			}

			string clause = "WHERE " + String.Join("", _clauses);

			return new WhereClauseResult(clause, new List<ParameterEntry>(_parameters), _dialect);
		}

		/// <summary>
		/// Creates a builder initialized with an existing <see cref="WhereClauseResult"/>. Inherits the existing result's dialect so the merged builder emits matching placeholders.
		/// </summary>
		/// <param name="existing">The existing result to extend.</param>
		/// <returns>A new builder with the existing clause and parameters.</returns>
		public static WhereClauseBuilder From(WhereClauseResult existing)
		{
			WhereClauseBuilder builder = new WhereClauseBuilder(existing.Dialect);

			if (!String.IsNullOrEmpty(existing.Clause))
			{
				// Strip "WHERE " prefix if present
				string clause = existing.Clause.StartsWith("WHERE ", StringComparison.OrdinalIgnoreCase)
						? existing.Clause.Substring(6)
						: existing.Clause;

				builder._clauses.Add(clause);
			}

			if (existing.Parameters != null)
			{
				builder._parameters.AddRange(existing.Parameters);
			}

			return builder;
		}

		#region -- Private Methods --

		private bool NeedsOperator()
		{
			bool clausesNeedOperator = _clauses.Count > 0 && _clauses[_clauses.Count - 1] != "(";
			bool groupNeedsOperator = _currentGroup.Length > 0 && !_currentGroup.ToString().EndsWith("(", StringComparison.InvariantCulture);

			return clausesNeedOperator || groupNeedsOperator;
		}

		private void AppendCondition(string column, string op, object value, DbType dbType)
		{
			string name = $"p{_parameters.Count}";
			string placeholder = _dialect.FormatParameterPlaceholder(name, dbType);

			_currentGroup.Append(_pendingOperator);
			_currentGroup.Append($"{column} {op} {placeholder}");

			_parameters.Add(new ParameterEntry(name, value, dbType));

			_pendingOperator = String.Empty;
		}

		private void FlushCurrentGroup()
		{
			if (_currentGroup.Length > 0)
			{
				_clauses.Add(_currentGroup.ToString());
				_currentGroup.Clear();
			}
		}

		#endregion -- Private Methods --
	}

	/// <summary>
	/// Utility struct representing a where clause and its bound parameters. Apply via <see cref="ApplyTo"/> on any <see cref="DbCommand"/>.
	/// </summary>
	public readonly struct WhereClauseResult
	{
		#region -- Public Api --

		/// <summary>
		/// The string representation of the where clause.
		/// </summary>
		public string Clause { get; }

		/// <summary>
		/// The parameters associated with the where <see cref="Clause"/>.
		/// </summary>
		public IReadOnlyList<ParameterEntry> Parameters { get; }

		/// <summary>
		/// The dialect that produced this result. Reused by <see cref="ApplyTo"/> for parameter binding and by <see cref="WhereClauseBuilder.From"/> when extending.
		/// </summary>
		public ISqlDialect Dialect { get; }

		/// <summary>
		/// Constructor that defaults to the positional <c>?</c> dialect. Used by tests and by call sites that don't have access to a backend's dialect.
		/// </summary>
		/// <param name="clause">The string representation of the where clause.</param>
		/// <param name="parameters">The parameters associated with the where clause.</param>
		public WhereClauseResult(string clause, IReadOnlyList<ParameterEntry> parameters)
			: this(clause, parameters, PositionalQuestionMarkDialect.Instance)
		{
		}

		/// <summary>
		/// Constructor that captures a backend-specific dialect.
		/// </summary>
		/// <param name="clause">The string representation of the where clause.</param>
		/// <param name="parameters">The parameters associated with the where clause.</param>
		/// <param name="dialect">The dialect that produced the clause and that should bind the parameters.</param>
		public WhereClauseResult(string clause, IReadOnlyList<ParameterEntry> parameters, ISqlDialect dialect)
		{
			Clause = clause;
			Parameters = parameters;
			Dialect = dialect;
		}

		/// <summary>
		/// Applies the where clause's parameters to a provided command. Binds via the captured dialect so the concrete <see cref="DbParameter"/> instances match the command (Odbc, ClickHouse, ADBC, etc.).
		/// </summary>
		/// <param name="command">The command to apply the where clause's parameters to.</param>
		public void ApplyTo(DbCommand command)
		{
			if (Parameters == null)
			{
				return;
			}

			ISqlDialect dialect = Dialect ?? PositionalQuestionMarkDialect.Instance;
			foreach (ParameterEntry entry in Parameters)
			{
				dialect.AddParameter(command, entry.Name, entry.Value, entry.DbType);
			}
		}

		#endregion -- Public Api --
	}

	/// <summary>
	/// Filter object for structured analytics queries.
	/// </summary>
	public struct StructuredAnalyticsFilter
	{
		/// <summary>
		/// The maximum record count to return from a query.
		/// </summary>
		public const int MaximumRecordCount = 10000;

		/// <summary>
		/// The default record count to return from a query.
		/// </summary>
		public const int DefaultRecordCount = 100;

		private int _recordCount;

		/// <summary>
		/// The maximum number of records to obtain.
		/// </summary>
		public int RecordCount
		{
			get => _recordCount == 0 ? DefaultRecordCount : _recordCount;
			set => _recordCount = Math.Clamp(value, 0, MaximumRecordCount);
		}

		/// <summary>
		/// The stream IDs to filter by.
		/// </summary>
		public StreamId[]? StreamIds { get; set; }

		/// <summary>
		/// The minimum date to obtain.
		/// </summary>
		public DateTime? MinDate { get; set; }

		/// <summary>
		/// The maximum date to obtain.
		/// </summary>
		public DateTime? MaxDate { get; set; }

		/// <summary>
		/// The minimum schema version to filter by.
		/// </summary>
		public int? MinSchemaVersion { get; set; }

		/// <summary>
		/// Constructs a <see cref="WhereClauseResult"/> based on the AND of every enabled filter parameter.
		/// </summary>
		/// <param name="streamIdColumn">The column name for stream ID filtering.</param>
		/// <param name="dateColumn">The column name for date filtering. When null, defaults to <see cref="EpicGames.Analytics.AbstractTelemetryRecord.TelemetryTimestampColumnName"/> — the wire-format-guaranteed per-record timestamp present on every <see cref="EpicGames.Analytics.AbstractTelemetryRecord"/>-derived row. Pass an explicit domain timestamp (e.g. <c>create_time_utc</c>) to filter on that column instead.</param>
		/// <param name="schemaVersionColumn">The column name for schema version filtering (optional).</param>
		/// <param name="dialect">Backend dialect supplying placeholder syntax. When null, defaults to positional <c>?</c> (ODBC/test behavior).</param>
		/// <returns>A where clause result containing the sql where clause, and the bound <see cref="ParameterEntry"/> list.</returns>
		public WhereClauseResult BuildWhereClause(string streamIdColumn, string? dateColumn = null, string? schemaVersionColumn = null, ISqlDialect? dialect = null)
		{
			string effectiveDateColumn = dateColumn ?? EpicGames.Analytics.AbstractTelemetryRecord.TelemetryTimestampColumnName;
			WhereClauseBuilder builder = new WhereClauseBuilder(dialect ?? PositionalQuestionMarkDialect.Instance);

			// Add stream ID filter
			if (StreamIds != null && StreamIds.Length > 0)
			{
				builder.BeginGroupAnd();

				for (int i = 0; i < StreamIds.Length; ++i)
				{
					builder.Or(streamIdColumn, "=", StreamIds[i].ToString(), DbType.AnsiString);
				}

				builder.EndGroup();
			}

			// Add date filters. effectiveDateColumn is never null (defaults to TelemetryTimestampColumnName when caller passes null).
			builder.AndIfNotNull(MinDate, effectiveDateColumn, ">=", DbType.DateTime);
			builder.AndIfNotNull(MaxDate, effectiveDateColumn, "<=", DbType.DateTime);

			// Add schema version filter
			if (!String.IsNullOrEmpty(schemaVersionColumn))
			{
				builder.AndIfNotNull(MinSchemaVersion, schemaVersionColumn, ">=", DbType.Int32);
			}

			return builder.Build();
		}
	}
}
