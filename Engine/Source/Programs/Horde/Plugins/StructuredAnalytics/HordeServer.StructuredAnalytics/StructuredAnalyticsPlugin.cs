// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.Analytics.Schemas;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the StructuredAnalytics plugin
	/// </summary>
	[Plugin("StructuredAnalytics", EnabledByDefault = false, GlobalConfigType = typeof(StructuredAnalyticsConfig), ServerConfigType = typeof(StructuredAnalyticsServerConfig))]
	public class StructuredAnalyticsPlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public StructuredAnalyticsPlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			serviceCollection.AddSingleton<AnalyticsService>();

			serviceCollection.AddHttpClient();

			// Schema management services
			serviceCollection.AddSingleton<ITelemetrySchemaCollection, TelemetrySchemaCollection>();
			serviceCollection.AddSingleton<TelemetrySchemaService>();
			serviceCollection.AddSingleton<SchemaInferenceService>();

			// Cluster-aware warm cache of schemas. Singleton + IHostedService so consumers (e.g. ClickHouseTelemetrySink) inject ITelemetrySchemaCache and the host owns the refresh ticker lifecycle.
			serviceCollection.AddSingleton<TelemetrySchemaCache>();
			serviceCollection.AddSingleton<ITelemetrySchemaCache>(sp => sp.GetRequiredService<TelemetrySchemaCache>());
			serviceCollection.AddSingleton<IHostedService>(sp => sp.GetRequiredService<TelemetrySchemaCache>());

			serviceCollection.AddHostedService<TelemetrySchemaStartupService>();
		}
	}

	/// <summary>
	/// Helper methods for StructuredAnalytics plugin config
	/// </summary>
	public static class StructuredAnalyticsExtensions
	{
		/// <summary>
		/// Configures the StructuredAnalytics plugin
		/// </summary>
		public static void AddStructuredAnalyticsConfig(this IDictionary<PluginName, IPluginConfig> dictionary, StructuredAnalyticsConfig config)
			=> dictionary[new PluginName("StructuredAnalytics")] = config;

		/// <summary>
		/// Gets configuration for the StructuredAnalytics plugin
		/// </summary>
		public static StructuredAnalyticsConfig GetStructuredAnalyticsConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (StructuredAnalyticsConfig)dictionary[new PluginName("StructuredAnalytics")];
	}
}
