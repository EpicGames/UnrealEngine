// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Server-specific configuration for the StructuredAnalytics plugin
	/// </summary>
	public class StructuredAnalyticsServerConfig : PluginServerConfig
	{
	}
}
