// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Text.Json;
using Dapper;
using EpicGames.Analytics.Generated;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Issues;
using MongoDB.Bson;

namespace HordeServer.Analytics
{
	/// <summary>
	/// Type handler helpers.
	/// </summary>
	public static class TypeHandlers
	{
		/// <summary>
		/// Registers type handlers.
		/// </summary>
		public static void RegisterTypeHandlers()
		{
			SqlMapper.AddTypeHandler(new StreamIdHandler());
			SqlMapper.AddTypeHandler(new NullableStreamIdHandler());
			SqlMapper.AddTypeHandler(new TemplateIdHandler());
			SqlMapper.AddTypeHandler(new JobIdHandler());
			SqlMapper.AddTypeHandler(new JobStepIdHandler());
			SqlMapper.AddTypeHandler(new StringArrayTypeHandler());
			SqlMapper.AddTypeHandler(new ObjectIdHandler());
			SqlMapper.AddTypeHandler(new FingerprintTelemetryHandler());
			SqlMapper.AddTypeHandler(new FailureInfoTelemetryHandler());
			SqlMapper.AddTypeHandler(new NullableFailureInfoTelemetryHandler());
			SqlMapper.AddTypeHandler(new AgentIdHandler());
			SqlMapper.AddTypeHandler(new LeaseIdHandler());
			SqlMapper.AddTypeHandler(new NullableLeaseIdHandler());
			SqlMapper.AddTypeHandler(new SessionIdHandler());
			SqlMapper.AddTypeHandler(new PoolIdHandler());
			SqlMapper.AddTypeHandler(new NullablePoolIdHandler());
			SqlMapper.AddTypeHandler(new LogIdHandler());
			SqlMapper.AddTypeHandler(new NullableLogIdHandler());
			SqlMapper.AddTypeHandler(new LeaseOutcomeHandler());
			SqlMapper.AddTypeHandler(new LeaseOutcomeReasonHandler());
			SqlMapper.AddTypeHandler(new JobStepOutcomeHandler());
			SqlMapper.AddTypeHandler(new NullableJobStepOutcomeHandler());
			SqlMapper.AddTypeHandler(new JobStepStateHandler());
			SqlMapper.AddTypeHandler(new NullableJobStepStateHandler());
			SqlMapper.AddTypeHandler(new UserIdHandler());
			SqlMapper.AddTypeHandler(new NullableUserIdHandler());
			SqlMapper.AddTypeHandler(new CommitIdHandler());
			SqlMapper.AddTypeHandler(new NullableCommitIdHandler());
		}

		/// <summary>
		/// Helper to check if a value is null or empty for parsing purposes.
		/// </summary>
		internal static bool IsNullOrEmpty(object? value)
			=> value == null || value == DBNull.Value || String.IsNullOrEmpty(value.ToString());
	}

	/// <summary>
	/// Type handler for string arrays (JSON serialized).
	/// </summary>
	public class StringArrayTypeHandler : SqlMapper.TypeHandler<string?[]>
	{
		/// <inheritdoc/>
		public override string?[] Parse(object? value)
		{
			if (value is string str)
			{
				return JsonSerializer.Deserialize<string?[]>(str) ?? [];
			}
			return [];
		}

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, string?[]? value)
		{
			parameter.Value = JsonSerializer.Serialize(value);
		}
	}

	#region -- Simple ID Type Handlers --

	/// <summary>
	/// Type handler for <see cref="StreamId"/>.
	/// </summary>
	public class StreamIdHandler : SqlMapper.TypeHandler<StreamId>
	{
		internal static StreamId ParseCore(object value) => new StreamId(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, StreamId value) => parameter.Value = value.Id.ToString();

		/// <inheritdoc/>
		public override StreamId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new StreamId(String.Empty) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="StreamId"/>.
	/// </summary>
	public class NullableStreamIdHandler : SqlMapper.TypeHandler<StreamId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, StreamId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override StreamId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : StreamIdHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="TemplateId"/>.
	/// </summary>
	public class TemplateIdHandler : SqlMapper.TypeHandler<TemplateId>
	{
		internal static TemplateId ParseCore(object value) => new TemplateId(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, TemplateId value) => parameter.Value = value.Id.ToString();

		/// <inheritdoc/>
		public override TemplateId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new TemplateId(String.Empty) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="JobId"/>.
	/// </summary>
	public class JobIdHandler : SqlMapper.TypeHandler<JobId>
	{
		internal static JobId ParseCore(object value) => new JobId(BinaryId.Parse(value.ToString()!));

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobId value) => parameter.Value = value.Id.ToString();

		/// <inheritdoc/>
		public override JobId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? JobId.Empty : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="JobStepId"/>.
	/// </summary>
	public class JobStepIdHandler : SqlMapper.TypeHandler<JobStepId>
	{
		internal static JobStepId ParseCore(object value) => JobStepId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobStepId value) => parameter.Value = value.Id.ToString();

		/// <inheritdoc/>
		public override JobStepId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new JobStepId(0) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="ObjectId"/>.
	/// </summary>
	public class ObjectIdHandler : SqlMapper.TypeHandler<ObjectId>
	{
		internal static ObjectId ParseCore(object value) => ObjectId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, ObjectId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override ObjectId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? ObjectId.Empty : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="AgentId"/>.
	/// </summary>
	public class AgentIdHandler : SqlMapper.TypeHandler<AgentId>
	{
		internal static AgentId ParseCore(object value) => new AgentId(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, AgentId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override AgentId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new AgentId(String.Empty) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="LeaseId"/>.
	/// </summary>
	public class LeaseIdHandler : SqlMapper.TypeHandler<LeaseId>
	{
		internal static LeaseId ParseCore(object value) => LeaseId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LeaseId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override LeaseId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? default : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="LeaseId"/>.
	/// </summary>
	public class NullableLeaseIdHandler : SqlMapper.TypeHandler<LeaseId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LeaseId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override LeaseId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : LeaseIdHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="SessionId"/>.
	/// </summary>
	public class SessionIdHandler : SqlMapper.TypeHandler<SessionId>
	{
		internal static SessionId ParseCore(object value) => SessionId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, SessionId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override SessionId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? SessionId.Empty : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="PoolId"/>.
	/// </summary>
	public class PoolIdHandler : SqlMapper.TypeHandler<PoolId>
	{
		internal static PoolId ParseCore(object value) => new PoolId(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, PoolId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override PoolId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new PoolId(String.Empty) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="PoolId"/>.
	/// </summary>
	public class NullablePoolIdHandler : SqlMapper.TypeHandler<PoolId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, PoolId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override PoolId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : PoolIdHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="LogId"/>.
	/// </summary>
	public class LogIdHandler : SqlMapper.TypeHandler<LogId>
	{
		internal static LogId ParseCore(object value) => LogId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LogId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override LogId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? LogId.Empty : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="LogId"/>.
	/// </summary>
	public class NullableLogIdHandler : SqlMapper.TypeHandler<LogId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LogId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override LogId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : LogIdHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="UserId"/>.
	/// </summary>
	public class UserIdHandler : SqlMapper.TypeHandler<UserId>
	{
		internal static UserId ParseCore(object value) => UserId.Parse(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, UserId value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override UserId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? UserId.Empty : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="UserId"/>.
	/// </summary>
	public class NullableUserIdHandler : SqlMapper.TypeHandler<UserId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, UserId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override UserId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : UserIdHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="CommitId"/>.
	/// </summary>
	public class CommitIdHandler : SqlMapper.TypeHandler<CommitId>
	{
		internal static CommitId ParseCore(object value) => new CommitId(value.ToString()!);

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, CommitId? value)
		{
			parameter.Value = (value == null ? String.Empty : value.ToString());
		}

		/// <inheritdoc/>
		public override CommitId Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? new CommitId(String.Empty) : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="CommitId"/>.
	/// </summary>
	public class NullableCommitIdHandler : SqlMapper.TypeHandler<CommitId?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, CommitId? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override CommitId? Parse(object? value) => TypeHandlers.IsNullOrEmpty(value) ? null : CommitIdHandler.ParseCore(value!);
	}

	#endregion

	#region -- Enum Type Handlers --

	/// <summary>
	/// Type handler for <see cref="LeaseOutcome"/>.
	/// </summary>
	public class LeaseOutcomeHandler : SqlMapper.TypeHandler<LeaseOutcome>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LeaseOutcome value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override LeaseOutcome Parse(object? value)
		{
			if (TypeHandlers.IsNullOrEmpty(value))
			{
				return LeaseOutcome.Unspecified;
			}

			return Enum.TryParse<LeaseOutcome>(value!.ToString(), ignoreCase: true, out LeaseOutcome result) ? result : LeaseOutcome.Success;
		}
	}

	/// <summary>
	/// Type handler for <see cref="LeaseOutcomeReason"/>.
	/// </summary>
	public class LeaseOutcomeReasonHandler : SqlMapper.TypeHandler<LeaseOutcomeReason>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, LeaseOutcomeReason value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override LeaseOutcomeReason Parse(object? value)
		{
			if (TypeHandlers.IsNullOrEmpty(value))
			{
				return LeaseOutcomeReason.None;
			}
			return Enum.TryParse<LeaseOutcomeReason>(value!.ToString(), ignoreCase: true, out LeaseOutcomeReason result) ? result : LeaseOutcomeReason.None;
		}
	}

	/// <summary>
	/// Type handler for <see cref="JobStepOutcome"/>.
	/// </summary>
	public class JobStepOutcomeHandler : SqlMapper.TypeHandler<JobStepOutcome>
	{
		internal static JobStepOutcome ParseCore(object value)
			=> Enum.TryParse<JobStepOutcome>(value.ToString(), ignoreCase: true, out JobStepOutcome result) ? result : JobStepOutcome.Unspecified;

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobStepOutcome value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override JobStepOutcome Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? JobStepOutcome.Unspecified : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="JobStepOutcome"/>.
	/// </summary>
	public class NullableJobStepOutcomeHandler : SqlMapper.TypeHandler<JobStepOutcome?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobStepOutcome? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override JobStepOutcome? Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? null : JobStepOutcomeHandler.ParseCore(value!);
	}

	/// <summary>
	/// Type handler for <see cref="JobStepState"/>.
	/// </summary>
	public class JobStepStateHandler : SqlMapper.TypeHandler<JobStepState>
	{
		internal static JobStepState ParseCore(object value)
			=> Enum.TryParse<JobStepState>(value.ToString(), ignoreCase: true, out JobStepState result) ? result : JobStepState.Unspecified;

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobStepState value) => parameter.Value = value.ToString();

		/// <inheritdoc/>
		public override JobStepState Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? JobStepState.Unspecified : ParseCore(value!);
	}

	/// <summary>
	/// Type handler for nullable <see cref="JobStepState"/>.
	/// </summary>
	public class NullableJobStepStateHandler : SqlMapper.TypeHandler<JobStepState?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, JobStepState? value) => parameter.Value = value?.ToString() ?? (object)DBNull.Value;

		/// <inheritdoc/>
		public override JobStepState? Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? null : JobStepStateHandler.ParseCore(value!);
	}

	#endregion

	#region -- Complex Type Handlers --

	/// <summary>
	/// Type handler for <see cref="FingerprintTelemetry"/>.
	/// </summary>
	public class FingerprintTelemetryHandler : SqlMapper.TypeHandler<FingerprintTelemetry>
	{
		private static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions { PropertyNameCaseInsensitive = true };

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, FingerprintTelemetry? value)
		{
			parameter.Value = JsonSerializer.Serialize(value);
		}

		internal static FingerprintTelemetry ParseCore(object value)
		{
			try
			{
				string json = value.ToString()!;
				FingerprintDto? dto = JsonSerializer.Deserialize<FingerprintDto>(json, s_jsonOptions);
				if (dto == null)
				{
					return new FingerprintTelemetry(String.Empty, new HashSet<IssueKey>());
				}

				HashSet<IssueKey> keys = new HashSet<IssueKey>();
				if (dto.Keys != null)
				{
					foreach (IssueKeyDto keyDto in dto.Keys)
					{
						IssueKeyType keyType = IssueKeyType.None;
						if (!String.IsNullOrEmpty(keyDto.Type))
						{
							Enum.TryParse(keyDto.Type, ignoreCase: true, out keyType);
						}
						keys.Add(new IssueKey(keyDto.Name ?? String.Empty, keyType, keyDto.Scope));
					}
				}

				return new FingerprintTelemetry(dto.Type ?? String.Empty, keys);
			}
			catch
			{
				return new FingerprintTelemetry(String.Empty, new HashSet<IssueKey>());
			}
		}

		/// <inheritdoc/>
		public override FingerprintTelemetry Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? new FingerprintTelemetry(String.Empty, new HashSet<IssueKey>()) : ParseCore(value!);

		private class FingerprintDto
		{
			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FingerprintTelemetryGen.Type)]
			public string? Type { get; set; }

			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FingerprintTelemetryGen.Keys)]
			public List<IssueKeyDto>? Keys { get; set; }
		}

		private class IssueKeyDto
		{
			public string? Name { get; set; }
			public string? Type { get; set; }
			public string? Scope { get; set; }
		}
	}

	/// <summary>
	/// Type handler for <see cref="FailureInfoTelemetry"/>.
	/// </summary>
	public class FailureInfoTelemetryHandler : SqlMapper.TypeHandler<FailureInfoTelemetry>
	{
		private static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions { PropertyNameCaseInsensitive = true };

		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, FailureInfoTelemetry? value)
		{
			parameter.Value = JsonSerializer.Serialize(value ?? new FailureInfoTelemetry(JobId.Empty, String.Empty, new CommitIdWithOrder(String.Empty, 0), ObjectId.Empty));
		}

		internal static FailureInfoTelemetry ParseCore(object value)
		{
			try
			{
				string json = value.ToString()!;
				FailureInfoDto? dto = JsonSerializer.Deserialize<FailureInfoDto>(json, s_jsonOptions);
				if (dto == null)
				{
					return new FailureInfoTelemetry(JobId.Empty, String.Empty, new CommitIdWithOrder(String.Empty, 0), ObjectId.Empty);
				}

				// Parse JobId
				JobId jobId = JobId.Empty;
				if (!String.IsNullOrEmpty(dto.JobId))
				{
					jobId = new JobId(BinaryId.Parse(dto.JobId));
				}

				// Parse Change (CommitIdWithOrder)
				CommitIdWithOrder change = new CommitIdWithOrder(String.Empty, 0);
				if (dto.Change != null)
				{
					change = new CommitIdWithOrder(dto.Change.Name ?? String.Empty, dto.Change.Order);
				}

				// Parse StepId (ObjectId)
				ObjectId stepId = ObjectId.Empty;
				if (!String.IsNullOrEmpty(dto.StepId))
				{
					stepId = ObjectId.Parse(dto.StepId);
				}

				return new FailureInfoTelemetry(jobId, dto.JobName ?? String.Empty, change, stepId);
			}
			catch
			{
				return new FailureInfoTelemetry(JobId.Empty, String.Empty, new CommitIdWithOrder(String.Empty, 0), ObjectId.Empty);
			}
		}

		/// <inheritdoc/>
		public override FailureInfoTelemetry Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? new FailureInfoTelemetry(JobId.Empty, String.Empty, new CommitIdWithOrder(String.Empty, 0), ObjectId.Empty) : ParseCore(value!);

		private class FailureInfoDto
		{
			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FailureInfoTelemetryGen.JobId)]
			public string? JobId { get; set; }

			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FailureInfoTelemetryGen.JobName)]
			public string? JobName { get; set; }

			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FailureInfoTelemetryGen.Change)]
			public CommitIdWithOrderDto? Change { get; set; }

			[System.Text.Json.Serialization.JsonPropertyName(HordeServer_Issues_FailureInfoTelemetryGen.StepId)]
			public string? StepId { get; set; }
		}

		private class CommitIdWithOrderDto
		{
			[System.Text.Json.Serialization.JsonPropertyName("name")]
			public string? Name { get; set; }

			[System.Text.Json.Serialization.JsonPropertyName("order")]
			public int Order { get; set; }
		}
	}

	/// <summary>
	/// Type handler for nullable <see cref="FailureInfoTelemetry"/>.
	/// </summary>
	public class NullableFailureInfoTelemetryHandler : SqlMapper.TypeHandler<FailureInfoTelemetry?>
	{
		/// <inheritdoc/>
		public override void SetValue(IDbDataParameter parameter, FailureInfoTelemetry? value)
			=> parameter.Value = value != null ? JsonSerializer.Serialize(value) : DBNull.Value;

		/// <inheritdoc/>
		public override FailureInfoTelemetry? Parse(object? value)
			=> TypeHandlers.IsNullOrEmpty(value) ? null : FailureInfoTelemetryHandler.ParseCore(value!);
	}

	#endregion
}
