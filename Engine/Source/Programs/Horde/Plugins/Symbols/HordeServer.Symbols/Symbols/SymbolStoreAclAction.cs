// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Symbols
{
	/// <summary>
	/// ACL actions which apply to symbols
	/// </summary>
	public static class SymbolStoreAclAction
	{
		/// <summary>
		/// Ability to download symbols
		/// </summary>
		public static AclAction ReadSymbols { get; } = new AclAction("ReadSymbols");
	}
}
