// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Symbols;
using HordeServer.Acls;

namespace HordeServer.Symbols
{
	/// <summary>
	/// Configuration for a symbol store
	/// </summary>
	public class SymbolStoreConfig
	{
		/// <summary>
		/// Identifier for this store
		/// </summary>
		public SymbolStoreId Id { get; set; }

		/// <summary>
		/// Configuration for the symbol store backend
		/// </summary>
		public NamespaceId NamespaceId { get; set; }

		/// <summary>
		/// Whether to make this store available without auth
		/// </summary>
		public bool Public { get; set; }

		/// <summary>
		/// Access to the symbol store
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Authorize a user to perform the given action
		/// </summary>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
		{
			if (Public)
			{
				return true;
			}
			else if (user.Identity == null || !user.Identity.IsAuthenticated)
			{
				return false;
			}
			else
			{
				return Acl.Authorize(action, user);
			}
		}
	}
}
