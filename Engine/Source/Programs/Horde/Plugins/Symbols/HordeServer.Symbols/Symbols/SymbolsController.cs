// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Symbols;
using HordeServer.Storage;
using HordeServer.Symbols;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Tools
{
	/// <summary>
	/// Controller for the /api/v1/symbols endpoint
	/// </summary>
	[ApiController]
	[TryAuthorize]
	public class SymbolsController : HordeControllerBase
	{
		readonly StorageService _storageService;
		readonly IOptionsSnapshot<SymbolsConfig> _symbolsConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public SymbolsController(StorageService storageService, IOptionsSnapshot<SymbolsConfig> symbolsConfig)
		{
			_storageService = storageService;
			_symbolsConfig = symbolsConfig;
		}

		/// <summary>
		/// Reads a file from the symbol store. 
		/// </summary>
		/// <param name="storeId">Id of the symbol store</param>
		/// <param name="path">Path for the file to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Route("/api/v1/symbols/{storeId}/{*path}")]
		public async Task<ActionResult> ReadFileAsync(SymbolStoreId storeId, string path, CancellationToken cancellationToken)
		{
			SymbolStoreConfig? symbolStoreConfig;
			if (!_symbolsConfig.Value.TryGetStore(storeId, out symbolStoreConfig))
			{
				return NotFound(storeId);
			}
			if (!symbolStoreConfig.Authorize(SymbolStoreAclAction.ReadSymbols, User))
			{
				return Forbid(SymbolStoreAclAction.ReadSymbols, storeId);
			}

			IStorageNamespace storageNamespace = _storageService.GetNamespace(symbolStoreConfig.NamespaceId);
			BlobAlias? alias = await storageNamespace.FindAliasAsync($"sym:{path.ToUpperInvariant()}", cancellationToken);
			if (alias == null)
			{
				return NotFound();
			}

			IBlobRef<ChunkedDataNode> blobRef = alias.Target.ForType<ChunkedDataNode>();

			Response.StatusCode = (int)HttpStatusCode.OK;

			await Response.StartAsync(cancellationToken);
			await blobRef.CopyToStreamAsync(Response.Body, cancellationToken);
			await Response.CompleteAsync();

			return NoContent();
		}
	}
}
