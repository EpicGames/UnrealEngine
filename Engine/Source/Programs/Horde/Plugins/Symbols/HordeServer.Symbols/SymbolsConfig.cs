// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Symbols;
using HordeServer.Acls;
using HordeServer.Plugins;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Symbols
{
	/// <summary>
	/// Configuration for the tools system
	/// </summary>
	public class SymbolsConfig : IPluginConfig
	{
		/// <summary>
		/// List of symbol stores
		/// </summary>
		public List<SymbolStoreConfig> Stores { get; set; } = new List<SymbolStoreConfig>();

		readonly Dictionary<SymbolStoreId, SymbolStoreConfig> _storeLookup = new Dictionary<SymbolStoreId, SymbolStoreConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			foreach (SymbolStoreConfig store in Stores)
			{
				store.Acl.PostLoad(configOptions.ParentAcl, $"symbol-store:{store.Id}", AclConfig.GetActions([typeof(SymbolStoreAclAction)]));
			}

			_storeLookup.Clear();
			foreach (SymbolStoreConfig store in Stores)
			{
				_storeLookup.Add(store.Id, store);
			}
		}

		/// <summary>
		/// Finds a store by id
		/// </summary>
		public bool TryGetStore(SymbolStoreId storeId, [NotNullWhen(true)] out SymbolStoreConfig? store)
			=> _storeLookup.TryGetValue(storeId, out store);
	}
}
