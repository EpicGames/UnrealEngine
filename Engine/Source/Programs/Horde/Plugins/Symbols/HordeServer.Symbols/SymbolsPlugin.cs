// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;
using HordeServer.Symbols;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the symbols plugin
	/// </summary>
	[Plugin("Symbols", GlobalConfigType = typeof(SymbolsConfig))]
	public class SymbolsPlugin : IPluginStartup
	{
		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{ }

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{ }
	}
}
