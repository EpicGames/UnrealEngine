// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO.Compression;
using System.Text;
using EpicGames.Horde.Tools;
using HordeCommon;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Tools;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Tools
{
	[TestClass]
	public class ToolTests : ServerTestSetup
	{
		readonly ToolId _toolId = new ToolId("ugs");

		[TestInitialize]
		public async Task AddPluginsAsync()
		{
			AddPlugin<StoragePlugin>();
			AddPlugin<ToolsPlugin>();

			StorageConfig storageConfig = new StorageConfig();
			storageConfig.Backends.Clear();
			storageConfig.Backends.Add(new BackendConfig { Id = new BackendId("tools-backend"), Type = StorageBackendType.Memory });
			storageConfig.Namespaces.Clear();
			storageConfig.Namespaces.Add(new NamespaceConfig { Id = ToolConfig.DefaultNamespaceId, Backend = new BackendId("tools-backend") });

			ToolsConfig toolsConfig = new ToolsConfig();
			toolsConfig.Tools.Add(new ToolConfig(_toolId) { Name = "UnrealGameSync", Description = "Tool for syncing content from source control" });

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddStorageConfig(storageConfig);
			globalConfig.Plugins.AddToolsConfig(toolsConfig);
			await SetConfigAsync(globalConfig);
		}

		[TestMethod]
		public async Task AddToolAsync()
		{
			IToolCollection collection = ServiceProvider.GetRequiredService<IToolCollection>();

			ITool tool = Deref(await collection.GetAsync(_toolId));
			Assert.AreEqual(tool.Id, new ToolId("ugs"));
			Assert.AreEqual("UnrealGameSync", tool.Name);
			Assert.AreEqual("Tool for syncing content from source control", tool.Description);

			ITool tool2 = Deref(await collection.GetAsync(tool.Id));
			Assert.AreEqual(tool.Id, tool2.Id);
			Assert.AreEqual(tool.Name, tool2.Name);
			Assert.AreEqual(tool.Description, tool2.Description);
		}

		[TestMethod]
		public async Task AddDeploymentAsync()
		{
			IToolCollection collection = ServiceProvider.GetRequiredService<IToolCollection>();

			ITool tool = Deref(await collection.GetAsync(_toolId));
			Assert.AreEqual(new ToolId("ugs"), tool.Id);
			Assert.AreEqual("UnrealGameSync", tool.Name);
			Assert.AreEqual("Tool for syncing content from source control", tool.Description);
			Assert.AreEqual(0, tool.Deployments.Count);

			const string FileName = "test.txt";
			byte[] fileData = Encoding.UTF8.GetBytes("hello world");

			byte[] zipData;
			using (MemoryStream stream = new MemoryStream())
			{
				using (ZipArchive archive = new ZipArchive(stream, ZipArchiveMode.Create))
				{
					ZipArchiveEntry entry = archive.CreateEntry(FileName);
					await using (Stream entryStream = await entry.OpenAsync())
					{
						await entryStream.WriteAsync(fileData);
					}
				}
				zipData = stream.ToArray();
			}

			ToolDeploymentId deploymentId;
			using (MemoryStream stream = new MemoryStream(zipData))
			{
				tool = Deref(await tool.CreateDeploymentAsync(new ToolDeploymentConfig { Version = "1.0", Duration = TimeSpan.FromMinutes(5.0), CreatePaused = true }, stream, CancellationToken.None));
				Assert.AreEqual(1, tool.Deployments.Count);
				Assert.IsNull(tool.Deployments[0].StartedAt);
				deploymentId = tool.Deployments[^1].Id;
			}

			// Check that the deployment doesn't do anything until started
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			await clock.AdvanceAsync(TimeSpan.FromHours(1.0));

			tool = Deref(await collection.GetAsync(tool.Id));
			Assert.AreEqual(1, tool.Deployments.Count);
			Assert.IsNull(tool.Deployments[0].StartedAt);

			// Start the deployment
			IToolDeployment deployment = tool.Deployments[0];
			deployment = Deref(await deployment.UpdateAsync(ToolDeploymentState.Active));
			tool = Deref(await collection.GetAsync(tool.Id));
			Assert.AreEqual(1, tool.Deployments.Count);
			Assert.IsNotNull(tool.Deployments[0].StartedAt);
			Assert.IsTrue(Math.Abs((tool.Deployments[0].StartedAt!.Value - clock.UtcNow).TotalSeconds) < 1.0);

			// Check it updates
			await clock.AdvanceAsync(TimeSpan.FromMinutes(2.5));
			deployment = Deref(await deployment.UpdateAsync(ToolDeploymentState.Paused));
			tool = Deref(await collection.GetAsync(tool.Id));
			Assert.AreEqual(1, tool.Deployments.Count);
			Assert.IsNull(deployment.StartedAt);
			Assert.IsTrue((deployment.Progress - 0.5) < 0.1);

			// Check it stays paused
			await clock.AdvanceAsync(TimeSpan.FromHours(1.0));
			tool = Deref(await collection.GetAsync(tool.Id));
			Assert.AreEqual(1, tool.Deployments.Count);
			Assert.IsNull(tool.Deployments[0].StartedAt);
			Assert.IsTrue((tool.Deployments[0].Progress - 0.5) < 0.1);

			// Get the deployment data
			using Stream dataStream = await tool.Deployments[0].OpenZipStreamAsync(CancellationToken.None);
			using (ZipArchive archive = new ZipArchive(dataStream, ZipArchiveMode.Read))
			{
				ZipArchiveEntry entry = archive.Entries.First();
				Assert.AreEqual(FileName, entry.Name);

				byte[] outputFileData;
				using (MemoryStream stream = new MemoryStream())
				{
					await using Stream entryStream = await entry.OpenAsync();
					await entryStream.CopyToAsync(stream);
					outputFileData = stream.ToArray();
				}

				Assert.IsTrue(fileData.AsSpan().SequenceEqual(outputFileData.AsSpan()));
			}
		}

		[TestMethod]
		public async Task MaxDeploymentsExceededPurgesOldestAsync()
		{
			// Reconfigure the tool with a low MaxDeployments limit
			const int MaxDeployments = 3;

			StorageConfig storageConfig = new StorageConfig();
			storageConfig.Backends.Clear();
			storageConfig.Backends.Add(new BackendConfig { Id = new BackendId("tools-backend"), Type = StorageBackendType.Memory });
			storageConfig.Namespaces.Clear();
			storageConfig.Namespaces.Add(new NamespaceConfig { Id = ToolConfig.DefaultNamespaceId, Backend = new BackendId("tools-backend") });

			ToolsConfig toolsConfig = new ToolsConfig();
			toolsConfig.Tools.Add(new ToolConfig(_toolId) { Name = "UnrealGameSync", Description = "Tool for syncing content from source control", MaxDeployments = MaxDeployments });

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddStorageConfig(storageConfig);
			globalConfig.Plugins.AddToolsConfig(toolsConfig);
			await SetConfigAsync(globalConfig);

			IToolCollection collection = ServiceProvider.GetRequiredService<IToolCollection>();
			ITool tool = Deref(await collection.GetAsync(_toolId));

			// Create more deployments than the limit
			List<string> versions = new List<string>();
			for (int i = 0; i < MaxDeployments + 2; i++)
			{
				string version = $"{i + 1}.0";
				versions.Add(version);

				byte[] zipData = await CreateZipFileDataAsync($"file-{version}.txt", $"content-{version}");
				using MemoryStream stream = new MemoryStream(zipData);
				tool = Deref(await tool.CreateDeploymentAsync(new ToolDeploymentConfig { Version = version }, stream, CancellationToken.None));
			}

			// Verify only MaxDeployments remain and they are the most recent ones
			tool = Deref(await collection.GetAsync(_toolId));
			Assert.AreEqual(MaxDeployments, tool.Deployments.Count);

			List<string> expectedVersions = versions.Skip(versions.Count - MaxDeployments).ToList();
			for (int i = 0; i < MaxDeployments; i++)
			{
				Assert.AreEqual(expectedVersions[i], tool.Deployments[i].Version);
			}
		}

		public static async Task<byte[]> CreateZipFileDataAsync(string fileName, string fileData)
		{
			using MemoryStream stream = new();
			using (ZipArchive archive = new(stream, ZipArchiveMode.Create))
			{
				ZipArchiveEntry entry = archive.CreateEntry(fileName);
				await using (Stream entryStream = await entry.OpenAsync())
				{
					await entryStream.WriteAsync(Encoding.UTF8.GetBytes(fileData));
				}
			}
			return stream.ToArray();
		}
	}
}