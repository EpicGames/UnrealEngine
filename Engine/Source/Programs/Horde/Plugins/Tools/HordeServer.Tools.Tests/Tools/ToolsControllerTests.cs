// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Http.Headers;
using EpicGames.Horde.Server;
using EpicGames.Horde.Tools;
using HordeServer.Acls;
using HordeServer.Configuration;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.ServiceAccounts;
using HordeServer.Storage;
using HordeServer.Tools;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Tools;

[TestClass]
public class ToolsControllerTests
{
	[TestMethod]
	public async Task DownloadToolWithAclAsync()
	{
		AclEntryConfig aclEntryConfig = new(HordeClaims.AgentDedicatedRoleClaim, [ToolAclAction.DownloadTool]);
		ToolId toolId = new("foo");

		PluginCollection pluginCollection = new PluginCollection();
		pluginCollection.Add<AnalyticsPlugin>();
		pluginCollection.Add<BuildPlugin>();
		pluginCollection.Add<ComputePlugin>();
		pluginCollection.Add<StoragePlugin>();

		StorageConfig storageConfig = new StorageConfig();
		storageConfig.Backends.Clear();
		storageConfig.Backends.Add(new BackendConfig { Id = new BackendId("tools-backend"), Type = StorageBackendType.Memory });
		storageConfig.Namespaces.Clear();
		storageConfig.Namespaces.Add(new NamespaceConfig { Id = ToolConfig.DefaultNamespaceId, Backend = new BackendId("tools-backend") });

		ToolsConfig toolsConfig = new ToolsConfig();
		toolsConfig.Tools.Add(new ToolConfig(toolId) { Name = "Foo", Description = "This is foo", Acl = new AclConfig() { Entries = [aclEntryConfig] }, Public = false });

		GlobalConfig globalConfig = new();
		globalConfig.Plugins.AddStorageConfig(storageConfig);
		globalConfig.Plugins.AddToolsConfig(toolsConfig);

		ServerSettings serverSettings = new() { AuthMethod = AuthMethod.Horde };
		globalConfig.PostLoad(serverSettings, pluginCollection.LoadedPlugins, Array.Empty<IDefaultAclModifier>());

		Dictionary<string, string> settings = new() { { "Horde:AuthMethod", AuthMethod.Horde.ToString() } };
		await using FakeHordeWebApp app = new(settings: settings);

		ConfigService configService = app.ServiceProvider.GetRequiredService<ConfigService>();
		IToolCollection tools = app.ServiceProvider.GetRequiredService<IToolCollection>();
		IServiceAccountCollection serviceAccounts = app.ServiceProvider.GetRequiredService<IServiceAccountCollection>();

		configService.OverrideConfig(globalConfig);

		List<IUserClaim> claims = [new UserClaim("http://epicgames.com/ue/horde/role", "agent")];
		(IServiceAccount _, string token) = await serviceAccounts.CreateAsync(new CreateServiceAccountOptions("myName", "myDesc", claims));

		// Create tool and deployment
		using MemoryStream ms = new(await ToolTests.CreateZipFileDataAsync("foo.txt", "foo content"));
		ITool? tool = await tools.GetAsync(toolId);
		Assert.IsNotNull(tool);
		tool = await tool.CreateDeploymentAsync(new ToolDeploymentConfig() { Version = "1" }, ms);

		HttpClient client = app.CreateHttpClient();
		using HttpRequestMessage req = new(HttpMethod.Get, $"/api/v1/tools/{toolId.Id}?action=download");
		req.Headers.Authorization = new AuthenticationHeaderValue("ServiceAccount", token);
		HttpResponseMessage res = await client.SendAsync(req);
		Assert.AreEqual(HttpStatusCode.OK, res.StatusCode);
	}
}
