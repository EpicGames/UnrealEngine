// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Text.Json.Nodes;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using HordeServer.Tools;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer.Commands.Generate
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	[Command("generate", "tooldata", "Creates bundled tool data, and registers it with the server config file")]
	class ToolDataCommand : Command
	{
		[CommandLine(Required = true)]
		[Description("Identifier for the tool")]
		public string Id { get; set; } = String.Empty;

		[CommandLine]
		[Description("Name of the tool")]
		public string? Name { get; set; }

		[CommandLine]
		[Description("Description for the tool")]
		public string? Description { get; set; }

		[CommandLine]
		[Description("Category for the tool")]
		public string? Category { get; set; }

		[CommandLine]
		[Description("Group for the tool")]
		public string? Group { get; set; }

		[CommandLine(ListSeparator = ';')]
		[Description("Semicolon delimited list of tool platforms")]
		public List<string> Platforms { get; set; } = new List<string>();

		[CommandLine]
		[Description("Version string for the tool")]
		public string? Version { get; set; }

		[CommandLine]
		[Description("If true, the tool may be downloaded by any user without authentication")]
		public bool Public { get; set; }

		[CommandLine]
		[Description("Shows the tool for download in UGS")]
		public bool ShowInUgs { get; set; }

		[CommandLine]
		[Description("Shows the tool for download in the Unreal Toolbox")]
		public bool ShowInToolbox { get; set; }

		[CommandLine("-ShowInDashboard=")]
		[Description("Shows the tool for download on the dashboard")]
		public bool ShowInDashboard { get; set; } = true;

		[CommandLine(ListSeparator = ';')]
		[Description("Semicolon delimited list of key=value metadata pairs")]
		public List<string> Metadata { get; set; } = new List<string>();

		[CommandLine(Required = true)]
		[Description("Source directory for tool data")]
		public DirectoryReference InputDir { get; set; } = null!;

		[CommandLine]
		[Description("Directory containing the server to modify")]
		public DirectoryReference ServerDir { get; set; }

		public ToolDataCommand(IServerInfo serverInfo)
		{
			ServerDir = serverInfo.AppDir;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			// Create the local agent bundle
			DirectoryReference bundleDir = DirectoryReference.Combine(ServerDir, "Tools");
			DirectoryReference.CreateDirectory(bundleDir);

			RefName refName = new RefName(Id);
			await using (BundleCache bundleCache = new BundleCache())
			using (MemoryMappedFileCache memoryMappedFileCache = new MemoryMappedFileCache())
			{
				IStorageNamespace client = BundleStorageNamespace.CreateFromDirectory(bundleDir, bundleCache, memoryMappedFileCache, logger);

				IHashedBlobRef<DirectoryNode> dirNodeRef;
				await using (DedupeBlobWriter writer = client.CreateDedupeBlobWriter(refName))
				{
					logger.LogInformation("Populating cache with existing refs...");
					await PopulateCacheAsync(client, writer, bundleDir, CancellationToken.None);

					logger.LogInformation("");
					logger.LogInformation("Writing tool data for {ToolId}", refName);
					DirectoryNode dirNode = new DirectoryNode();
					await dirNode.AddFilesAsync(InputDir.ToDirectoryInfo(), writer);
					dirNodeRef = await writer.WriteBlobAsync(dirNode);

					logger.LogInformation("");
					writer.GetStats().Print(logger);

					logger.LogInformation("");
				}
				await client.AddRefAsync(refName, dirNodeRef);
			}

			// Update the server config to include the bundled tool
			FileReference serverConfigFile = FileReference.Combine(ServerDir, "appsettings.json");
			{
				JsonConfigFile serverConfig = await JsonConfigFile.ReadAsync(serverConfigFile);

				JsonObject hordeConfig = JsonConfigFile.FindOrAddNode(serverConfig.Root, "Horde", () => new JsonObject());
				JsonObject pluginsConfig = JsonConfigFile.FindOrAddNode(hordeConfig, "Plugins", () => new JsonObject());
				JsonObject toolsConfig = JsonConfigFile.FindOrAddNode(pluginsConfig, "Tools", () => new JsonObject());
				JsonArray bundledTools = JsonConfigFile.FindOrAddNode(toolsConfig, nameof(ToolsServerConfig.BundledTools), () => new JsonArray());

				JsonObject bundledTool = JsonConfigFile.FindOrAddElementByKey(bundledTools, nameof(BundledToolConfig.Id), Id);
				bundledTool[nameof(BundledToolConfig.Name)] = Name ?? Id.ToString();
				if (!String.IsNullOrEmpty(Description))
				{
					bundledTool[nameof(BundledToolConfig.Description)] = Description;
				}
				if (!String.IsNullOrEmpty(Category))
				{
					bundledTool[nameof(BundledToolConfig.Category)] = Category;
				}
				if (!String.IsNullOrEmpty(Group))
				{
					bundledTool[nameof(BundledToolConfig.Group)] = Group;
				}
				if (!String.IsNullOrEmpty(Version))
				{
					bundledTool[nameof(BundledToolConfig.Version)] = Version;
				}

				if (Platforms.Count > 0)
				{
					JsonArray platforms = new JsonArray();
					foreach (string platform in Platforms) 
					{
						platforms.Add(platform);
					}
					bundledTool[nameof(BundledToolConfig.Platforms)] = platforms;
				}

				if (Metadata.Count > 0)
				{
					JsonObject metadata = new JsonObject();
					foreach (string metadataItem in Metadata)
					{
						int equalsIdx = metadataItem.IndexOf(':', StringComparison.Ordinal);
						if (equalsIdx == -1)
						{
							throw new InvalidOperationException($"Missing colon symbol in {metadataItem}");
						}
						metadata.Add(metadataItem.Substring(0, equalsIdx), metadataItem.Substring(equalsIdx + 1));
					}
					bundledTool[nameof(BundledToolConfig.Metadata)] = metadata;
				}

				bundledTool[nameof(BundledToolConfig.RefName)] = refName.ToString();

				if (Public)
				{
					bundledTool[nameof(BundledToolConfig.Public)] = true;
				}
				if (ShowInToolbox)
				{
					bundledTool[nameof(BundledToolConfig.ShowInToolbox)] = true;
				}
				if (ShowInUgs)
				{
					bundledTool[nameof(BundledToolConfig.ShowInUgs)] = true;
				}

				logger.LogInformation("Updating {File}", serverConfigFile);
				await serverConfig.WriteAsync(serverConfigFile);
			}

			return 0;
		}

		static async Task PopulateCacheAsync(IStorageNamespace client, DedupeBlobWriter writer, DirectoryReference searchDir, CancellationToken cancellationToken)
		{
			foreach (RefName refName in FileStorageBackend.EnumerateRefs(searchDir))
			{
				IHashedBlobRef? blobRef = await client.TryReadRefAsync(refName, cancellationToken: cancellationToken);
				if (blobRef != null)
				{
					using BlobData blobData = await blobRef.ReadBlobDataAsync(cancellationToken);
					if (blobData.Type.Guid == DirectoryNode.BlobTypeGuid)
					{
						IHashedBlobRef<DirectoryNode> directoryRef = HashedBlobRef.Create<DirectoryNode>(blobRef);
						await writer.AddToCacheAsync(directoryRef, cancellationToken);
					}
				}
			}
		}
	}
}
