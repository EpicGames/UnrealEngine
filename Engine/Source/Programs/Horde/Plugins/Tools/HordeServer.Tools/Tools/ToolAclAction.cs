// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Tools
{
	/// <summary>
	/// ACL actions which apply to tools
	/// </summary>
	public static class ToolAclAction
	{
		/// <summary>
		/// Ability to download a tool
		/// </summary>
		public static AclAction DownloadTool { get; } = new AclAction("DownloadTool");

		/// <summary>
		/// Ability to upload new tool versions
		/// </summary>
		public static AclAction UploadTool { get; } = new AclAction("UploadTool");
	}
}
