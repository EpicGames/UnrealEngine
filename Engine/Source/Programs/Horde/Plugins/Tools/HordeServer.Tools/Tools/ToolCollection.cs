// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers.Binary;
using System.Security.Claims;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Storage.ObjectStores;
using EpicGames.Horde.Tools;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Tools
{
	/// <summary>
	/// Collection of tool documents
	/// </summary>
	public class ToolCollection : IToolCollection
	{
		class ToolDocument
		{
			public ToolId Id { get; set; }

			[BsonElement("dep")]
			public List<ToolDeploymentDocument> Deployments { get; set; } = new List<ToolDeploymentDocument>();

			// Last time that the document was updated. This field is checked and updated as part of updates to ensure atomicity.
			[BsonElement("_u")]
			public DateTime LastUpdateTime { get; set; }

			[BsonConstructor]
			public ToolDocument(ToolId id)
			{
				Id = id;
			}
		}

		class ToolDeploymentDocument
		{
			public ToolDeploymentId Id { get; set; }

			[BsonElement("ver")]
			public string Version { get; set; }

			[BsonElement("bpr")]
			public double BaseProgress { get; set; }

			[BsonElement("stm")]
			public DateTime? StartedAt { get; set; }

			[BsonElement("dur")]
			public TimeSpan Duration { get; set; }

			[BsonElement("ns")]
			public NamespaceId NamespaceId { get; set; } = ToolConfig.DefaultNamespaceId;

			[BsonElement("ref")]
			public RefName RefName { get; set; }

			[BsonConstructor]
			public ToolDeploymentDocument(ToolDeploymentId id)
			{
				Id = id;
				Version = String.Empty;
			}

			public ToolDeploymentDocument(ToolDeploymentId id, ToolDeploymentConfig options, NamespaceId namespaceId, RefName refName)
			{
				Id = id;
				Version = options.Version;
				Duration = options.Duration;
				NamespaceId = namespaceId;
				RefName = refName;
			}
		}

		class Tool : ITool
		{
			readonly ToolCollection _collection;
			readonly ToolDocument _document;
			readonly ToolConfig _config;
			readonly List<ToolDeployment> _deployments;

			public ToolConfig Config => _config;
			public ToolDocument Document => _document;

			public ToolId Id => _config.Id;
			public string Name => _config.Name;
			public string Description => _config.Description;
			public string? Category => _config.Category;
			public string? Group => _config.Group;
			public IReadOnlyList<string>? Platforms => _config.Platforms;
			public bool Public => _config.Public;
			public bool Bundled => _config is BundledToolConfig;
			public bool ShowInUgs => _config.ShowInUgs;
			public bool ShowInDashboard => _config.ShowInDashboard;
			public bool ShowInToolbox => _config.ShowInToolbox;
			public IReadOnlyDictionary<string, string> Metadata => _config.Metadata;
			public IReadOnlyList<IToolDeployment> Deployments => _deployments;

			public Tool(ToolCollection collection, ToolDocument document, ToolConfig config, DateTime utcNow)
			{
				_collection = collection;
				_document = document;
				_config = config;
				_deployments = document.Deployments.ConvertAll(x => new ToolDeployment(this, _collection, x, utcNow));
			}

			public bool Authorize(AclAction action, ClaimsPrincipal principal)
				=> _config.Authorize(action, principal);

			public async Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, Stream stream, CancellationToken cancellationToken = default)
			{
				ToolDocument? document = await _collection.CreateDeploymentAsync(_document, _config, options, stream, cancellationToken);
				return _collection.CreateToolObject(document);
			}

			public async Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, HashedBlobRefValue target, CancellationToken cancellationToken = default)
			{
				ToolDocument? document = await _collection.CreateDeploymentAsync(_document, _config, options, target, cancellationToken);
				return _collection.CreateToolObject(document);
			}

			public IStorageBackend GetStorageBackend()
				=> _collection.CreateStorageBackend(_config);

			public IStorageNamespace GetStorageNamespace()
				=> _collection.GetStorageNamespace(_config);
		}

		class ToolDeployment : IToolDeployment
		{
			readonly Tool _tool;
			readonly ToolCollection _collection;
			readonly ToolDeploymentDocument _document;

			public ToolDeploymentId Id => _document.Id;
			public string Version => _document.Version;
			public ToolDeploymentState State { get; }
			public double Progress { get; }
			public DateTime? StartedAt => _document.StartedAt;
			public TimeSpan Duration => _document.Duration;
			public NamespaceId NamespaceId => _document.NamespaceId;
			public RefName RefName => _document.RefName;

			public IBlobRef<DirectoryNode> Content
				=> _collection.Open(_tool.Config, _document);

			public ToolDeployment(Tool tool, ToolCollection collection, ToolDeploymentDocument document, DateTime utcNow)
			{
				_tool = tool;
				_collection = collection;
				_document = document;

				if (document.BaseProgress >= 1.0)
				{
					State = ToolDeploymentState.Complete;
					Progress = 1.0;
				}
				else if (StartedAt == null)
				{
					State = ToolDeploymentState.Paused;
					Progress = document.BaseProgress;
				}
				else if (Duration > TimeSpan.Zero)
				{
					State = ToolDeploymentState.Active;
					Progress = Math.Clamp((utcNow - StartedAt.Value) / Duration, 0.0, 1.0);
				}
				else
				{
					State = ToolDeploymentState.Complete;
					Progress = 1.0;
				}
			}

			public Task<Stream> OpenZipStreamAsync(CancellationToken cancellationToken = default)
				=> _collection.GetDeploymentZipAsync(_tool.Config, _document, cancellationToken);

			public async Task<IToolDeployment?> UpdateAsync(ToolDeploymentState action, CancellationToken cancellationToken = default)
			{
				ToolDocument? newDocument = await _collection.UpdateDeploymentAsync(_tool.Document, Id, action, Progress, cancellationToken);
				Tool? newTool = _collection.CreateToolObject(newDocument);
				return newTool?.Deployments.FirstOrDefault(x => x.Id == Id);
			}
		}

		private readonly IServerInfo _serverInfo;
		private readonly IMongoCollection<ToolDocument> _tools;
		private readonly StorageService _storageService;
		private readonly FileObjectStoreFactory _fileObjectStoreFactory;
		private readonly IClock _clock;
		private readonly BundleCache _bundleCache;
		private readonly MemoryMappedFileCache _memoryMappedFileCache;
		private readonly IOptionsMonitor<ToolsConfig> _toolsConfig;
		private readonly IOptions<ToolsServerConfig> _toolsServerConfig;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ToolCollection(IMongoService mongoService, IServerInfo serverInfo, StorageService storageService, BundleCache bundleCache, MemoryMappedFileCache memoryMappedFileCache, FileObjectStoreFactory fileObjectStoreFactory, IClock clock, IOptionsMonitor<ToolsConfig> toolsConfig, IOptions<ToolsServerConfig> toolsServerConfig, ILogger<ToolCollection> logger)
		{
			_serverInfo = serverInfo;
			_tools = mongoService.GetCollection<ToolDocument>("Tools");
			_storageService = storageService;
			_fileObjectStoreFactory = fileObjectStoreFactory;
			_clock = clock;
			_toolsConfig = toolsConfig;
			_toolsServerConfig = toolsServerConfig;
			_bundleCache = bundleCache;
			_memoryMappedFileCache = memoryMappedFileCache;
			_logger = logger;
		}

		Tool? CreateToolObject(ToolDocument? document)
		{
			if (document == null)
			{
				return null;
			}

			ToolConfig? toolConfig;
			if (!_toolsConfig.CurrentValue.TryGetTool(document.Id, out toolConfig))
			{
				return null;
			}

			return new Tool(this, document, toolConfig, _clock.UtcNow);
		}

		/// <inheritdoc/>
		public async Task<ITool?> GetAsync(ToolId id, CancellationToken cancellationToken)
		{
			ToolsConfig toolsConfig = _toolsConfig.CurrentValue;

			ToolConfig? toolConfig;
			if (toolsConfig.TryGetTool(id, out toolConfig))
			{
				ToolDocument document = await FindOrAddDocumentAsync(toolConfig, cancellationToken);
				return new Tool(this, document, toolConfig, _clock.UtcNow);
			}

			BundledToolConfig? bundledToolConfig;
			if (_toolsServerConfig.Value.TryGetBundledTool(id, out bundledToolConfig))
			{
				ToolDocument document = CreateBundledToolDocument(bundledToolConfig);
				return new Tool(this, document, bundledToolConfig, _clock.UtcNow);
			}

			return null;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITool>> GetAllAsync(CancellationToken cancellationToken)
		{
			DateTime utcNow = _clock.UtcNow;
			ToolsConfig toolsConfig = _toolsConfig.CurrentValue;

			List<Tool> tools = new List<Tool>();
			foreach (ToolConfig toolConfig in toolsConfig.Tools)
			{
				ToolDocument document = await FindOrAddDocumentAsync(toolConfig, cancellationToken);
				tools.Add(new Tool(this, document, toolConfig, utcNow));
			}
			foreach (BundledToolConfig bundledToolConfig in _toolsServerConfig.Value.BundledTools)
			{
				ToolDocument document = CreateBundledToolDocument(bundledToolConfig);
				tools.Add(new Tool(this, document, bundledToolConfig, utcNow));
			}

			return tools;
		}

		async Task<ToolDocument> FindOrAddDocumentAsync(ToolConfig toolConfig, CancellationToken cancellationToken)
		{
			ToolDocument? tool;
			for (; ; )
			{
				tool = await _tools.Find(x => x.Id == toolConfig.Id).FirstOrDefaultAsync(cancellationToken);
				if (tool != null)
				{
					break;
				}

				tool = new ToolDocument(toolConfig.Id);
				if (await _tools.InsertOneIgnoreDuplicatesAsync(tool, cancellationToken))
				{
					break;
				}
			}
			return tool;
		}

		static ToolDocument CreateBundledToolDocument(BundledToolConfig bundledToolConfig)
		{
			ToolDocument tool = new ToolDocument(bundledToolConfig.Id);

			ToolDeploymentId deploymentId = GetDeploymentId(bundledToolConfig);

			ToolDeploymentDocument deployment = new ToolDeploymentDocument(deploymentId);
			deployment.StartedAt = DateTime.MinValue;
			deployment.Version = bundledToolConfig.Version;
			deployment.RefName = bundledToolConfig.RefName;
			tool.Deployments.Add(deployment);

			return tool;
		}

		static ToolDeploymentId GetDeploymentId(BundledToolConfig bundledToolConfig)
		{
			// Create a pseudo-random deployment id from the tool id and version
			IoHash hash = IoHash.Compute(Encoding.UTF8.GetBytes($"{bundledToolConfig.Id}/{bundledToolConfig.Version}"));

			Span<byte> bytes = stackalloc byte[IoHash.NumBytes];
			hash.CopyTo(bytes);

			// Set a valid timestamp to allow it to appear as an ObjectId
			BinaryPrimitives.WriteInt32BigEndian(bytes, 1446492960);
			return new ToolDeploymentId(new BinaryId(bytes));
		}

		async Task<ToolDocument?> CreateDeploymentAsync(ToolDocument tool, ToolConfig toolConfig, ToolDeploymentConfig options, Stream stream, CancellationToken cancellationToken)
		{
			ToolDeploymentId deploymentId = new ToolDeploymentId(BinaryIdUtils.CreateNew());

			IStorageNamespace client = _storageService.GetNamespace(toolConfig.NamespaceId);

			IHashedBlobRef<DirectoryNode> nodeRef;
			await using (IBlobWriter writer = client.CreateBlobWriter($"{tool.Id}/{deploymentId}", cancellationToken: cancellationToken))
			{
				DirectoryNode directoryNode = new DirectoryNode();
				await directoryNode.CopyFromZipStreamAsync(stream, writer, new ChunkingOptions(), cancellationToken: cancellationToken);
				nodeRef = await writer.WriteBlobAsync(directoryNode, cancellationToken: cancellationToken);
			}

			return await CreateDeploymentInternalAsync(tool, toolConfig, deploymentId, options, client, nodeRef, cancellationToken);
		}

		async Task<ToolDocument?> CreateDeploymentAsync(ToolDocument tool, ToolConfig toolConfig, ToolDeploymentConfig options, HashedBlobRefValue target, CancellationToken cancellationToken)
		{
			ToolDeploymentId deploymentId = new ToolDeploymentId(BinaryIdUtils.CreateNew());

			IStorageNamespace client = _storageService.GetNamespace(toolConfig.NamespaceId);
			return await CreateDeploymentInternalAsync(tool, toolConfig, deploymentId, options, client, client.CreateBlobRef(target), cancellationToken);
		}

		async Task<ToolDocument?> CreateDeploymentInternalAsync(ToolDocument tool, ToolConfig toolConfig, ToolDeploymentId deploymentId, ToolDeploymentConfig options, IStorageNamespace storageNamespace, IHashedBlobRef content, CancellationToken cancellationToken)
		{
			if (toolConfig is BundledToolConfig)
			{
				throw new InvalidOperationException("Cannot update the state of bundled tools.");
			}

			// Write a ref for the deployment so the blobs aren't GC'd
			RefName refName = new RefName($"{tool.Id}/{deploymentId}");
			await storageNamespace.AddRefAsync(refName, content, cancellationToken: cancellationToken);

			// Create the new deployment object
			ToolDeploymentDocument deployment = new ToolDeploymentDocument(deploymentId, options, toolConfig.NamespaceId, refName);

			// Start the deployment
			DateTime utcNow = _clock.UtcNow;
			if (!options.CreatePaused)
			{
				deployment.StartedAt = utcNow;
			}

			// Create the deployment
			ToolDocument? newTool = (ToolDocument)tool;
			for (; ; )
			{
				newTool = await TryAddDeploymentAsync(newTool, deployment, toolConfig.MaxDeployments, cancellationToken);
				if (newTool != null)
				{
					return newTool;
				}

				newTool = await FindOrAddDocumentAsync(toolConfig, cancellationToken);
				if (newTool == null)
				{
					return null;
				}
			}
		}

		async ValueTask<ToolDocument?> TryAddDeploymentAsync(ToolDocument tool, ToolDeploymentDocument deployment, int maxDeploymentCount, CancellationToken cancellationToken)
		{
			maxDeploymentCount = Math.Max(1, maxDeploymentCount);

			ToolDocument? newTool = tool;

			// If there are already a maximum number of deployments, remove the oldest one
			while (newTool.Deployments.Count >= maxDeploymentCount)
			{
				newTool = await UpdateAsync(newTool, Builders<ToolDocument>.Update.PopFirst(x => x.Deployments), cancellationToken);
				if (newTool == null)
				{
					return null;
				}

				ToolDeploymentDocument removeDeployment = tool.Deployments[0];
				IStorageNamespace client = _storageService.GetNamespace(removeDeployment.NamespaceId);
				await client.RemoveRefAsync(removeDeployment.RefName, cancellationToken);
			}

			// Add the new deployment
			return await UpdateAsync(newTool, Builders<ToolDocument>.Update.Push(x => x.Deployments, deployment), cancellationToken);
		}

		async Task<ToolDocument?> UpdateDeploymentAsync(ToolDocument tool, ToolDeploymentId deploymentId, ToolDeploymentState action, double currentProgress, CancellationToken cancellationToken)
		{
			int idx = tool.Deployments.FindIndex(x => x.Id == deploymentId);
			if (idx == -1)
			{
				return null;
			}

			ToolDeploymentDocument deployment = tool.Deployments[idx];
			switch (action)
			{
				case ToolDeploymentState.Complete:
					return await UpdateAsync(tool, Builders<ToolDocument>.Update.Set(x => x.Deployments[idx].BaseProgress, 1.0).Unset(x => x.Deployments[idx].StartedAt), cancellationToken);

				case ToolDeploymentState.Cancelled:
					List<ToolDeploymentDocument> newDeployments = tool.Deployments.Where(x => x != deployment).ToList();
					return await UpdateAsync(tool, Builders<ToolDocument>.Update.Set(x => x.Deployments, newDeployments), cancellationToken);

				case ToolDeploymentState.Paused:
					if (deployment.StartedAt == null)
					{
						return tool;
					}
					else
					{
						return await UpdateAsync(tool, Builders<ToolDocument>.Update.Set(x => x.Deployments[idx].BaseProgress, currentProgress).Set(x => x.Deployments[idx].StartedAt, null), cancellationToken);
					}

				case ToolDeploymentState.Active:
					if (deployment.StartedAt != null)
					{
						return tool;
					}
					else
					{
						return await UpdateAsync(tool, Builders<ToolDocument>.Update.Set(x => x.Deployments[idx].StartedAt, _clock.UtcNow), cancellationToken);
					}

				default:
					throw new ArgumentException("Invalid action for deployment", nameof(action));
			}
		}

		/// <summary>
		/// Gets the storage namespace containing data for a particular tool
		/// </summary>
		/// <param name="toolConfig">Identifier for the tool</param>
		/// <returns>Storage namespace for the data</returns>
		IStorageNamespace GetStorageNamespace(ToolConfig toolConfig)
		{
			if (toolConfig is BundledToolConfig bundledConfig)
			{
				return BundleStorageNamespace.CreateFromDirectory(DirectoryReference.Combine(_serverInfo.AppDir, bundledConfig.DataDir ?? "Tools"), _bundleCache, _memoryMappedFileCache, _logger);
			}
			else
			{
				return _storageService.GetNamespace(toolConfig.NamespaceId);
			}
		}

		/// <summary>
		/// Gets the storage backend containing data for a particular tool
		/// </summary>
		/// <param name="toolConfig">Identifier for the tool</param>
		/// <returns>Storage client for the data</returns>
		IStorageBackend CreateStorageBackend(ToolConfig toolConfig)
		{
			if (toolConfig is BundledToolConfig bundledConfig)
			{
				return new FileStorageBackend(_fileObjectStoreFactory.CreateStore(DirectoryReference.Combine(_serverInfo.AppDir, bundledConfig.DataDir ?? "Tools")), _logger);
			}
			else
			{
				return _storageService.CreateBackend(toolConfig.NamespaceId);
			}
		}

		IBlobRef<DirectoryNode> Open(ToolConfig tool, ToolDeploymentDocument deployment)
		{
			IStorageNamespace client = GetStorageNamespace(tool);
			return client.CreateBlobRef<DirectoryNode>(deployment.RefName, DateTime.UtcNow - TimeSpan.FromDays(2.0));
		}

		/// <summary>
		/// Opens a stream to the data for a particular deployment
		/// </summary>
		/// <param name="tool">Identifier for the tool</param>
		/// <param name="deployment">The deployment</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Stream for the data</returns>
		async Task<Stream> GetDeploymentZipAsync(ToolConfig tool, ToolDeploymentDocument deployment, CancellationToken cancellationToken)
		{
#pragma warning disable CA2000
			IStorageNamespace client = GetStorageNamespace(tool);

			IHashedBlobRef<DirectoryNode> nodeRef = await client.ReadRefAsync<DirectoryNode>(deployment.RefName, DateTime.UtcNow - TimeSpan.FromDays(2.0), cancellationToken: cancellationToken);
			return nodeRef.AsZipStream();
#pragma warning restore CA2000
		}

		async Task<ToolDocument> UpdateAsync(ToolDocument tool, UpdateDefinition<ToolDocument> update, CancellationToken cancellationToken)
		{
			update = update.Set(x => x.LastUpdateTime, new DateTime(Math.Max(tool.LastUpdateTime.Ticks + 1, DateTime.UtcNow.Ticks)));

			FilterDefinition<ToolDocument> filter = Builders<ToolDocument>.Filter.Eq(x => x.Id, tool.Id) & Builders<ToolDocument>.Filter.Eq(x => x.LastUpdateTime, tool.LastUpdateTime);
			return await _tools.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<ToolDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}
	}
}
