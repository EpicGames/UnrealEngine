// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Tools;
using HordeServer.Acls;
using HordeServer.Utilities;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Tools
{
	/// <summary>
	/// Options for configuring a tool
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class ToolConfig
	{
		/// <summary>
		/// The default namespace for tool data
		/// </summary>
		public static NamespaceId DefaultNamespaceId { get; } = new NamespaceId("horde-tools");

		/// <summary>
		/// Unique identifier for the tool
		/// </summary>
		[Required]
		public ToolId Id { get; set; }

		/// <summary>
		/// Name of the tool
		/// </summary>
		[Required]
		public string Name { get; set; }

		/// <summary>
		/// Description for the tool
		/// </summary>
		public string Description { get; set; }

		/// <summary>
		/// Category for the tool. Will cause the tool to be shown in a different tab in the dashboard.
		/// </summary>
		public string? Category { get; set; }

		/// <summary>
		/// Grouping key for different variations of the same tool. The dashboard will show these together.
		/// </summary>
		public string? Group { get; set; }

		/// <summary>
		/// Platforms for this tool. Takes the form of a NET RID (https://learn.microsoft.com/en-us/dotnet/core/rid-catalog).
		/// </summary>
		public List<string>? Platforms { get; set; }

		/// <summary>
		/// Whether this tool should be exposed for download on a public endpoint without authentication
		/// </summary>
		public bool Public { get; set; }

		/// <summary>
		/// Whether to show this tool for download in the UGS tools menu
		/// </summary>
		public bool ShowInUgs { get; set; }

		/// <summary>
		/// Whether to show this tool for download in the dashboard
		/// </summary>
		public bool ShowInDashboard { get; set; } = true;

		/// <summary>
		/// Whether to show this tool for download in Unreal Toolbox
		/// </summary>
		public bool ShowInToolbox { get; set; }

		/// <summary>
		/// Metadata for this tool
		/// </summary>
		public CaseInsensitiveDictionary<string> Metadata { get; set; } = new CaseInsensitiveDictionary<string>();

		/// <summary>
		/// Default namespace for new deployments of this tool
		/// </summary>
		public NamespaceId NamespaceId { get; set; } = DefaultNamespaceId;

		/// <summary>
		/// Maximum number of deployments to retain for this tool. When exceeded, the oldest deployments are removed.
		/// </summary>
		public int MaxDeployments { get; set; } = 5;

		/// <summary>
		/// Permissions for the tool
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Default constructor for serialization
		/// </summary>
		public ToolConfig()
		{
			Name = String.Empty;
			Description = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ToolConfig(ToolId id)
		{
			Id = id;
			Name = id.ToString();
			Description = String.Empty;
		}

		/// <summary>
		/// Called after the config has been read
		/// </summary>
		/// <param name="parentAcl">Parent ACL object</param>
		public void PostLoad(AclConfig parentAcl)
		{
			Acl.PostLoad(parentAcl, $"tool:{Id}", AclConfig.GetActions([typeof(ToolAclAction)]));
		}

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);
	}

	/// <summary>
	/// Configuration for a tool bundled alongsize the server
	/// </summary>
	public class BundledToolConfig : ToolConfig
	{
		/// <summary>
		/// Version string for the current tool data
		/// </summary>
		public string Version { get; set; } = "1.0";

		/// <summary>
		/// Ref name in the tools directory
		/// </summary>
		public RefName RefName { get; set; } = new RefName("default-ref");

		/// <summary>
		/// Directory containing blob data for this tool. If empty, the tools/{id} folder next to the server will be used.
		/// </summary>
		public string? DataDir { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundledToolConfig()
		{
			Public = true;
		}
	}
}
