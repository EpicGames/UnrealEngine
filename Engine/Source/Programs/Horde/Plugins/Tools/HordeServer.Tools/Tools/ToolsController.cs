// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using HordeServer.Storage;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.StaticFiles;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tools
{
	/// <summary>
	/// Controller for the /api/v1/tools endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class ToolsController : HordeControllerBase
	{
		readonly IToolCollection _toolCollection;

		/// <summary>
		/// Constructor
		/// </summary>
		public ToolsController(IToolCollection toolCollection)
		{
			_toolCollection = toolCollection;
		}

		/// <summary>
		/// Uploads blob data for a new tool deployment.
		/// </summary>
		/// <param name="id">Identifier of the tool to upload</param>
		/// <param name="request">Upload request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPost]
		[Route("/api/v1/tools/{id}/blobs")]
		public async Task<ActionResult<WriteBlobResponse>> WriteBlobAsync(ToolId id, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!tool.Authorize(ToolAclAction.UploadTool, User))
			{
				return Forbid(ToolAclAction.UploadTool, id);
			}

			if (String.IsNullOrEmpty(request.Prefix))
			{
				request.Prefix = id.ToString();
			}
			else
			{
				request.Prefix = $"{id}/{request.Prefix}";
			}

			IStorageBackend storageBackend = tool.GetStorageBackend();
			return await StorageController.WriteBlobAsync(storageBackend, null, request, cancellationToken);
		}

		/// <summary>
		/// Uploads blob data for a new tool deployment.
		/// </summary>
		/// <param name="id">Identifier of the tool to upload</param>
		/// <param name="locator">Locator for the blob</param>
		/// <param name="request">Upload request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPut]
		[Route("/api/v1/tools/{id}/blobs/{*locator}")]
		public async Task<ActionResult<WriteBlobResponse>> WriteBlobAsync(ToolId id, BlobLocator locator, WriteBlobRequest request, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!tool.Authorize(ToolAclAction.UploadTool, User))
			{
				return Forbid(ToolAclAction.UploadTool, id);
			}
			if (!locator.ToString().StartsWith($"{id}/", StringComparison.Ordinal))
			{
				return BadRequest("Tool blobs must start with tool id");
			}

			IStorageBackend storageBackend = tool.GetStorageBackend();
			return await StorageController.WriteBlobAsync(storageBackend, locator, request, cancellationToken);
		}

		/// <summary>
		/// Create a new deployment of the given tool.
		/// </summary>
		/// <returns>Information about the registered agent</returns>
		[HttpPost]
		[Route("/api/v1/tools/{id}/deployments")]
		public async Task<ActionResult<CreateToolDeploymentResponse>> CreateDeploymentAsync(ToolId id, [FromForm] ToolDeploymentConfig options, [FromForm] IFormFile file, CancellationToken cancellationToken)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!tool.Authorize(ToolAclAction.UploadTool, User))
			{
				return Forbid(ToolAclAction.UploadTool, id);
			}

			using (Stream stream = file.OpenReadStream())
			{
				tool = await tool.CreateDeploymentAsync(options, stream, cancellationToken);
				if (tool == null)
				{
					return NotFound(id);
				}
			}

			return new CreateToolDeploymentResponse(tool.Deployments[^1].Id);
		}

		/// <summary>
		/// Create a new deployment of the given tool.
		/// </summary>
		[HttpPost]
		[Route("/api/v2/tools/{id}/deployments")]
		public async Task<ActionResult<CreateToolDeploymentResponse>> CreateDeploymentAsync(ToolId id, CreateToolDeploymentRequest request, CancellationToken cancellationToken)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);

			if (tool == null)
			{
				return NotFound(id);
			}
			if (!tool.Authorize(ToolAclAction.UploadTool, User))
			{
				return Forbid(ToolAclAction.UploadTool, id);
			}

			ToolDeploymentConfig options = new ToolDeploymentConfig { Version = request.Version, Duration = TimeSpan.FromMinutes(request.Duration ?? 0.0), CreatePaused = request.CreatePaused ?? false };

			tool = await tool.CreateDeploymentAsync(options, request.Content, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}

			return new CreateToolDeploymentResponse(tool.Deployments[^1].Id);
		}

		/// <summary>
		/// Updates the state of an active deployment.
		/// </summary>
		[HttpPatch]
		[Route("/api/v1/tools/{id}/deployments/{deploymentId}")]
		public async Task<ActionResult> UpdateDeploymentAsync(ToolId id, ToolDeploymentId deploymentId, [FromBody] UpdateDeploymentRequest request)
		{
			ITool? tool = await _toolCollection.GetAsync(id, HttpContext.RequestAborted);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!tool.Authorize(ToolAclAction.UploadTool, User))
			{
				return Forbid(ToolAclAction.UploadTool, id);
			}

			IToolDeployment? deployment = tool.Deployments.FirstOrDefault(x => x.Id == deploymentId);
			if (deployment == null)
			{
				return NotFound(deploymentId);
			}

			if (request.State != null)
			{
				deployment = await deployment.UpdateAsync(request.State.Value, HttpContext.RequestAborted);
				if (request.State != ToolDeploymentState.Cancelled && deployment == null)
				{
					return NotFound(id, deploymentId);
				}
			}
			return Ok();
		}
	}

	/// <summary>
	/// Public methods available without authorization (or with very custom authorization)
	/// </summary>
	[ApiController]
	[TryAuthorize]
	[Tags("Tools")]
	public class PublicToolsController : HordeControllerBase
	{
		readonly IToolCollection _toolCollection;
		readonly IClock _clock;

		/// <summary>
		/// Constructor
		/// </summary>
		public PublicToolsController(IToolCollection toolCollection, IClock clock)
		{
			_toolCollection = toolCollection;
			_clock = clock;
		}

		/// <summary>
		/// Enumerates all the available tools.
		/// </summary>
		[HttpGet]
		[TryAuthorize]
		[Route("/api/v1/tools")]
		public async Task<ActionResult<GetToolsSummaryResponse>> GetToolsAsync()
		{
			IReadOnlyList<ITool> tools = await _toolCollection.GetAllAsync(HttpContext.RequestAborted);

			List<GetToolSummaryResponse> toolSummaryList = new List<GetToolSummaryResponse>();
			foreach (ITool tool in tools.OrderBy(x => x.Name, StringComparer.Ordinal))
			{
				if (AuthorizeDownload(tool))
				{
					toolSummaryList.Add(CreateGetToolSummaryResponse(tool));
				}
			}

			return new GetToolsSummaryResponse(toolSummaryList);
		}

		static GetToolSummaryResponse CreateGetToolSummaryResponse(ITool tool)
		{
			IToolDeployment? deployment = (tool.Deployments.Count == 0) ? null : tool.Deployments[^1];
			return new GetToolSummaryResponse(tool.Id, tool.Name, tool.Description, tool.Category, tool.Group, tool.Platforms?.ToList(), deployment?.Version, deployment?.Id, deployment?.State, deployment?.Progress, tool.Bundled, tool.ShowInUgs, tool.ShowInDashboard, tool.ShowInToolbox, new Dictionary<string, string>(tool.Metadata, StringComparer.OrdinalIgnoreCase));
		}

		/// <summary>
		/// Gets information about a particular tool
		/// </summary>
		/// <returns>Information about the registered agent</returns>
		[HttpGet]
		[Route("/api/v1/tools/{id}")]
		public async Task<ActionResult> GetToolAsync(ToolId id, GetToolAction action = GetToolAction.Info, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!AuthorizeDownload(tool))
			{
				return Forbid(ToolAclAction.DownloadTool, id);
			}

			if (action == GetToolAction.Info)
			{
				List<GetToolDeploymentResponse> deploymentResponses = new List<GetToolDeploymentResponse>();
				foreach (IToolDeployment deployment in tool.Deployments)
				{
					GetToolDeploymentResponse deploymentResponse = await GetDeploymentInfoResponseAsync(tool, deployment, cancellationToken);
					deploymentResponses.Add(deploymentResponse);
				}
				return Ok(CreateGetToolResponse(tool, deploymentResponses));
			}
			else
			{
				if (tool.Deployments.Count == 0)
				{
					return NotFound(LogEvent.Create(LogLevel.Error, "Tool {ToolId} does not currently have any deployments", id));
				}

				return await GetDeploymentResponseAsync(tool, tool.Deployments[^1], action, cancellationToken);
			}
		}

		static GetToolResponse CreateGetToolResponse(ITool tool, List<GetToolDeploymentResponse> deployments)
		{
			return new GetToolResponse(tool.Id, tool.Name, tool.Description, tool.Category, tool.Group, tool.Platforms?.ToList(), deployments, tool.Public, tool.Bundled, tool.ShowInUgs, tool.ShowInDashboard, tool.ShowInToolbox, new Dictionary<string, string>(tool.Metadata, StringComparer.OrdinalIgnoreCase));
		}

		/// <summary>
		/// Finds deployments of a particular tool.
		/// </summary>
		/// <param name="id">The tool identifier</param>
		/// <param name="phase">Value indicating the client's preference for deployment to receive.</param>
		/// <param name="action">Information about the returned deployment</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the registered agent</returns>
		[HttpGet]
		[Route("/api/v1/tools/{id}/deployments")]
		public async Task<ActionResult> FindDeploymentAsync(ToolId id, [FromQuery] double phase = 0.0, [FromQuery] GetToolAction action = GetToolAction.Info, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!AuthorizeDownload(tool))
			{
				return Forbid(ToolAclAction.DownloadTool, id);
			}

			IToolDeployment? deployment = tool.GetCurrentDeployment(phase, _clock.UtcNow);
			if (deployment == null)
			{
				return NotFound(LogEvent.Create(LogLevel.Error, "Tool {ToolId} does not currently have any deployments", id));
			}

			return await GetDeploymentResponseAsync(tool, deployment, action, cancellationToken);
		}

		/// <summary>
		/// Gets information about a specific tool deployment.
		/// </summary>
		/// <returns>Information about the registered agent</returns>
		[HttpGet]
		[Route("/api/v1/tools/{id}/deployments/{deploymentId}")]
		public async Task<ActionResult> GetDeploymentAsync(ToolId id, ToolDeploymentId deploymentId, [FromQuery] GetToolAction action = GetToolAction.Info, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!AuthorizeDownload(tool))
			{
				return Forbid(ToolAclAction.DownloadTool, id);
			}

			IToolDeployment? deployment = tool.Deployments.FirstOrDefault(x => x.Id == deploymentId);
			if (deployment == null)
			{
				return NotFound(id, deploymentId);
			}

			return await GetDeploymentResponseAsync(tool, deployment, action, cancellationToken);
		}

		private async Task<ActionResult> GetDeploymentResponseAsync(ITool tool, IToolDeployment deployment, GetToolAction action, CancellationToken cancellationToken)
		{
			if (action == GetToolAction.Info)
			{
				GetToolDeploymentResponse response = await GetDeploymentInfoResponseAsync(tool, deployment, cancellationToken);
				return Ok(response);
			}

			IStorageNamespace client = tool.GetStorageNamespace();
			IHashedBlobRef<DirectoryNode> nodeRef = await client.ReadRefAsync<DirectoryNode>(deployment.RefName, DateTime.UtcNow - TimeSpan.FromDays(2.0), cancellationToken: cancellationToken);

			// If we weren't specifically asked for a zip, see if this download is a single file. If it is, allow downloading it directory.
			if (action != GetToolAction.Zip)
			{
				DirectoryNode node = await nodeRef.ReadBlobAsync(cancellationToken);
				if (node.Directories.Count == 0 && node.Files.Count == 1)
				{
					FileEntry entry = node.Files.First();

					string? contentType;
					if (!new FileExtensionContentTypeProvider().TryGetContentType(entry.Name.ToString(), out contentType))
					{
						contentType = "application/octet-stream";
					}

					Response.Headers.ContentLength = entry.Length;

					Stream fileStream = entry.OpenAsStream();
					return new FileStreamResult(fileStream, contentType) { FileDownloadName = entry.Name.ToString() };
				}
			}

			Stream stream = nodeRef.AsZipStream();
			return new FileStreamResult(stream, "application/zip") { FileDownloadName = $"{tool.Id}-{deployment.Version}.zip" };
		}

		static async Task<GetToolDeploymentResponse> GetDeploymentInfoResponseAsync(ITool tool, IToolDeployment deployment, CancellationToken cancellationToken)
		{
			IStorageNamespace client = tool.GetStorageNamespace();
			IBlobRef rootHandle = await client.ReadRefAsync(deployment.RefName, cancellationToken: cancellationToken);

			return new GetToolDeploymentResponse(deployment.Id, deployment.Version, deployment.State, deployment.Progress, deployment.StartedAt, deployment.Duration, deployment.RefName, rootHandle.GetLocator());
		}

		/// <summary>
		/// Retrieves blobs for a particular tool
		/// </summary>
		/// <param name="id">Identifier of the tool to retrieve</param>
		/// <param name="locator">The blob locator</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v1/tools/{id}/blobs/{*locator}")]
		public async Task<ActionResult> ReadToolBlobAsync(ToolId id, BlobLocator locator, CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(id, cancellationToken);
			if (tool == null)
			{
				return NotFound(id);
			}
			if (!AuthorizeDownload(tool))
			{
				return Forbid(ToolAclAction.DownloadTool, id);
			}

			if (!locator.WithinFolder(tool.Id.Id.Text) && !tool.Bundled)
			{
				return BadRequest("Invalid blob id for tool");
			}

			IStorageBackend storageBackend = tool.GetStorageBackend();
			return await StorageController.ReadBlobInternalAsync(storageBackend, locator, Request.Headers, cancellationToken);
		}

		bool AuthorizeDownload(ITool tool)
		{
			if (!tool.Public)
			{
				if (User.Identity == null || !User.Identity.IsAuthenticated)
				{
					return false;
				}
				if (!tool.Authorize(ToolAclAction.DownloadTool, User))
				{
					return false;
				}
			}
			return true;
		}
	}
}
