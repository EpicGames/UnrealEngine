// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Acls;
using HordeServer.Tools;
using HordeServer.Utilities;

namespace HordeServer
{
	class ToolsAclModifier : IDefaultAclModifier
	{
		/// <inheritdoc/>
		public void Apply(DefaultAclBuilder acl)
		{
			acl.AddCustomRole(HordeClaims.UploadToolsClaim, new[] { ToolAclAction.UploadTool });
		}
	}
}
