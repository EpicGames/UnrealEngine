// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Tools;
using HordeServer.Plugins;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Tools
{
	/// <summary>
	/// Configuration for the tools system
	/// </summary>
	public class ToolsConfig : IPluginConfig
	{
		/// <summary>
		/// Tool configurations
		/// </summary>
		public List<ToolConfig> Tools { get; set; } = new List<ToolConfig>();

		private readonly Dictionary<ToolId, ToolConfig> _toolLookup = new Dictionary<ToolId, ToolConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			_toolLookup.Clear();
			foreach (ToolConfig tool in Tools)
			{
				_toolLookup.Add(tool.Id, tool);
				tool.PostLoad(configOptions.ParentAcl);
			}
		}

		/// <summary>
		/// Attempts to get configuration for a tool from this object
		/// </summary>
		/// <param name="toolId">The tool identifier</param>
		/// <param name="config">Configuration for the tool</param>
		/// <returns>True if the tool configuration was found</returns>
		public bool TryGetTool(ToolId toolId, [NotNullWhen(true)] out ToolConfig? config) => _toolLookup.TryGetValue(toolId, out config);
	}
}
