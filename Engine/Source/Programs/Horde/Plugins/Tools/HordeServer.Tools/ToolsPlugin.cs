// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde.Tools;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Tools;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the storage plugin
	/// </summary>
	[Plugin("Tools", GlobalConfigType = typeof(ToolsConfig), ServerConfigType = typeof(ToolsServerConfig))]
	public class ToolsPlugin : IPluginStartup
	{
		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{ }

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<IDefaultAclModifier, ToolsAclModifier>();

			services.AddSingleton<IToolCollection, ToolCollection>();
			services.AddCommandsFromAssembly(Assembly.GetExecutingAssembly());
		}
	}

	/// <summary>
	/// Helper methods for tools config
	/// </summary>
	public static class ToolsPluginExtensions
	{
		/// <summary>
		/// Configures the tools plugin
		/// </summary>
		public static void AddToolsConfig(this IDictionary<PluginName, IPluginConfig> dictionary, ToolsConfig toolsConfig)
			=> dictionary[new PluginName("Tools")] = toolsConfig;
	}
}
