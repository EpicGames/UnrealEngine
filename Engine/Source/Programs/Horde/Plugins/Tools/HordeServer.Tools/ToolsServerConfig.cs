// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Tools;
using HordeServer.Plugins;
using HordeServer.Tools;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Server configuration for bundled tools
	/// </summary>
	public class ToolsServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Tools bundled along with the server. Data for each tool can be produced using the 'bundle create' command, and should be stored in the Tools directory.
		/// </summary>
		public List<BundledToolConfig> BundledTools { get; set; } = new List<BundledToolConfig>();

		/// <summary>
		/// Attempts to get a bundled tool with the given id
		/// </summary>
		/// <param name="toolId">The tool id</param>
		/// <param name="bundledToolConfig">Configuration for the bundled tool</param>
		/// <returns>True if the tool was found</returns>
		public bool TryGetBundledTool(ToolId toolId, [NotNullWhen(true)] out BundledToolConfig? bundledToolConfig)
		{
			bundledToolConfig = BundledTools.FirstOrDefault(x => x.Id == toolId);
			return bundledToolConfig != null;
		}
	}
}
