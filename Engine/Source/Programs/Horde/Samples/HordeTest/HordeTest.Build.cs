// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class HordeTest : ModuleRules
{
	public HordeTest(ReadOnlyTargetRules target) : base(target)
	{
		PrivateDependencyModuleNames.AddRange([
			"Core",
			"Horde",
			"DesktopPlatform",
			"HTTP"
		]);
	}
}