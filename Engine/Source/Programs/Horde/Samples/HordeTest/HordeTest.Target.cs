// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1050:Declare types in namespaces", Justification = "<Pending>")]
public class HordeTestTarget : TargetRules
{
	public HordeTestTarget(TargetInfo target) : base(target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "HordeTest";
		
		bCompileICU = false;

		// Lean and mean
		bBuildDeveloperTools = false;

		// Compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// Logs are still useful to print the results
		bUseLoggingInShipping = true;

		// Make a console application under Windows, so entry point is main() everywhere
		bIsBuildingConsoleApplication = true;

		WindowsPlatform.bPixProfilingEnabled = false;
	}
}
