// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Storage/StorageClient.h"
#include "Storage/Clients/BundleStorageClient.h"
#include "Storage/Clients/MemoryStorageClient.h"
#include "Storage/Clients/FileStorageClient.h"
#include "Storage/Nodes/ChunkNode.h"
#include "Storage/Nodes/DirectoryNode.h"
#include "Storage/BlobWriter.h"
#include <iostream>
#include <fstream>
#include <thread>
#include <assert.h>
#include "Memory/SharedBuffer.h"
#include "Storage/ChunkedBufferWriter.h"
#include "Compute/ComputeBuffer.h"
#include "Compute/ComputeChannel.h"
#include "Compute/ComputeTransport.h"
#include "Compute/ComputeSocket.h"

extern TCHAR GInternalProjectName[64] = { 0, };
extern const TCHAR* GForeignEngineDir = nullptr;

int main(int ArgC, const char* ArgV[])
{
	if (ArgC == 5 && FCStringAnsi::Stricmp(ArgV[1], "bundle") == 0 && FCStringAnsi::Stricmp(ArgV[2], "create") == 0)
	{
		extern int CreateBundle(const std::filesystem::path & InputDir, const std::filesystem::path & OutputFile);
		return CreateBundle(ArgV[3], ArgV[4]);
	}

	if (ArgC == 5 && FCStringAnsi::Stricmp(ArgV[1], "bundle") == 0 && FCStringAnsi::Stricmp(ArgV[2], "extract") == 0)
	{
		extern int ExtractBundle(const std::filesystem::path & InputFile, const std::filesystem::path & OutputDir);
		return ExtractBundle(ArgV[3], ArgV[4]);
	}

	if (ArgC == 3 && FCStringAnsi::Stricmp(ArgV[1], "compute") == 0 && FCStringAnsi::Stricmp(ArgV[2], "test") == 0)
	{
		extern int RunComputeTests();
		return RunComputeTests();
	}

	if (ArgC == 3 && FCStringAnsi::Stricmp(ArgV[1], "compute") == 0 && FCStringAnsi::Stricmp(ArgV[2], "worker") == 0)
	{
		extern int RunComputeWorker();
		return RunComputeWorker();
	}

	printf("Arguments:\n");
	printf("\n");
	printf("  bundle create <InputDir> <OutputFile>\n");
	printf("     - Archives a folder to a set of bundles\n");
	printf("\n");
	printf("  bundle extract <InputFile> <OutputDir>\n");
	printf("     - Extracts a set of bundles to a folder\n");
	printf("\n");
	printf("  compute test\n");
	printf("     - Runs some basic tests on primitives in the compute system\n");
	printf("\n");
	printf("  compute worker\n");
	printf("     - Runs a worker app compatible with the RemoteClient sample in the Horde sln\n");
	return 1;
}

// --------------------------------------------------------------------------------------------------------
// 'bundle create' command

FBlobHandleWithHash CreateFromStream(std::ifstream& Stream, FBlobWriter& Writer, int64& OutLength, FIoHash& OutStreamHash)
{
	OutLength = 0;

	FChunkNodeWriter ChunkWriter(Writer);

	char ReadBuffer[4096];
	while (!Stream.eof())
	{
		Stream.read(ReadBuffer, sizeof(ReadBuffer));
		
		int64 ReadSize = Stream.gcount();
		if (ReadSize == 0)
		{
			break;
		}
		OutLength += ReadSize;

		ChunkWriter.Write(FMemoryView(ReadBuffer, ReadSize));
	}

	return ChunkWriter.Flush(OutStreamHash);
}

FFileEntry CreateFromFile(const std::filesystem::path& Path, FBlobWriter& Writer)
{
	std::ifstream InputStream(Path, std::ios::binary);
	check(InputStream);
	
	int64 Length = 0;
	FIoHash StreamHash;
	FBlobHandleWithHash Target = CreateFromStream(InputStream, Writer, Length, StreamHash);

	return FFileEntry(Target, FUtf8String(Path.filename().string().c_str()), EFileEntryFlags::None, Length, StreamHash, FSharedBufferView());
}

FDirectoryEntry CreateFromDirectory(const std::filesystem::path& Path, FBlobWriter& Writer)
{
	printf("Found %ws\n", Path.c_str());
	int64 Length = 0;

	FDirectoryNode DirectoryNode;
	for (const std::filesystem::directory_entry Entry : std::filesystem::directory_iterator(Path))
	{
		if (Entry.is_directory())
		{
			FDirectoryEntry NewEntry = CreateFromDirectory(Entry.path(), Writer);
			FUtf8String Name = NewEntry.Name;
			DirectoryNode.NameToDirectory.Add(Name, MoveTemp(NewEntry));
			Length += NewEntry.Length;
		}
		else if (Entry.is_regular_file())
		{
			FFileEntry NewEntry = CreateFromFile(Entry.path(), Writer);
			FUtf8String Name = NewEntry.Name;
			DirectoryNode.NameToFile.Add(Name, MoveTemp(NewEntry));
			Length += NewEntry.Length;
		}
	}

	FBlobHandle DirectoryHandle = DirectoryNode.Write(Writer);

	return FDirectoryEntry(DirectoryHandle, FIoHash(), FUtf8String(Path.filename().string().c_str()), Length);
}

int CreateBundle(const std::filesystem::path& InputDir, const std::filesystem::path& OutputFile)
{
	TSharedRef<FFileStorageClient> FileStorage = MakeShared<FFileStorageClient>(OutputFile.parent_path());
	TSharedRef<FBundleStorageClient> Storage = MakeShared<FBundleStorageClient>(FileStorage);

	TUniquePtr<FBlobWriter> Writer = Storage->CreateWriter("");
	FDirectoryEntry RootEntry = CreateFromDirectory(InputDir, *Writer.Get());
	Writer->Flush();

	FFileStorageClient::WriteRefToFile(OutputFile, RootEntry.Target->GetLocator());
	return 0;
}

// --------------------------------------------------------------------------------------------------------
// 'bundle extract' command

void ExtractFile(const std::filesystem::path& Path, const FBlobHandle& Handle)
{
	printf("Extracting %ws\n", Path.c_str());

	std::ofstream Stream(Path, std::ios::binary);

	FChunkNodeReader ChunkReader(Handle);
	while (ChunkReader)
	{
		FMemoryView View = ChunkReader.GetBuffer();
		Stream.write((const char*)View.GetData(), View.GetSize());
		ChunkReader.Advance(View.GetSize());
	}
}

void ExtractDirectory(const std::filesystem::path& Path, const FBlobHandle& Handle)
{
	printf("Extracting %ws\n", Path.c_str());
	std::filesystem::create_directory(Path);

	FBlob Blob = Handle->Read();
	FDirectoryNode Directory = FDirectoryNode::Read(Blob);

	for (TMap<FUtf8String, FDirectoryEntry>::TConstIterator Iter(Directory.NameToDirectory); Iter; ++Iter)
	{
		ExtractDirectory(Path / std::string((const char*)*Iter.Key()), Iter.Value().Target);
	}

	for (TMap<FUtf8String, FFileEntry>::TConstIterator Iter(Directory.NameToFile); Iter; ++Iter)
	{
		ExtractFile(Path / std::string((const char*)*Iter.Key()), Iter.Value().Target.Handle);
	}
}

int ExtractBundle(const std::filesystem::path& InputFile, const std::filesystem::path& OutputDir)
{
	TSharedRef<FFileStorageClient> FileStorage = MakeShared<FFileStorageClient>(InputFile.parent_path());
	TSharedRef<FBundleStorageClient> Storage = MakeShared<FBundleStorageClient>(FileStorage);

	FBlobHandle Handle = Storage->CreateHandle(FFileStorageClient::ReadRefFromFile(InputFile));
	ExtractDirectory(OutputDir, Handle);

	return 0;
}

// --------------------------------------------------------------------------------------------------------
// 'compute test' command

int RunComputeTests()
{
	void ComputeBufferTest();
	ComputeBufferTest();
	std::cout << "ComputeBufferTest ok" << std::endl;

	void ComputeSocketTest();
	ComputeSocketTest();
	std::cout << "ComputeSocketTest ok" << std::endl;

	return 0;
}

void ComputeBufferTest()
{
	char TestData[] = "hello world!";

	FComputeBuffer Buffer;
	verify(Buffer.CreateNew(FComputeBuffer::FParams()));

	FComputeBufferWriter Writer = Buffer.CreateWriter();
	unsigned char* WriteBuffer = Writer.WaitToWrite(sizeof(TestData));
	memcpy(WriteBuffer, TestData, sizeof(TestData));
	Writer.AdvanceWritePosition(sizeof(TestData));

	FComputeBufferReader Reader = Buffer.CreateReader();
	const unsigned char* ReadBuffer = Reader.WaitToRead(sizeof(TestData));
	check(memcmp(ReadBuffer, TestData, sizeof(TestData)) == 0);
	check(!Reader.IsComplete());

	Reader.AdvanceReadPosition(sizeof(TestData));
	check(!Reader.IsComplete());

	Writer.MarkComplete();
	check(Reader.IsComplete());
}

template<size_t TestDataSize> void CheckChannelSendRecv(TSharedPtr<FComputeChannel>& SendChannel, TSharedPtr<FComputeChannel>& RecvChannel, const char(&TestData)[TestDataSize])
{
	SendChannel->Send(TestData, TestDataSize);

	char RecvTestData[TestDataSize];
	verify(RecvChannel->Recv(RecvTestData, TestDataSize) == TestDataSize);

	verify(memcmp(TestData, RecvTestData, TestDataSize) == 0);
}

void ComputeSocketTest()
{
	// Buffers for transferring between client and server
	FComputeBuffer ClientToServerBuffer;
	verify(ClientToServerBuffer.CreateNew(FComputeBuffer::FParams()));

	FComputeBuffer ServerToClientBuffer;
	verify(ServerToClientBuffer.CreateNew(FComputeBuffer::FParams()));

	// Client transport
	TUniquePtr<FComputeSocket> ClientSocket = CreateComputeSocket(TUniquePtr<FComputeTransport>(new FBufferTransport(ClientToServerBuffer.CreateWriter(), ServerToClientBuffer.CreateReader())), EComputeSocketEndpoint::Local);
	TSharedPtr<FComputeChannel> ClientChannel1 = ClientSocket->CreateChannel(1);
	TSharedPtr<FComputeChannel> ClientChannel2 = ClientSocket->CreateChannel(2);

	// Server socket
	TUniquePtr<FComputeSocket> ServerSocket = CreateComputeSocket(TUniquePtr<FComputeTransport>(new FBufferTransport(ServerToClientBuffer.CreateWriter(), ClientToServerBuffer.CreateReader())), EComputeSocketEndpoint::Remote);
	TSharedPtr<FComputeChannel> ServerChannel1 = ServerSocket->CreateChannel(1);
	TSharedPtr<FComputeChannel> ServerChannel2 = ServerSocket->CreateChannel(2);

	// Close the original buffers now that the sockets are set up
	ClientToServerBuffer.Close();
	ServerToClientBuffer.Close();

	// Send data over channel 1
	CheckChannelSendRecv(ClientChannel1, ServerChannel1, "Channel 1: Client -> Server");

	CheckChannelSendRecv(ServerChannel2, ClientChannel2, "Channel 2: Server -> Client");
	CheckChannelSendRecv(ClientChannel2, ServerChannel2, "Channel 2: Client -> Server");

	// Send data over channel 1 again
	CheckChannelSendRecv(ClientChannel1, ServerChannel1, "Channel 1: Client -> Server (2)");
	CheckChannelSendRecv(ServerChannel1, ClientChannel1, "Channel 1: Server -> Client (2)");
}

// --------------------------------------------------------------------------------------------------------
// 'compute worker' command

int RunComputeWorker()
{
	const int ChannelId = 100;

	FWorkerComputeSocket Socket;
	if (!Socket.Open())
	{
		std::cout << "Environment variable not set correctly" << std::endl;
		return 1;
	}

	TSharedPtr<FComputeChannel> Channel = Socket.CreateChannel(ChannelId);
	if (!Channel->IsValid())
	{
		std::cout << "Unable to create channel to initiator" << std::endl;
		return 1;
	}

	std::cout << "Connected to initiator" << std::endl;

	size_t Length = 0;
	char Buffer[4];

	Buffer[0] = 0;
	Channel->Send(Buffer, 1); // Send a dummy one-byte message to let the remote know we're listening

	for (;;)
	{
		size_t RecvLength = Channel->Recv(Buffer + Length, sizeof(Buffer) - Length);
		if (RecvLength == 0)
		{
			return 0;
		}

		Length += RecvLength;

		if (Length >= 4)
		{
			std::cout << "Read value " << *(int*)Buffer << std::endl;
			Length = 0;
		}
	}
}
