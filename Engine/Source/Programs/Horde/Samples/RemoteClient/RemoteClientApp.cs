// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers.Binary;
using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Clients;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace RemoteClient
{
	class ClientAppOptions
	{
		[CommandLine("-Server=")]
		public string? Server { get; set; }

		[CommandLine("-Oidc=")]
		public string? OidcProvider { get; set; }

		[CommandLine("-Condition=")]
		public string? Condition { get; set; }

		[CommandLine]
		public bool InProc { get; set; }

		[CommandLine("-Cpp")]
		public bool UseCppWorker { get; set; }
	}

	class ClientApp
	{
		static FileReference CurrentAssemblyFile { get; } = new FileReference(Assembly.GetExecutingAssembly().Location);
		static DirectoryReference ClientSourceDir { get; } = DirectoryReference.Combine(CurrentAssemblyFile.Directory, "../../..");

		static async Task Main(string[] args)
		{
			ILogger logger = new DefaultConsoleLogger(LogLevel.Trace);

			// Parse the command line arguments
			ClientAppOptions options = new ClientAppOptions();
			CommandLineArguments arguments = new CommandLineArguments(args);
			arguments.ApplyTo(options);
			arguments.CheckAllArgumentsUsed(logger);

			// Create a DI container that can create and authenticate Horde HTTP clients for us
			ServiceCollection services = new ServiceCollection();
			if (options.Server == null)
			{
				DirectoryReference sandboxDir = DirectoryReference.Combine(new FileReference(Assembly.GetExecutingAssembly().Location).Directory, "Sandbox");
				services.AddSingleton<IComputeClient>(sp => new LocalComputeClient(2000, sandboxDir, options.InProc, new PrefixLogger("[REMOTE]", logger)));
			}
			else
			{
				services.AddHorde(x => x.ServerUrl = new Uri(options.Server));
				services.AddSingleton<IComputeClient, ServerComputeClient>();
			}

			// Create the client to handle our requests
			await using ServiceProvider serviceProvider = services.BuildServiceProvider();
			IComputeClient client = serviceProvider.GetRequiredService<IComputeClient>();

			// Allocate a worker
			Requirements? requirements = null;
			if (options.Condition != null)
			{
				requirements = new Requirements(Condition.Parse(options.Condition));
			}

			await using IComputeLease? lease = await client.TryAssignWorkerAsync(new ClusterId("default"), requirements, null, null, new PrefixLogger("[CLIENT]", logger));
			if (lease == null)
			{
				logger.LogInformation("Unable to connect to remote");
				return;
			}

			// Run the worker
			if (options.UseCppWorker)
			{
				FileReference remoteServerFile = FileReference.Combine(ClientSourceDir, "../RemoteWorkerCpp/bin/RemoteServerCpp.exe");
				await RunRemoteAsync(lease, remoteServerFile.Directory, "RemoteWorkerCpp.exe", new List<string>(), logger);
			}
			else
			{
				FileReference remoteServerFile = FileReference.Combine(ClientSourceDir, "../RemoteWorker", CurrentAssemblyFile.Directory.MakeRelativeTo(ClientSourceDir), "RemoteWorker.dll");
				await RunRemoteAsync(lease, remoteServerFile.Directory, @"C:\Program Files\dotnet\dotnet.exe", new List<string> { remoteServerFile.GetFileName() }, logger);
			}
		}

		const int PrimaryChannelId = 0;
		const int BackgroundChannelId = 1;
		const int ChildProcessChannelId = 100;

		static async Task RunRemoteAsync(IComputeLease lease, DirectoryReference uploadDir, string executable, List<string> arguments, ILogger logger)
		{
			// Create a message channel on channel id 0. The Horde Agent always listens on this channel for requests.
			using (AgentMessageChannel channel = lease.Socket.CreateAgentMessageChannel(PrimaryChannelId, 4 * 1024 * 1024))
			{
				await channel.WaitForAttachAsync();

				// Fork another message loop. We'll use this to run an XOR task in the background.
				using AgentMessageChannel backgroundChannel = lease.Socket.CreateAgentMessageChannel(BackgroundChannelId, 4 * 1024 * 1024);
				await using BackgroundTask otherChannelTask = BackgroundTask.StartNew(ctx => RunBackgroundXorAsync(backgroundChannel));
				await channel.ForkAsync(BackgroundChannelId, 4 * 1024 * 1024, default);

				// Upload the sandbox to the primary channel.
				BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(logger);

				await using (IBlobWriter writer = storage.CreateBlobWriter())
				{
					IHashedBlobRef<DirectoryNode> sandbox = await writer.AddFilesAsync(uploadDir);
					await writer.FlushAsync();
					await channel.UploadFilesAsync("", sandbox.GetLocator(), storage.Backend);
				}

				// Run the task remotely in the background and echo the output to the console
				using ComputeChannel childProcessChannel = lease.Socket.CreateChannel(ChildProcessChannelId);
				await using BackgroundTask tickTask = BackgroundTask.StartNew(ctx => WriteNumbersAsync(childProcessChannel, lease.Socket.Logger, ctx));

				await using (AgentManagedProcess process = await channel.ExecuteAsync(executable, arguments, null, null, ExecuteProcessFlags.None))
				{
					string? line;
					while ((line = await process.ReadLineAsync()) != null)
					{
						lease.Socket.Logger.LogInformation("{Line}", line);
					}
				}
			}
			await lease.CloseAsync();
		}
		
		static async Task WriteNumbersAsync(ComputeChannel channel, ILogger logger, CancellationToken cancellationToken)
		{
			// Wait until the remote sends a message indicating that it's ready
			if (!await channel.Reader.WaitToReadAsync(1, cancellationToken))
			{
				throw new NotImplementedException();
			}

			// Write data to the child process channel. The remote server will echo them back to us as it receives them, then exit when the channel is complete/closed.
			byte[] buffer = new byte[4];
			for (int idx = 0; idx < 3; idx++)
			{
				cancellationToken.ThrowIfCancellationRequested();
				logger.LogInformation("Writing value: {Value}", idx);
				BinaryPrimitives.WriteInt32LittleEndian(buffer, idx);
				await channel.Writer.WriteAsync(buffer, cancellationToken);
				await Task.Delay(1000, cancellationToken);
			}

			channel.MarkComplete();
		}

		static async Task RunBackgroundXorAsync(AgentMessageChannel channel)
		{
			await channel.WaitForAttachAsync();

			byte[] dataToXor = new byte[] { 1, 2, 3, 4, 5 };
			await channel.SendXorRequestAsync(dataToXor, 123);

			using AgentMessage response = await channel.ReceiveAsync(AgentMessageType.XorResponse);
			for (int idx = 0; idx < dataToXor.Length; idx++)
			{
				if (response.Data.Span[idx] != (byte)(dataToXor[idx] ^ 123))
				{
					throw new InvalidOperationException();
				}
			}

			await channel.CloseAsync();
		}
	}
}