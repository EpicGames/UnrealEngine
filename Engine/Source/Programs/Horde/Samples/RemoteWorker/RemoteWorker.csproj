<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <ImplicitUsings>enable</ImplicitUsings>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>

</Project>
