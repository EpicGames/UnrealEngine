// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers.Binary;
using EpicGames.Horde.Compute;

namespace RemoteWorker
{
	class WorkerApp
	{
		static async Task Main()
		{
			const int ChannelId = 100;

			using WorkerComputeSocket socket = WorkerComputeSocket.Open();

			using ComputeChannel channel = socket.CreateChannel(ChannelId);
			Console.WriteLine("Connected to initiator");

			byte[] data = new byte[4];
			await channel.SendAsync(data.AsMemory(0, 1)); // Let the remote know we're ready

			while(await channel.TryRecvMessageAsync(data))
			{
				int value = BinaryPrimitives.ReadInt32LittleEndian(data);
				Console.WriteLine("Read value {0}", value);
			}

			Console.WriteLine("Exiting worker");
		}
	}
}