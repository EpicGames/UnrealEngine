#!/bin/bash
# Run tests (run from Dockerfile)

export UE_DOTNET_VERSION=net10.0

# Start Redis and MongoDB in the background for tests to use
redis-server --save "" --appendonly no --daemonize yes --databases 3000 || exit 1
mongod --noauth --quiet --fork --dbpath /tmp/mongodb --logpath /tmp/mongod.log || exit 1

testProjects=(
	"Source/Programs/Shared/EpicGames.BuildGraph.Tests/EpicGames.BuildGraph.Tests.csproj"
	"Source/Programs/Shared/EpicGames.Core.Tests/EpicGames.Core.Tests.csproj"
	"Source/Programs/Shared/EpicGames.Horde.Tests/EpicGames.Horde.Tests.csproj"
	"Source/Programs/Shared/EpicGames.IoHash.Tests/EpicGames.IoHash.Tests.csproj"
	"Source/Programs/Shared/EpicGames.Redis.Tests/EpicGames.Redis.Tests.csproj"
	"Source/Programs/Shared/EpicGames.Serialization.Tests/EpicGames.Serialization.Tests.csproj"
	"Source/Programs/Horde/HordeServer.Tests/HordeServer.Tests.csproj"
	"Source/Programs/Horde/Plugins/Analytics/HordeServer.Analytics.Tests/HordeServer.Analytics.Tests.csproj"
	"Source/Programs/Horde/Plugins/Build/HordeServer.Build.Tests/HordeServer.Build.Tests.csproj"
	"Source/Programs/Horde/Plugins/Compute/HordeServer.Compute.Tests/HordeServer.Compute.Tests.csproj"
	"Source/Programs/Horde/Plugins/Experimental/HordeServer.Experimental.Tests/HordeServer.Experimental.Tests.csproj"
	"Source/Programs/Horde/Plugins/Secrets/HordeServer.Secrets.Tests/HordeServer.Secrets.Tests.csproj"
	"Source/Programs/Horde/Plugins/Storage/HordeServer.Storage.Tests/HordeServer.Storage.Tests.csproj"
	"Source/Programs/Horde/Plugins/Tools/HordeServer.Tools.Tests/HordeServer.Tools.Tests.csproj"
)

for csProj in "${testProjects[@]}"; do
	filename="${csProj##*/}"
	args=("test")
	if [ "$code_coverage" = "true" ]; then
		args=(dotcover cover-dotnet --output=/tmp/${filename}.dcvr --filters="+:EpicGames*;+:Horde*;-:*.Tests" -- test)
	fi
	dotnet "${args[@]}" "$csProj" --blame-hang-timeout 5m --blame-hang-dump-type mini --logger 'console;verbosity=normal' || exit 1
done

mkdir /tmp/dotcover-report
touch /tmp/empty
if [ "$code_coverage" = "true" ]; then
	dotnet dotcover merge --source=/tmp/*.dcvr --output=/tmp/dotcover-merged.dcvr
	dotnet dotcover report --source=/tmp/dotcover-merged.dcvr --output=/tmp/dotcover-report/report.html:/tmp/dotcover-report/report.json --reportType=HTML,JSON
	zip -r /tmp/dotcover-report/dotcover-report.zip /tmp/dotcover-report
fi