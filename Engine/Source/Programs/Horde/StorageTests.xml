<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../Engine/Build/Graph/Schema.xsd" >

	<Option Name="PreflightChange" DefaultValue="" Description="Preflight changelist number"/>
	
	<!-- Small Chunks -->
	<Agent Name="Small Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Small Chunks" Produces="#SmallChunks">
			<RandomData Size="1024" Count="2000000" Different="true" Tag="#SmallChunks"/>
		</Node>
	</Agent>
	<Agent Name="Small Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Small Chunks" Requires="#SmallChunks">
		</Node>
	</Agent>

	<!-- Medium Chunks -->
	<Agent Name="Medium Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Medium Chunks" Produces="#MediumChunks" AllowRetry="False">
			<RandomData Size="1048576" Count="20000" Different="true" Tag="#MediumChunks"/>
		</Node>
	</Agent>
	<Agent Name="Medium Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Medium Chunks" Requires="#MediumChunks">
		</Node>
	</Agent>

	<!-- Large Chunks -->
	<Agent Name="Large Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Large Chunks" Produces="#LargeChunks">
			<RandomData Size="1073741824" Count="20" Different="true" Tag="#LargeChunks"/>
		</Node>
	</Agent>
	<Agent Name="Large Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Large Chunks" Requires="#LargeChunks">
			<Log Message="Output Files:" Files="#LargeChunks"/>
		</Node>
	</Agent>
	
	<!-- Small Identical Chunks -->
	<Agent Name="Small Identical Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Small Identical Chunks" Produces="#SmallIdenticalChunks">
			<RandomData Size="1024" Count="2000000" Different="false" Tag="#SmallIdenticalChunks"/>
		</Node>
	</Agent>
	<Agent Name="Small Identical Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Small Identical Chunks" Requires="#SmallIdenticalChunks">
		</Node>
	</Agent>

	<!-- Medium Identical Chunks -->
	<Agent Name="Medium Identical Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Medium Identical Chunks" Produces="#MediumIdenticalChunks">
			<RandomData Size="1048576" Count="20000" Different="false" Tag="#MediumIdenticalChunks"/>
		</Node>
	</Agent>
	<Agent Name="Medium Identical Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Medium Identical Chunks" Requires="#MediumIdenticalChunks">
		</Node>
	</Agent>

	<!-- Large Identical Chunks -->
	<Agent Name="Large Identical Agent A" Type="CompileWin64;Win64">
		<Node Name="Create Large Identical Chunks" Produces="#LargeIdenticalChunks">
			<RandomData Size="1073741824" Count="20" Different="true" Tag="#LargeIdenticalChunks"/>
		</Node>
	</Agent>
	<Agent Name="Large Identical Agent B" Type="CompileWin64;Win64">
		<Node Name="Read Large Identical Chunks" Requires="#LargeIdenticalChunks">
			<Log Message="Output Files:" Files="#LargeIdenticalChunks"/>
		</Node>
	</Agent>

	<!-- Aggregate for the whole build -->
	<Aggregate Name="Storage Tests" Requires="Read Small Chunks;Read Medium Chunks;Read Large Chunks;Read Small Identical Chunks;Read Medium Identical Chunks;Read Large Identical Chunks"/>

</BuildGraph>
