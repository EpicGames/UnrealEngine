// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Server;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace CreateReleaseNotes
{
	internal class Program
	{
		static readonly string[] s_paths =
		[
			"Engine/Source/Programs/Horde/...",
			"Engine/Source/Programs/Shared/...",
			"Engine/Restricted/NotForLicensees/Source/Programs/Horde/...",
		];
		
		private static readonly string[] s_ignoreFilter =
		[
			// Ignore these as they have (almost) no impact on Horde
			"Engine/Source/Programs/Shared/EpicGames.UBA/...",
			"Engine/Source/Programs/Shared/EpicGames.Build/...",
			"Engine/Source/Programs/Shared/EpicGames.UHT/...",
			"Engine/Source/Programs/Shared/UnrealBuildTool.Tests/...",
			"Engine/Source/Programs/Horde/Docs/ReleaseNotes.md",
			
			// TODO: Flag as low risk changes
			"Engine/Source/Programs/Shared/EpicGames.AspNet/...",
			"Engine/Source/Programs/Shared/EpicGames.Core/...",
			"Engine/Source/Programs/Shared/EpicGames.Serialization/...",
		];
		
		static readonly string[] s_serverFilter =
		[
			"...",
			"-/Engine/Source/Programs/Horde/HordeDashboard/...",
			"-/Engine/Source/Programs/Horde/HordeAgent/..."
		];

		static readonly string[] s_agentFilter =
		[
			"...",
			"-/Engine/Source/Programs/Horde/HordeDashboard/...",
			"-/Engine/Source/Programs/Horde/HordeServer*/...",
			"-/Engine/Source/Programs/Horde/Plugins/*/HordeServer*/..."
		];

		static readonly string[] s_dashboardFilter =
		[
			"-/...",
			"/Engine/Source/Programs/Horde/HordeDashboard/..."
		];
		
		static readonly string[] s_restrictedFilter =
		[
			"-/...",
			"/Engine/Restricted/..."
		];

		record ChangeInfo(int Number, string Description, string? JiraTicket);

		class Options
		{
			[CommandLine("-Server=", Required = true)]
			public Uri Server { get; set; } = null!;

			[CommandLine("-Change=")]
			public int? Change { get; set; }

			[CommandLine("-ServerChange=")]
			public int? ServerChange { get; set; }

			[CommandLine("-ClientChange=")]
			public int? AgentChange { get; set; }

			[CommandLine("-DashboardChange=")]
			public int? DashboardChange { get; set; }
		}

		static async Task<int> Main(string[] args)
		{
			DefaultConsoleLogger defaultLogger = new DefaultConsoleLogger();

			CommandLineArguments arguments = new CommandLineArguments(args);
			Options options = arguments.ApplyTo<Options>(defaultLogger);
			arguments.CheckAllArgumentsUsed(defaultLogger);

			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.AddLogging(builder =>
			{
				builder.AddEpicDefault();
				builder.AddFilter("System.Net.Http.HttpClient", LogLevel.Warning);
			});
			serviceCollection.AddHorde(x => x.ServerUrl = options.Server);

			await using ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider();
			ILogger logger = serviceProvider.GetRequiredService<ILogger<Program>>();

			IHordeClient horde = serviceProvider.GetRequiredService<IHordeClient>();
			HordeHttpClient hordeHttpClient = horde.CreateHttpClient();

			int serverChange = options.ServerChange ?? options.Change ?? 0;
			int agentChange = options.AgentChange ?? options.Change ?? 0;

			if (serverChange == 0 || agentChange == 0)
			{
				logger.LogInformation("Querying Horde for currently deployed version...");

				GetServerInfoResponse info = await hordeHttpClient.GetServerInfoAsync();
				if (serverChange == 0 && !TryParseChange(info.ServerVersion, out serverChange))
				{
					logger.LogError("Unexpected server version format: {Version}", info.ServerVersion);
					return 1;
				}
				if (agentChange == 0 && !TryParseChange(info.AgentVersion, out agentChange))
				{
					logger.LogError("Unexpected agent version format: {Version}", info.AgentVersion);
					return 1;
				}
			}

			int dashboardChange = options.DashboardChange ?? serverChange;

			logger.LogInformation("");
			logger.LogInformation("Current versions:");
			logger.LogInformation("  Server:    {ServerChange}", serverChange);
			logger.LogInformation("  Agent:     {AgentChange}", agentChange);
			logger.LogInformation("  Dashboard: {DashboardChange}", dashboardChange);
			logger.LogInformation("");

			int change = Math.Min(serverChange, agentChange);
			logger.LogInformation("Finding changes after CL {MinChange}...", change);
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(serviceProvider.GetRequiredService<ILogger<PerforceConnection>>());

			List<ChangeInfo> parsedChanges = new List<ChangeInfo>();

			InfoRecord perforceInfo = await perforce.GetInfoAsync(InfoOptions.None);

			List<string> paths = s_paths.Select(x => $"//{perforceInfo.ClientName}/{x}").ToList();

			List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.LongOutput, null, change + 1, -1, ChangeStatus.Submitted, null, paths);
			changes.RemoveAll(x => x.User.Equals("buildmachine", StringComparison.OrdinalIgnoreCase));
			changes = changes.DistinctBy(x => x.Number).ToList();

			if (changes.Count == 0)
			{
				logger.LogError("No changes found. Verify server and client CL numbers.");
				return 1;
			}

			List<DescribeRecord> describeRecords = await perforce.DescribeAsync(changes.Select(x => x.Number).ToArray());

			HashSet<int> changeNumbers = [];
			FileFilter ignoreFilter = new (s_ignoreFilter);
			FileFilter serverFilter = new (s_serverFilter);
			FileFilter agentFilter = new (s_agentFilter);
			FileFilter dashboardFilter = new (s_dashboardFilter);
			FileFilter restrictedFilter = new (s_restrictedFilter);

			logger.LogInformation("");
			foreach (DescribeRecord describeRecord in describeRecords.OrderBy(x => x.Number))
			{
				string description = describeRecord.Description;

				Match jiraMatch = Regex.Match(description, @"^\s*#jira ([a-zA-Z]+-[0-9]+.*)$");
				string? jiraTicket = jiraMatch.Success ? jiraMatch.Groups[1].Value : null;

				description = Regex.Replace(description, @"^[a-zA-Z]+:\s*", "");
				description = Regex.Replace(description, @"^(?:\[[A-Za-z0-9_-]+\]\s*)+(?::\s*)?", "");
				description = Regex.Replace(description, @"^\s*#.*$", "", RegexOptions.Multiline);
				description = Regex.Replace(description, @"\n\s*", "\n");
				description = Regex.Replace(description, @"^\s*\*\s*", "");
				description = description.Trim();

				string logDescription = Regex.Replace(description, @"\s+", " ");

				const int MaxDescriptionLength = 80;
				if (logDescription.Length > MaxDescriptionLength)
				{
					logDescription = logDescription.Substring(0, MaxDescriptionLength);
				}

				bool affectsServer = false;
				bool affectsAgent = false;
				bool affectsDashboard = false;
				bool affectsRestricted = false;
				bool ignoreChangelist = false;

				foreach (DescribeFileRecord fileRecord in describeRecord.Files)
				{
					string path = fileRecord.DepotFile;
					path = Regex.Replace(path, @"^//[^/]+/[^/]+/", "");
					
					if (ignoreFilter.Matches(path))
					{
						ignoreChangelist = true;
						break;
					}
					
					if (restrictedFilter.Matches(path))
					{
						affectsServer |= Path.GetExtension(path) is ".cs" or ".csproj";
						affectsDashboard |= Path.GetExtension(path) is ".ts" or ".tsx";
						affectsRestricted = true;
					}
					else
					{
						if (describeRecord.Number > serverChange && serverFilter.Matches(path))
						{
							affectsServer = true;
						}
						if (describeRecord.Number > agentChange && agentFilter.Matches(path))
						{
							affectsAgent = true;
						}
						if (describeRecord.Number > dashboardChange && dashboardFilter.Matches(path))
						{
							affectsDashboard = true;
						}
					}
				}
				
				if (ignoreChangelist)
				{
					continue;
				}
				
				if (!Regex.IsMatch(describeRecord.Description, @"^\s*#rnx\s*$", RegexOptions.Multiline) && !affectsRestricted)
				{
					parsedChanges.Add(new ChangeInfo(describeRecord.Number, description, jiraTicket));
				}

				if (affectsServer || affectsAgent || affectsDashboard || affectsRestricted)
				{
					string types = (affectsServer? "S" : " ") + (affectsAgent ? "A" : " ") + (affectsDashboard? "D" : " ") + (affectsRestricted? "R" : " ");
					logger.LogInformation("[{Types}] {Change} {Author,-20} {Restricted}{Description}", types, describeRecord.Number, describeRecord.User.ToLower(), affectsRestricted ? "Restricted - " : "", logDescription);
				}
			}

			if (parsedChanges.Count == 0)
			{
				logger.LogError("No changes to deploy.");
				return 1;
			}

			DateTime now = DateTime.Now;

			List<string> lines = new List<string>();
			lines.Add($"## {now.Year}-{now.Month:00}-{now.Day:00}");
			lines.Add("");
			foreach (ChangeInfo parsedChange in parsedChanges.OrderByDescending(x => x.Number))
			{
				lines.Add($"* {parsedChange.Description.Replace("\n", "\n  ", StringComparison.Ordinal)} ({parsedChange.Number})");
			}
			lines.Add("");

			string releaseNotesFile = $"//{perforceInfo.ClientName}/Engine/Source/Programs/Horde/Docs/ReleaseNotes.md";
			await perforce.TryRevertAsync(-1, null, RevertOptions.None, releaseNotesFile);
			await perforce.EditAsync(-1, null, EditOptions.None, releaseNotesFile);

			WhereRecord? where = await perforce.WhereAsync(releaseNotesFile).FirstOrDefaultAsync();
			if (where == null || String.IsNullOrEmpty(where.Path))
			{
				logger.LogError("Unable to get local path for file {File}", releaseNotesFile);
				return 1;
			}

			FileReference file = new FileReference(where.Path);

			List<string> fileLines = new List<string>(await FileReference.ReadAllLinesAsync(file));

			int insertIdx = 0;
			while (insertIdx < fileLines.Count && Regex.IsMatch(fileLines[insertIdx], @"^(# .*|\s*)$"))
			{
				insertIdx++;
			}

			fileLines.InsertRange(insertIdx, lines);

			logger.LogInformation("");
			logger.LogInformation("Writing {File}", file);
			await FileReference.WriteAllLinesAsync(file, fileLines);
			return 0;
		}

		static bool TryParseChange(string? name, out int version)
		{
			if (name == null)
			{
				version = 0;
				return false;
			}

			Match match = Regex.Match(name, @"-(\d+)$");
			if (!match.Success)
			{
				version = 0;
				return false;
			}
			else
			{
				version = Int32.Parse(match.Groups[1].Value);
				return true;
			}
		}
	}
}
