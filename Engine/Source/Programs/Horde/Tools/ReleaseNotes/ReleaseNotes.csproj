<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <ImplicitUsings>enable</ImplicitUsings>
    <PublishSingleFile>true</PublishSingleFile>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Logging.Console" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
  </ItemGroup>

</Project>
