<?xml version="1.0" encoding="utf-8"?>
<configuration>
	<packageSources>
		<add key="nuget.org" value="https://api.nuget.org/v3/index.json" protocolVersion="3" />
		<add key="Azure DevOps Public Feed" value="https://pkgs.dev.azure.com/dnceng/public/_packaging/dotnet-tools/nuget/v3/index.json" protocolVersion="3" />
	</packageSources>
</configuration>
